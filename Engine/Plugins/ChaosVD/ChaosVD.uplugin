{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "0.1",
	"FriendlyName": "Chaos Visual Debugger",
	"Description": "Enables support for Visual debugging of Chaos Physics simulations",
	"Category": "Physics",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": true,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "ChaosVD",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosVDBlueprint",
			"Type": "RuntimeAndProgram",
			"LoadingPhase": "Default"
		},
		{
			"Name": "ChaosVDBuiltInExtensions",
			"Type": "EditorAndProgram",
			"LoadingPhase": "Default"
		}
	],
	"Plugins": [
		{
			"Name": "EditorDataStorage",
			"Enabled": true
		},
		{
			"Name": "EditorDataStorageFeatures",
			"Enabled": true
		},
		{
			"Name": "GeometryProcessing",
			"Enabled": true
		}
	],
	"SupportedPrograms": ["ChaosVisualDebugger"]
}