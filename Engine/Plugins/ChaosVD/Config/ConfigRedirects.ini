[SectionNameRemap]
/Script/ChaosVD.ChaosVDGenericDebugDrawSettings Instance=/Script/ChaosVDBuiltInExtensions.ChaosVDGenericDebugDrawSettings Instance
/Script/ChaosVD.ChaosVDAccelerationStructureVisualizationSettings Instance=/Script/ChaosVDBuiltInExtensions.ChaosVDAccelerationStructureVisualizationSettings Instance