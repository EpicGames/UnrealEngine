[SectionsToSave]
bCanSaveAllSections=true

[/Script/ChaosVD.ChaosVDParticleVisualizationColorSettings]
ParticleColorMode=State
ColorsByShapeType=(SimpleTypeColor=(B=115,G=158,R=0,A=255),ConvexColor=(B=66,G=228,R=240,A=255),HeightFieldColor=(B=233,G=180,R=86,A=255),TriangleMeshColor=(B=0,G=94,R=213,A=255),LevelSetColor=(B=167,G=121,R=204,A=255))
ColorsByParticleState=(DynamicColor=(B=98,G=246,R=253,A=255),SleepingColor=(B=80,G=92,R=231,A=255),KinematicColor=(B=178,G=114,R=0,A=255),StaticColor=(B=156,G=159,R=150,A=255))
ColorsByClientServer=(ServerBaseColor=(B=80,G=92,R=231,A=255),ClientBaseColor=(B=178,G=114,R=0,A=255))