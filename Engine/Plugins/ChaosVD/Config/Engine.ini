[/Script/ChaosVD.ChaosVDCoreSettings]
QueryOnlyMeshesMaterial=/ChaosVD/Materials/ChaosVDQueryOnlyMesh.ChaosVDQueryOnlyMesh
SimOnlyMeshesMaterial=/ChaosVD/Materials/ChasoVDSimMesh.ChasoVDSimMesh
InstancedMeshesMaterial=/ChaosVD/Materials/ChaosVDInstanced.ChaosVDInstanced
InstancedMeshesQueryOnlyMaterial=/ChaosVD/Materials/ChaosVDInstancedQueryOnly.ChaosVDInstancedQueryOnly
SkySphereActorClass=/ChaosVD/BP_CVD_SkySphere.BP_CVD_SkySphere_C
AmbientCubeMapTexture=/ChaosVD/Textures/DaylightAmbientCubemap.DaylightAmbientCubemap
BoxMesh=/ChaosVD/Meshes/Box.Box
SphereMesh=/ChaosVD/Meshes/Sphere.Sphere