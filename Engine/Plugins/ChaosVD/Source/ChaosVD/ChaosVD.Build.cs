// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosVD : ModuleRules
{
	public ChaosVD(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core"
			});

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"ApplicationCore",
				"CoreUObject",
				"Engine",
				"RenderCore",
				"ChaosVDRuntime",
				"TraceServices",
				"ChaosVDData",
				"DesktopPlatform",
				"Projects",
				"InputCore",
				"EditorFramework",
				"UnrealEd",
				"ToolMenus",
				"Slate",
				"SlateCore", 
				"OutputLog",
				"WorkspaceMenuStructure",
				"TraceAnalysis", 
				"TraceBasedDebuggers",
				"TraceBasedDebuggersAnalysis",
				"TraceInsights",
				"TraceLog", 
				"GeometryFramework",
				"GeometryCore",
				"MeshConversion",
				"MeshDescription",
				"StaticMeshDescription", 
				"DynamicMesh",
				"SceneOutliner",
				"TypedElementRuntime",
				"TypedElementFramework",
				"StatusBar",
				"SubobjectEditor",
				"SubobjectDataInterface",
				"PropertyEditor",
				"CommonMenuExtensions", 
				"EditorWidgets", 
				"TedsOutliner",
				"ChaosSolverEngine",
				"Sockets",
			}
			);

		CppCompileWarningSettings.UnsafeTypeCastWarningLevel = WarningLevel.Error;

		SetupModulePhysicsSupport(Target);
	}
}
