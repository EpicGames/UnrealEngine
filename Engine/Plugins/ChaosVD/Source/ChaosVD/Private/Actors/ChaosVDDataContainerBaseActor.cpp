// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/ChaosVDDataContainerBaseActor.h"

#include "ChaosVDModule.h"
#include "ChaosVDSceneCompositionReport.h"
#include "Components/ChaosVDSolverDataComponent.h"
#include "ExtensionsSystem/ChaosVDExtensionsManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDDataContainerBaseActor)

namespace Chaos::VD::Test::SceneObjectTypes
{
	FName SolverID = FName("SolverID");
}

AChaosVDDataContainerBaseActor::AChaosVDDataContainerBaseActor()
{
	PrimaryActorTick.bCanEverTick = false;
}

AChaosVDDataContainerBaseActor::~AChaosVDDataContainerBaseActor()
{
	FChaosVDExtensionsManager::Get().OnExtensionRegistered().RemoveAll(this);
}

void AChaosVDDataContainerBaseActor::UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData)
{
	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->UpdateFromNewGameFrameData(InGameFrameData);
		}
	}
}

void AChaosVDDataContainerBaseActor::UpdateFromNewSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->UpdateFromSolverFrameData(InSolverFrameData);
		}
	}
}

void AChaosVDDataContainerBaseActor::UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData)
{
	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->UpdateFromNewSolverStageData(InSolverFrameData, InSolverFrameStageData);
		}
	}
}

void AChaosVDDataContainerBaseActor::Destroyed()
{
	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->ClearData();
		}
	}

	Super::Destroyed();
}

void AChaosVDDataContainerBaseActor::PostActorCreated()
{
	Super::PostActorCreated();
	
	FChaosVDExtensionsManager::Get().EnumerateExtensions([this](const TSharedRef<FChaosVDExtension>& Extension)
	{		
		HandlePostInitializationExtensionRegistered(Extension);

		return true;
	});

	FChaosVDExtensionsManager::Get().OnExtensionRegistered().AddUObject(this, &AChaosVDDataContainerBaseActor::HandlePostInitializationExtensionRegistered);
}

void AChaosVDDataContainerBaseActor::SetIsTemporarilyHiddenInEditor(bool bIsHidden)
{
	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->SetVisibility(!bIsHidden);
		}
	}

	Super::SetIsTemporarilyHiddenInEditor(bIsHidden);
}

void AChaosVDDataContainerBaseActor::AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData)
{
	int32& CurrentCount = OutStateTestData.ObjectsCountByType.FindOrAdd(Chaos::VD::Test::SceneObjectTypes::SolverID);
	CurrentCount++;

	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->AppendSceneCompositionTestData(OutStateTestData);
		}
	}
}

void AChaosVDDataContainerBaseActor::HandlePostInitializationExtensionRegistered(const TSharedRef<FChaosVDExtension>& NewExtension)
{
	TConstArrayView<TSubclassOf<UActorComponent>> CustomDataComponentsClasses = NewExtension->GetSolverDataComponentsClasses();

	for (const TSubclassOf<UActorComponent>& ComponentClass : CustomDataComponentsClasses)
	{
		UActorComponent* DataComponent = NewObject<UActorComponent>(this, ComponentClass);
		AddOwnedComponent(DataComponent);
		DataComponent->RegisterComponent();

		if (UChaosVDSolverDataComponent* SolverDataComponent = Cast<UChaosVDSolverDataComponent>(DataComponent))
		{
			SolverDataComponent->SetSolverID(SolverDataID);
		}
	}
}

void AChaosVDDataContainerBaseActor::HandleWorldStreamingLocationUpdated(const FVector& InLocation)
{
	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->HandleWorldStreamingLocationUpdated(InLocation);
		}
	}
}

void AChaosVDDataContainerBaseActor::SetSolverID(int32 InSolverID)
{
	SolverDataID = InSolverID;
	
	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->SetSolverID(InSolverID);
		}
	}
}

void AChaosVDDataContainerBaseActor::UpdateVisibility(bool bIsVisible)
{
	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if (Component)
		{
			Component->SetVisibility(bIsVisible);
		}
	}
}
