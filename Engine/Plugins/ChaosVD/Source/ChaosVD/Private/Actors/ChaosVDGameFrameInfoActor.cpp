// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/ChaosVDGameFrameInfoActor.h"

#include "Components/ChaosVDSceneQueryDataComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDGameFrameInfoActor)

AChaosVDGameFrameInfoActor::AChaosVDGameFrameInfoActor()
{
	PrimaryActorTick.bCanEverTick = false;

}
