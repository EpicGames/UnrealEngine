// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Actors/ChaosVDDataContainerBaseActor.h"
#include "GameFramework/Actor.h"
#include "UObject/ObjectPtr.h"

#include "ChaosVDGameFrameInfoActor.generated.h"

class UChaosVDGenericDebugDrawDataComponent;
class UChaosVDSceneQueryDataComponent;

/** Actor that contains game frame related data */
UCLASS(NotBlueprintable, NotPlaceable)
class AChaosVDGameFrameInfoActor : public AChaosVDDataContainerBaseActor
{
	GENERATED_BODY()

public:
	AChaosVDGameFrameInfoActor();
};
