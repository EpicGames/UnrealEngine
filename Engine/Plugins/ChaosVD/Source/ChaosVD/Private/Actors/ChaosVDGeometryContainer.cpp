// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGeometryContainer.h"
#include "Components/MeshComponent.h"
#include "Containers/Array.h"
#include "Misc/MemStack.h"
#include "Misc/ScopedSlowTask.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDGeometryContainer)

AChaosVDGeometryContainer::AChaosVDGeometryContainer()
{
	PrimaryActorTick.bCanEverTick = false;
}

void AChaosVDGeometryContainer::CleanUp()
{
	// We can have ten of thousands of components, so the InlineComponents arrays will not be better than a normal array.
	FMemMark Mark(FMemStack::Get());
	TArray<UMeshComponent*, TMemStackAllocator<>> ComponentsToDestroy;

	GetComponents<UMeshComponent>(ComponentsToDestroy);

	constexpr float AmountOfWork = 1.0f;
	const float PercentagePerElement = 1.0f / static_cast<float>(ComponentsToDestroy.Num());

	FScopedSlowTask CleaningDataSlowTask(AmountOfWork, NSLOCTEXT("ChaosVisualDebugger", "CleaningupGeometryData", "Clearing Geoemtry Data ..."));
	CleaningDataSlowTask.MakeDialog();

	for (UMeshComponent* Component : ComponentsToDestroy)
	{
		if (Component)
		{
			Component->DestroyComponent();
		}

		CleaningDataSlowTask.EnterProgressFrame(PercentagePerElement);
	}
}
