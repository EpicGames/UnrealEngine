// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "UObject/ObjectPtr.h"
#include "ChaosVDScene.h"
#include "Templates/SharedPointer.h"

#include "ChaosVDGeometryContainer.generated.h"

/** Actor that contains Static Mesh Components used to visualize the geometry we generated from the recorded data */
UCLASS()
class AChaosVDGeometryContainer : public AActor
{
	GENERATED_BODY()

public:
	AChaosVDGeometryContainer();

	virtual void SetScene(TWeakPtr<FChaosVDScene> InScene)
	{
		SceneWeakPtr = InScene;
	}

	TSharedPtr<FChaosVDScene> GetScene() const 
	{
		return SceneWeakPtr.Pin();
	}

	void CleanUp();

protected:
	TWeakPtr<FChaosVDScene> SceneWeakPtr;
};