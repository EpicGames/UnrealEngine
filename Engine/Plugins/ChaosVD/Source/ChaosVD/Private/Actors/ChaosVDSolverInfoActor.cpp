// Copyright Epic Games, Inc. All Rights Reserved.

#include "Actors/ChaosVDSolverInfoActor.h"

#include "ChaosVDModule.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDScene.h"

#include "EditorActorFolders.h"
#include "Components/ChaosVDAdditionalGTDataRouterComponent.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "Components/ChaosVDParticleExtraDataComponent.h"
#include "Components/ChaosVDSceneQueryDataComponent.h"
#include "Components/ChaosVDSolverCharacterGroundConstraintDataComponent.h"
#include "Components/ChaosVDSolverCollisionDataComponent.h"
#include "Components/ChaosVDSolverJointConstraintDataComponent.h"
#include "Elements/Framework/TypedElementSelectionSet.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSolverInfoActor)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

AChaosVDSolverInfoActor::AChaosVDSolverInfoActor()
{
	CollisionDataComponent = CreateDefaultSubobject<UChaosVDSolverCollisionDataComponent>(TEXT("SolverCollisionDataComponent"));
	ParticleDataComponent = CreateDefaultSubobject<UChaosVDParticleDataComponent>(TEXT("ParticleDataComponent"));
	JointsDataComponent = CreateDefaultSubobject<UChaosVDSolverJointConstraintDataComponent>(TEXT("JointDataComponent"));
	CharacterGroundConstraintDataComponent = CreateDefaultSubobject<UChaosVDSolverCharacterGroundConstraintDataComponent>(TEXT("CharacterGroundConstraintDataComponent"));
	SceneQueryDataComponent = CreateDefaultSubobject<UChaosVDSceneQueryDataComponent>(TEXT("ChaosVDSceneQueryDataComponent"));
	GTDataReRouteComponent = CreateDefaultSubobject<UChaosVDAdditionalGTDataRouterComponent>(TEXT("ChaosVDAdditionalGTDataRouterComponent"));
	ParticleExtraDataComponent = CreateDefaultSubobject<UChaosVDParticleExtraDataComponent>(TEXT("ParticleExtraDataComponent"));

	bIsServer = false;
}

void AChaosVDSolverInfoActor::SetSolverName(const FName& InSolverName)
{
	SolverName = InSolverName;
	SetActorLabel(TEXT("Solver Data Container | ") + InSolverName.ToString());
}

void AChaosVDSolverInfoActor::FindAndUpdateFromCorrectGameFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	// Note: This is needed because the original implementation of game thread data playback don't support different sync modes
	// therefore until that is re-implemented in a future version (UE-277464), we work around it by re-routing update game frame data callbacks from
	// each solver data actor, using solver timing data as a starting point

	TSharedPtr<FChaosVDScene> CVDScene = SceneWeakPtr.Pin();
	TSharedPtr<FChaosVDRecording> Recording = CVDScene ? CVDScene->GetLoadedRecording() : nullptr;

	const FChaosVDGameFrameData* GameFrameData = Recording ? Recording->GetGameFrameData_AssumesLocked(Recording->GetLowestGameFrameNumberAtCycle_AssumesLocked(InSolverFrameData.FrameCycle)) : nullptr;
	if (!GameFrameData)
	{
		return;
	}

	{
		FScopedGameFrameDataReRouting ScopedGTDataReRoute(this);
		UpdateFromNewGameFrameData(*GameFrameData);
	}
}

void AChaosVDSolverInfoActor::UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData)
{
	if (!ensure(bInternallyReRoutingGameFrameData))
	{
		return;
	}

	Super::UpdateFromNewGameFrameData(InGameFrameData);
}

void AChaosVDSolverInfoActor::UpdateFromNewSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	if (!EnumHasAnyFlags(InSolverFrameData.GetAttributes(), EChaosVDSolverFrameAttributes::HasGTDataToReRoute))
	{
		FindAndUpdateFromCorrectGameFrameData(InSolverFrameData);
	}

	SetSimulationTransform(InSolverFrameData.SimulationTransform);
	Super::UpdateFromNewSolverFrameData(InSolverFrameData);
}

void AChaosVDSolverInfoActor::SetScene(TWeakPtr<FChaosVDScene> InScene)
{
	Super::SetScene(InScene);

	if (TSharedPtr<FChaosVDScene> ScenePtr = InScene.Pin())
	{
		RegisterSelectionSetObject(ScenePtr->GetElementSelectionSet());

		ScenePtr->OnSolverVisibilityUpdated().Broadcast(SolverDataID, IsVisible());
	}

	TInlineComponentArray<UChaosVDSolverDataComponent*> SolverDataComponents(this);
	GetComponents(SolverDataComponents);

	for (UChaosVDSolverDataComponent* Component : SolverDataComponents)
	{
		if(!Component)
		{
			continue;
		}

		Component->SetScene(InScene);
	}
}

FName AChaosVDSolverInfoActor::GetCustomIconName() const
{
	static FName SolverIconName = FName("SolverIcon");
	return SolverIconName;
}

TSharedPtr<FChaosVDSceneParticle> AChaosVDSolverInfoActor::GetParticleInstance(int32 ParticleID) const
{
	if (ensure(ParticleDataComponent))
	{
		return ParticleDataComponent->GetParticleInstanceByID(ParticleID);
	}

	return nullptr;
}

bool AChaosVDSolverInfoActor::IsVisible() const
{
	return !IsTemporarilyHiddenInEditor();
}

#if WITH_EDITOR
void AChaosVDSolverInfoActor::SetIsTemporarilyHiddenInEditor(bool bIsHidden)
{
	if (IsTemporarilyHiddenInEditor() != bIsHidden)
	{
		if (const TSharedPtr<FChaosVDScene> CVDScene = SceneWeakPtr.Pin())
		{
			CVDScene->OnSolverVisibilityUpdated().Broadcast(SolverDataID, !bIsHidden);
		}
	}

	Super::SetIsTemporarilyHiddenInEditor(bIsHidden);
}
#endif

#undef LOCTEXT_NAMESPACE
