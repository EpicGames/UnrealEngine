// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Actors/ChaosVDDataContainerBaseActor.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDSceneSelectionObserver.h"
#include "GameFramework/Actor.h"
#include "ChaosVDSolverInfoActor.generated.h"

class UChaosVDAdditionalGTDataRouterComponent;
class UChaosVDGTAccelerationStructuresDataComponent;
class UChaosVDSolverCharacterGroundConstraintDataComponent;
class UChaosVDSolverJointConstraintDataComponent;
struct FChaosVDParticleDataWrapper;
class UChaosVDParticleDataComponent;
class UChaosVDParticleExtraDataComponent;
class UChaosVDSceneQueryDataComponent;
class UChaosVDSolverCollisionDataComponent;
class UChaosVDCameraDataComponent;
struct FChaosVDCameraDataWrapper;
struct FChaosVDGameFrameData;

enum class EChaosVDParticleType : uint8;

/** Actor that contains all relevant data for the current visualized solver frame */
UCLASS(NotBlueprintable, NotPlaceable)
class AChaosVDSolverInfoActor : public AChaosVDDataContainerBaseActor, public FChaosVDSceneSelectionObserver
{
	GENERATED_BODY()

public:

	AChaosVDSolverInfoActor();

	void SetSolverName(const FName& InSolverName);
	const FName& GetSolverName()
	{
		return SolverName;
	}

	void SetIsServer(bool bInIsServer)
	{
		bIsServer = bInIsServer;
	}

	bool GetIsServer() const
	{
		return bIsServer;
	}


	virtual void UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData) override;

	virtual void UpdateFromNewSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData) override;

	virtual void SetScene(TWeakPtr<FChaosVDScene> InScene) override;

	void SetSimulationTransform(const FTransform& InSimulationTransform)
	{
		SimulationTransform = InSimulationTransform;
	}
	virtual const FTransform& GetSimulationTransform() const override
	{
		return SimulationTransform;
	}

	UChaosVDSolverCollisionDataComponent* GetCollisionDataComponent()
	{
		return CollisionDataComponent;
	}

	UChaosVDParticleDataComponent* GetParticleDataComponent()
	{
		return ParticleDataComponent;
	}

	UChaosVDSolverCharacterGroundConstraintDataComponent* GetCharacterGroundConstraintDataComponent()
	{
		return CharacterGroundConstraintDataComponent;
	}

	UChaosVDSceneQueryDataComponent* GetSceneQueryDataComponent() const
	{
		return SceneQueryDataComponent.Get();
	}

	virtual FName GetCustomIconName() const override;

	TSharedPtr<FChaosVDSceneParticle> GetParticleInstance(int32 ParticleID) const;

	virtual bool IsVisible() const override;

#if WITH_EDITOR
	void SetIsTemporarilyHiddenInEditor(bool bIsHidden) override;
#endif

protected:
	/**
	 * Finds the correct game frame data and executes the update flow for it based on the provided solver frame timing data.
	 * @param InSolverFrameData Solver frame data where to get the timing data from
	 */
	void FindAndUpdateFromCorrectGameFrameData(const FChaosVDSolverFrameData& InSolverFrameData);

	UPROPERTY(VisibleAnywhere, Category="Solver Data")
	FTransform SimulationTransform;

	UPROPERTY(VisibleAnywhere, Category="Solver Data")
	FName SolverName;

	UPROPERTY()
	TObjectPtr<UChaosVDSolverCollisionDataComponent> CollisionDataComponent;

	bool bIsServer;

	UPROPERTY()
	TObjectPtr<UChaosVDParticleDataComponent> ParticleDataComponent;

	UPROPERTY()
	TObjectPtr<UChaosVDSolverJointConstraintDataComponent> JointsDataComponent;

	UPROPERTY()
	TObjectPtr<UChaosVDSolverCharacterGroundConstraintDataComponent> CharacterGroundConstraintDataComponent;

	UPROPERTY()
	TObjectPtr<UChaosVDSceneQueryDataComponent> SceneQueryDataComponent;

	UPROPERTY()
	TObjectPtr<UChaosVDAdditionalGTDataRouterComponent> GTDataReRouteComponent;

	UPROPERTY()
	TObjectPtr<UChaosVDParticleExtraDataComponent> ParticleExtraDataComponent;
};
