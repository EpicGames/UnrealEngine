// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCollisionDataDetailsTab.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"

#include "IStructureDetailsView.h"
#include "Widgets/SChaosVDCollisionDataInspector.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDCollisionDataDetailsTab::~FChaosVDCollisionDataDetailsTab()
{
}

TSharedRef<SDockTab> FChaosVDCollisionDataDetailsTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DetailsPanelTab =
		SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("CollisionInspectorTab", "Collision Data Inspector"))
		.ToolTipText(LOCTEXT("CollisionInspectorTabTip", "See the details of the any collision data for the selected object"));

	if (const TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin())
	{
		DetailsPanelTab->SetContent
		(
			SAssignNew(CollisionDataInspector, SChaosVDCollisionDataInspector, GetChaosVDScene(), MainTabPtr.ToSharedRef())
		);
	}
	else
	{
		DetailsPanelTab->SetContent(GenerateErrorWidget());
	}

	DetailsPanelTab->SetTabIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "CollisionAnalyzer.TabIcon").GetIcon());

	HandleTabSpawned(DetailsPanelTab);

	return DetailsPanelTab;
}

void FChaosVDCollisionDataDetailsTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	FChaosVDTabSpawnerBase::HandleTabClosed(InTabClosed);

	CollisionDataInspector.Reset();
}

#undef LOCTEXT_NAMESPACE
