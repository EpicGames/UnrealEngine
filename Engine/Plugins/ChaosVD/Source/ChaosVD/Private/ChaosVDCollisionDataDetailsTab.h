// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDTabSpawnerBase.h"
#include "Templates/SharedPointer.h"

class SChaosVDCollisionDataInspector;
class IStructureDetailsView;
class FStructOnScope;

class FChaosVDCollisionDataDetailsTab : public FChaosVDTabSpawnerBase
{
public:
	FChaosVDCollisionDataDetailsTab(const FName& InTabID, const TSharedPtr<FTabManager>& InTabManager, const TWeakPtr<SChaosVDMainTab>& InOwningTabWidget)
		: FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{
	}

	virtual ~FChaosVDCollisionDataDetailsTab() override;
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

	TWeakPtr<SChaosVDCollisionDataInspector> GetCollisionInspectorInstance() const { return CollisionDataInspector; };

protected:

	TSharedPtr<SChaosVDCollisionDataInspector> CollisionDataInspector;
};
