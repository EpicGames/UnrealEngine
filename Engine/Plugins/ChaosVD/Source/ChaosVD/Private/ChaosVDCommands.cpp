// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCommands.h"

#include "ChaosVDStyle.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDCommands::FChaosVDCommands() : TCommands<FChaosVDCommands>(TEXT("ChaosVDEditor"), LOCTEXT("ChaosVisualDebuggerEditor", "Chaos Visual Debugger Editor"), NAME_None, FChaosVDStyle::GetStyleSetName())
{
}

void FChaosVDCommands::RegisterCommands()
{
	// Viewport Commands
	UI_COMMAND(ToggleFollowSelectedObject, "Follow Selected Object", "Start or Stop following the selected object", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::F8));
	UI_COMMAND(OverridePlaybackFrameRate, "Override Recorded Framerate", "When enabled, allows to playback the recording at a fixed framerate", EUserInterfaceActionType::ToggleButton, FInputChord(EModifierKey::Control, EKeys::R));
	UI_COMMAND(AllowTranslucentSelection, "Allow Translucent Selection", "Allows translucent objects to be selected", EUserInterfaceActionType::ToggleButton, FInputChord(EKeys::T));
	UI_COMMAND(HideSelected, "Hide Selected", "Hides the selected particle", EUserInterfaceActionType::Button, FInputChord(EKeys::H));
	UI_COMMAND(ShowAll, "Show All Particles", "Un-hides any manually hidden particle", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control , EKeys::H));
	UI_COMMAND(DeselectAll, "Deselect all objects", "Clears any active selection (Particle or Solver Data)", EUserInterfaceActionType::Button, FInputChord(EKeys::Escape));

	// Main Toolbar Commands
	UI_COMMAND(OpenFile, "Open File", "Opens the file browser modal", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control ,EKeys::O));
	UI_COMMAND(BrowseLiveSessions, "Browse Live Sessions", "Open the connect to session modal", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::S));
	UI_COMMAND(DisconnectFromLiveSession, "Disconnect from Live Session", "Disconnects from a legacy live session", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift, EKeys::X));
	UI_COMMAND(CombineOpenFiles, "Combine Files", "Combines all open sessions into a single file (if possible)", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control | EModifierKey::Shift , EKeys::C));
	UI_COMMAND(OpenSceneQueryBrowser, "Scene query Browser", "Opens the Scene Query browser window", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control , EKeys::Q));

	
	// Playback Controls commands
    UI_COMMAND(PlayPauseTrack, "Play/Pause", "Plays or pauses the playback for the current active track", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Control, EKeys::SpaceBar));
    UI_COMMAND(StopTrack, "Stop", "Stops the playback for the current active track", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::SpaceBar));

    UI_COMMAND(NextFrame, "Next Frame", "Plays the next frame of the current track", EUserInterfaceActionType::Button, FInputChord(EKeys::Period));
    UI_COMMAND(PrevFrame, "Prev Frame", "Plays the previous frame of the current track", EUserInterfaceActionType::Button, FInputChord(EKeys::Comma));

    UI_COMMAND(NextStage, "Next Stage", "Plays the next solver stage of the current track", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Period));
    UI_COMMAND(PrevStage, "Prev Stage", "Plays the previous solver stage of the current track", EUserInterfaceActionType::Button, FInputChord(EModifierKey::Shift, EKeys::Comma));
}

#undef LOCTEXT_NAMESPACE
