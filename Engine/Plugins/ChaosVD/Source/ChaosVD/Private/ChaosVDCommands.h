// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"
#include "Templates/SharedPointer.h"

class FChaosVDCommands : public TCommands<FChaosVDCommands>
{
public:

	FChaosVDCommands();

	// TCommands<> interface
	virtual void RegisterCommands() override;

	// Viewport Commands
	TSharedPtr<FUICommandInfo> ToggleFollowSelectedObject;
	TSharedPtr<FUICommandInfo> OverridePlaybackFrameRate;
	TSharedPtr<FUICommandInfo> AllowTranslucentSelection;
	TSharedPtr<FUICommandInfo> DeselectAll;
    TSharedPtr<FUICommandInfo> HideSelected;
    TSharedPtr<FUICommandInfo> ShowAll;

	// Main Toolbar Commands
	TSharedPtr<FUICommandInfo> OpenFile;
	TSharedPtr<FUICommandInfo> BrowseLiveSessions;
	TSharedPtr<FUICommandInfo> DisconnectFromLiveSession;
	TSharedPtr<FUICommandInfo> CombineOpenFiles;
	TSharedPtr<FUICommandInfo> OpenSceneQueryBrowser;

	// Playback Controls commands
	TSharedPtr<FUICommandInfo> PlayPauseTrack;
	TSharedPtr<FUICommandInfo> StopTrack;
	TSharedPtr<FUICommandInfo> NextFrame;
	TSharedPtr<FUICommandInfo> PrevFrame;
	TSharedPtr<FUICommandInfo> NextStage;
	TSharedPtr<FUICommandInfo> PrevStage;
};