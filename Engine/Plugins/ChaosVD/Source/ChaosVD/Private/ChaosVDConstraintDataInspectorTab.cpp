// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDConstraintDataInspectorTab.h"

#include "ChaosVDStyle.h"
#include "Widgets/SChaosVDConstraintDataInspector.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDConstraintDataInspectorTab::~FChaosVDConstraintDataInspectorTab()
{
}

TSharedRef<SDockTab> FChaosVDConstraintDataInspectorTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DetailsPanelTab =
	SNew(SDockTab)
	.TabRole(ETabRole::PanelTab)
	.Label(LOCTEXT("ConstraintDataInspectorTab", "Constraint Inspector"))
	.ToolTipText(LOCTEXT("ConstraintDataInspectorTabToolTip", "See the details of the any non-collision constraint selected in the viewport"));

	if (const TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin())
	{
		DetailsPanelTab->SetContent
		(
			SAssignNew(ConstraintDataInspector, SChaosVDConstraintDataInspector, GetChaosVDScene(), MainTabPtr.ToSharedRef())
		);
	}
	else
	{
		DetailsPanelTab->SetContent(GenerateErrorWidget());
	}

	DetailsPanelTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("ConnectionIcon"));

	HandleTabSpawned(DetailsPanelTab);

	return DetailsPanelTab;
}

void FChaosVDConstraintDataInspectorTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	FChaosVDTabSpawnerBase::HandleTabClosed(InTabClosed);

	ConstraintDataInspector.Reset();
}

#undef LOCTEXT_NAMESPACE
