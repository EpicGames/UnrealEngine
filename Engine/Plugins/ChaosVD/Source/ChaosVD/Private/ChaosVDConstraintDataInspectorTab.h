// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDTabSpawnerBase.h"
#include "Templates/SharedPointer.h"

class SChaosVDConstraintDataInspector;
class SChaosVDSceneQueryDataInspector;

/** Spawns and handles and instance for the visual debugger Constraints Inspector tab */
class FChaosVDConstraintDataInspectorTab final : public FChaosVDTabSpawnerBase
{
public:
	FChaosVDConstraintDataInspectorTab(const FName& InTabID, const TSharedPtr<FTabManager>& InTabManager, const TWeakPtr<SChaosVDMainTab>& InOwningTabWidget)
		: FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{
	}

	virtual ~FChaosVDConstraintDataInspectorTab() override;
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

	TWeakPtr<SChaosVDConstraintDataInspector> GetConstraintDataInspectorInstance() const { return ConstraintDataInspector; }

protected:
	TSharedPtr<SChaosVDConstraintDataInspector> ConstraintDataInspector;
};
