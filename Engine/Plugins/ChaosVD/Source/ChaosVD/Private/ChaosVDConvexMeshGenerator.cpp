// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDConvexMeshGenerator.h"

#include "Chaos/Core.h"
#include "Chaos/Convex.h"
#include "Chaos/Utilities.h"
#include "CompGeom/PolygonTriangulation.h"

using namespace UE::Geometry;

struct FChaosVDTriangleData
{
	int32 FaceIndex;
	TArray<FIndex3i> FaceTriangles;
	int NormalIndices[3];
	FVector Normal;
	double TriangleArea = 0;
};

void FChaosVDConvexMeshGenerator::GenerateFromConvex(const Chaos::FConvex& Convex)
{
	using namespace Chaos;
	using FRealType = FRealSingle;
	using FVec3Type = TVec3<FRealType>;

	const int32 FacesNum = Convex.GetFaces().Num();

	TArray<FChaosVDTriangleData> PerFaceTriangleData;
	PerFaceTriangleData.Reserve(FacesNum);
	
	// Triangulate each face of the Convex shape
	int32 TriangleCount = 0;
	for (int32 PlaneIndex = 0; PlaneIndex < Convex.GetFaces().Num(); ++PlaneIndex)
	{
		const int32 PlaneVerticesNum = Convex.NumPlaneVertices(PlaneIndex);

		// Get the positions of each vertex of the plane
		TArray<FVector> PlaneVertices;
		PlaneVertices.Reserve(PlaneVerticesNum);

		for (int32 PlaneLocalVertexIndex = 0; PlaneLocalVertexIndex < PlaneVerticesNum; ++PlaneLocalVertexIndex)
		{
			const int32 GlobalVertexIndex = Convex.GetPlaneVertex(PlaneIndex, PlaneLocalVertexIndex);
			PlaneVertices.Add(FVector(UE::Math::TVector<double>(Convex.GetVertex(GlobalVertexIndex))));
		}

		FChaosVDTriangleData FaceTriangleData;
		PolygonTriangulation::TriangulateSimplePolygon(PlaneVertices,FaceTriangleData.FaceTriangles);

		FaceTriangleData.Normal = Convex.GetPlane(PlaneIndex).Normal();
		FaceTriangleData.FaceIndex = PlaneIndex;

		TriangleCount += FaceTriangleData.FaceTriangles.Num();
		
		PerFaceTriangleData.Add(MoveTemp(FaceTriangleData));
	}

	const TArray<FVec3Type>& ConvexVertices = Convex.GetVertices();
	const int32 NormalsNum = TriangleCount * 3;
	constexpr int32 UVsNum = 0;
	SetBufferSizes(ConvexVertices.Num(), TriangleCount, UVsNum, NormalsNum);

	// Fill the vertex buffer with the transformed vertices of the Convex Shape
	for (int32 i = 0; i < ConvexVertices.Num(); ++i) 
	{
		Vertices[i] = FVector3d(UE::Math::TVector<double>(ConvexVertices[i]));
	}

	int32 TriangleIndex = 0;
	int CurrentNormalIndex = 0;
	for (FChaosVDTriangleData& Data : PerFaceTriangleData)
	{
		for (FIndex3i& PolyTriangle : Data.FaceTriangles)
		{
			// Re-map from Plane Local vertex to Global Vertices
			PolyTriangle.A = Convex.GetPlaneVertex(Data.FaceIndex, PolyTriangle.A);
			PolyTriangle.B = Convex.GetPlaneVertex(Data.FaceIndex, PolyTriangle.B);
			PolyTriangle.C = Convex.GetPlaneVertex(Data.FaceIndex, PolyTriangle.C);

			// Use the Face Normal in all Vertices of the triangle
			for (int LocalVertexIndex = 0; LocalVertexIndex < 3; ++LocalVertexIndex)
			{
				Normals[CurrentNormalIndex] = UE::Math::TVector<float>(Data.Normal);
				Data.NormalIndices[LocalVertexIndex] = CurrentNormalIndex;
				CurrentNormalIndex++;
			}

			SetTriangle(TriangleIndex, PolyTriangle);
			SetTrianglePolygon(TriangleIndex,Data.FaceIndex);
			SetTriangleNormals(TriangleIndex, Data.NormalIndices[0], Data.NormalIndices[1], Data.NormalIndices[2]);
			TriangleIndex++;
		}
	}

	bIsGenerated = true;
}

UE::Geometry::FMeshShapeGenerator& FChaosVDConvexMeshGenerator::Generate()
{
	ensureAlwaysMsgf(bIsGenerated, TEXT("You need to call FChaosVDConvexMeshGenerator::GenerateFromConvex before calling Generate"));
	return *this;
}
