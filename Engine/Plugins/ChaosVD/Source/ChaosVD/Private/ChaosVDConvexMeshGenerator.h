// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Generators/MeshShapeGenerator.h"

namespace Chaos
{
	class FConvex;
}
;
/**
 * Generates an convex mesh with the smallest number of triangles possible 
 */
class FChaosVDConvexMeshGenerator : public UE::Geometry::FMeshShapeGenerator
{
public:

	/** Prepares the Mesh Generator so the Generate method can generate the desired Dynamic Mesh
	 * @param Convex shape data used to generate the dynamic mesh
	 */
	void GenerateFromConvex(const Chaos::FConvex& Convex);
	
	virtual FMeshShapeGenerator& Generate() override;

private:

	bool bIsGenerated = false; 
};

