// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDEditorMode.h"

#include "Actors/ChaosVDSolverInfoActor.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Elements/Actor/ActorElementEditorViewportInteractionCustomization.h"
#include "Selection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDEditorMode)

const FEditorModeID UChaosVDEditorMode::EM_ChaosVisualDebugger(TEXT("EM_ChaosVisualDebugger"));

FVector FChaosVDEdModeWidgetHelper::GetWidgetLocation() const
{
	if (const AActor* SelectedActor = GetCurrentSelectedActor())
	{
		return SelectedActor->GetActorLocation();
	}
	else
	{
		return FLegacyEdModeWidgetHelper::GetWidgetLocation();
	}
}

AActor* FChaosVDEdModeWidgetHelper::GetCurrentSelectedActor() const
{
	if (!ensure(Owner))
	{
		return nullptr;
	}
	
	if (USelection* CurrentSelection = Owner->GetSelectedActors())
	{
		ensureMsgf(CurrentSelection->Num() < 2, TEXT("CVD does not support multi selection yet"));
		return CurrentSelection->GetTop<AActor>();
	}

	return nullptr;
}

bool FChaosVDEdModeWidgetHelper::InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale)
{
	bool bHandled = false;
	if (InViewportClient->bWidgetAxisControlledByDrag)
	{
		if (AActor* SelectedActor = GetCurrentSelectedActor())
		{
			
			FActorElementEditorViewportInteractionCustomization::ApplyDeltaToActor(
							SelectedActor, true, &InDrag,
							&InRot, &InScale, SelectedActor->GetActorLocation(), FInputDeviceState());

			bHandled = true;
		}
	}
	
	if (bHandled)
	{
		return true;
	}
	else
	{
		return FLegacyEdModeWidgetHelper::InputDelta(InViewportClient, InViewport, InDrag, InRot, InScale);
	}
}

bool FChaosVDEdModeWidgetHelper::UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const
{
	// Currently all modes follow the same rule. ParticleActors cannot be moved
	return UsesTransformWidget();
}

bool FChaosVDEdModeWidgetHelper::UsesTransformWidget() const
{
	if (const AActor* SelectedActor = GetCurrentSelectedActor())
	{
		// CVD Data Actors cannot be moved
		return !SelectedActor->IsA(AChaosVDSolverInfoActor::StaticClass());
	}

	return false;
}

UChaosVDEditorMode::UChaosVDEditorMode() : UBaseLegacyWidgetEdMode()
{
	Info = FEditorModeInfo(UChaosVDEditorMode::EM_ChaosVisualDebugger,
		NSLOCTEXT("ChaosVisualDebugger", "ChaosVisualDebuggerEditorModeName", "Chaos Visual Debugger Editor Mode"),
		FSlateIcon(),
		false);

	Toolkit = nullptr;
}

bool UChaosVDEditorMode::CanAutoSave() const
{
	return false;
}

UWorld* UChaosVDEditorMode::GetWorld() const
{
	return CVDWorldPtr.IsValid() ? CVDWorldPtr.Get() : nullptr;
}

TSharedRef<FLegacyEdModeWidgetHelper> UChaosVDEditorMode::CreateWidgetHelper()
{
	return MakeShared<FChaosVDEdModeWidgetHelper>();
}
