// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tools/LegacyEdModeWidgetHelpers.h"
#include "Tools/LegacyEdModeInterfaces.h"

#include "ChaosVDEditorMode.generated.h"

class FChaosVDEditorModeTools;

/**
 * LegacyEdModeWidgetHelper implementation for the Chaos Visual Debugger tool
 */
class FChaosVDEdModeWidgetHelper final : public FLegacyEdModeWidgetHelper
{
public:
	virtual FVector GetWidgetLocation() const override;
	virtual bool InputDelta(FEditorViewportClient* InViewportClient, FViewport* InViewport, FVector& InDrag, FRotator& InRot, FVector& InScale) override;
	virtual bool UsesTransformWidget(UE::Widget::EWidgetMode CheckMode) const override;
	virtual bool UsesTransformWidget() const override;
	
private:
	AActor* GetCurrentSelectedActor() const;
};

/**
 * Main Editor Mode for the Chaos Visual Debugger tool.
 */
UCLASS()
class UChaosVDEditorMode : public UBaseLegacyWidgetEdMode
{
	GENERATED_BODY()

public:
	UChaosVDEditorMode();

	virtual bool IsCompatibleWith(FEditorModeID OtherModeID) const override { return false; };

	static const FEditorModeID EM_ChaosVisualDebugger;

	virtual bool UsesToolkits() const override { return false; };

	virtual bool CanAutoSave() const override;

	virtual UWorld* GetWorld() const override;

	void SetWorld(UWorld* InWorld) { CVDWorldPtr = InWorld; }

protected:

	virtual TSharedRef<FLegacyEdModeWidgetHelper> CreateWidgetHelper() override;

	TWeakObjectPtr<UWorld> CVDWorldPtr;
};
