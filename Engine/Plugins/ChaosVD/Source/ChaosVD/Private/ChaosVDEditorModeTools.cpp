// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDEditorModeTools.h"

#include "ChaosVDEditorMode.h"
#include "ChaosVDScene.h"
#include "Selection.h"


FChaosVDEditorModeTools::FChaosVDEditorModeTools(const TWeakPtr<FChaosVDScene>& InScenePtr) : FEditorModeTools(), ScenePtr(InScenePtr)
{
	USelection::SelectNoneEvent.RemoveAll(this);
}

FString FChaosVDEditorModeTools::GetReferencerName() const
{
	return TEXT("FChaosVDEditorModeTools");
}

USelection* FChaosVDEditorModeTools::GetSelectedActors() const
{
	if (const TSharedPtr<FChaosVDScene> SceneSharedPtr = ScenePtr.Pin())
	{
		return SceneSharedPtr->GetActorSelectionObject();
	}
	
	return nullptr;
}

USelection* FChaosVDEditorModeTools::GetSelectedObjects() const
{
	if (const TSharedPtr<FChaosVDScene> SceneSharedPtr = ScenePtr.Pin())
	{
		return SceneSharedPtr->GetObjectsSelectionObject();
	}

	return nullptr;
}

USelection* FChaosVDEditorModeTools::GetSelectedComponents() const
{
	if (const TSharedPtr<FChaosVDScene> SceneSharedPtr = ScenePtr.Pin())
	{
		return SceneSharedPtr->GetComponentsSelectionObject();
	}
	
	return nullptr;
}

UTypedElementSelectionSet* FChaosVDEditorModeTools::GetEditorSelectionSet() const
{
	if (const TSharedPtr<FChaosVDScene> SceneSharedPtr = ScenePtr.Pin())
	{
		return SceneSharedPtr->GetElementSelectionSet();
	}
	
	return nullptr;
}

UWorld* FChaosVDEditorModeTools::GetWorld() const
{
	if (const TSharedPtr<FChaosVDScene> SceneSharedPtr = ScenePtr.Pin())
	{
		return SceneSharedPtr->GetUnderlyingWorld();
	}

	return nullptr;
}
