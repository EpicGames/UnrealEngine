// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "EditorModeManager.h"

class FChaosVDScene;

/** Editor Mode tools for the Chaos Visual Debugger tool */
class FChaosVDEditorModeTools : public FEditorModeTools
{
public:
	FChaosVDEditorModeTools(const TWeakPtr<FChaosVDScene>& InScenePtr);

	virtual FString GetReferencerName() const override;
	virtual USelection* GetSelectedActors() const override;
	virtual USelection* GetSelectedObjects() const override;
	virtual USelection* GetSelectedComponents() const override;
	virtual UTypedElementSelectionSet* GetEditorSelectionSet() const override;
	virtual UWorld* GetWorld() const override;

	/** Returns the Chaos Visual Debugger Scene */
	TWeakPtr<FChaosVDScene> GetScene() const { return ScenePtr; }

protected:
	TWeakPtr<FChaosVDScene> ScenePtr;
};
