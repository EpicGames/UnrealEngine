// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDEngine.h"

#include "Chaos/ChaosVDEngineEditorBridge.h"
#include "ChaosVDModule.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDScene.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVisualDebugger/ChaosVDOptionalDataChannel.h"
#include "RemoteSessionsManager.h"
#include "Settings/ChaosVDMiscSettings.h"
#include "Trace/ChaosVDCombinedTraceFile.h"
#include "TraceSessionsManager.h"
#include "Trace/ChaosVDTraceManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDEngine)

FChaosVDEngine::FChaosVDEngine()
{
	InstanceGUID = FGuid::NewGuid();
}

FChaosVDEngine::~FChaosVDEngine()
{
}

void FChaosVDEngine::Initialize()
{
	if (bIsInitialized)
	{
		return;
	}

	// Create an Empty Scene
	CurrentScene = MakeShared<FChaosVDScene>();
	CurrentScene->Initialize();

	PlaybackController = MakeShared<FChaosVDPlaybackController>(CurrentScene);
	PlaybackController->Initialize();

	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		LiveSessionStoppedDelegateHandle = RemoteSessionManager->OnSessionRecordingStopped().AddLambda([WeakThis = AsWeak()](TWeakPtr<FSessionInfo> Session)
		{
			if (const TSharedPtr<FChaosVDEngine> CVDEngine = WeakThis.Pin())
			{
				if (CVDEngine->HasAnyLiveSessionActive())
				{
					CVDEngine->StopActiveTraceSessions();

					if (CVDEngine->PlaybackController)
					{
						CVDEngine->PlaybackController->HandleDisconnectedFromSession();
					}
				}
			}
		});
	}

	RestoreDataChannelsEnabledStateFromSave();
	
	bIsInitialized = true;
}

void FChaosVDEngine::CloseActiveTraceSessions()
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FTraceSessionsManager> CVDTraceManager = FChaosVDModule::Get().GetTraceSessionsManager())
	{
		for (FTraceSessionDescriptor& SessionDescriptor : CurrentSessionDescriptors)
		{
			OnTraceSessionClosed().Broadcast(SessionDescriptor);

			CVDTraceManager->CloseSession(SessionDescriptor.SessionName);
			SessionDescriptor.bIsLiveSession = false;
		}
	}

	CurrentSessionDescriptors.Reset();

	if (PlaybackController)
	{
		PlaybackController->UnloadCurrentRecording(EChaosVDUnloadRecordingFlags::BroadcastChanges);
	}
}

void FChaosVDEngine::CloseSessionByRemoteSessionID(FGuid RemoteSessionID)
{
	using namespace UE::TraceBasedDebuggers;
	FTraceSessionDescriptor DescriptorFoundCopy = GetTraceSessionDescriptor(RemoteSessionID);

	if (!DescriptorFoundCopy.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FTraceSessionsManager> CVDTraceManager = FChaosVDModule::Get().GetTraceSessionsManager())
	{
		CVDTraceManager->CloseSession(DescriptorFoundCopy.SessionName);
	}

	OnTraceSessionClosed().Broadcast(DescriptorFoundCopy);
	CurrentSessionDescriptors.Remove(DescriptorFoundCopy);

	if (PlaybackController && CurrentSessionDescriptors.IsEmpty())
	{
		PlaybackController->UnloadCurrentRecording(EChaosVDUnloadRecordingFlags::BroadcastChanges);
	}
}

void FChaosVDEngine::StopActiveTraceSessions()
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FTraceSessionsManager> CVDTraceManager = FChaosVDModule::Get().GetTraceSessionsManager())
	{
		for (FTraceSessionDescriptor& SessionDescriptor : CurrentSessionDescriptors)
		{
			CVDTraceManager->StopSession(SessionDescriptor.SessionName);
		}
	}
}

void FChaosVDEngine::DeInitialize()
{
	if (!bIsInitialized)
	{
		return;
	}

	CurrentScene->DeInitialize();
	CurrentScene.Reset();
	PlaybackController->DeInitialize();
	PlaybackController.Reset();

	CloseActiveTraceSessions();

	if (const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		RemoteSessionManager->OnSessionRecordingStopped().Remove(LiveSessionStoppedDelegateHandle);
	}

	LiveSessionStoppedDelegateHandle = FDelegateHandle();

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

#if WITH_CHAOS_VISUAL_DEBUGGER

	if (DataChannelStateUpdatedHandle.IsValid())
	{
		Chaos::VisualDebugger::FChaosVDDataChannelsManager::Get().OnChannelStateChanged().Remove(DataChannelStateUpdatedHandle);
	}

#endif

	bIsInitialized = false;
}

void FChaosVDEngine::RestoreDataChannelsEnabledStateFromSave()
{
#if WITH_CHAOS_VISUAL_DEBUGGER

	UChaosVDMiscSettings* MiscSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMiscSettings>();
	if (!MiscSettings)
	{
		return;
	}

	using namespace Chaos::VisualDebugger;
	FChaosVDDataChannelsManager::Get().EnumerateDataChannels([MiscSettings](const TWeakPtr<FChaosVDOptionalDataChannel>& Channel)
	{
		if (TSharedPtr<FChaosVDOptionalDataChannel> LockedChannel = Channel.Pin())
		{
			if (bool* SavedEnabledState = MiscSettings->DataChannelEnabledState.Find(LockedChannel->GetId().ToString()))
			{
				LockedChannel->SetChannelEnabled(*SavedEnabledState);
			}
		}
		return true;
	});

	DataChannelStateUpdatedHandle = FChaosVDDataChannelsManager::Get().OnChannelStateChanged().AddSP(this, &FChaosVDEngine::UpdateSavedDataChannelsEnabledState);

#endif
}

void FChaosVDEngine::UpdateSavedDataChannelsEnabledState(TWeakPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel> DataChannelChanged)
{
#if WITH_CHAOS_VISUAL_DEBUGGER

	UChaosVDMiscSettings* MiscSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMiscSettings>();
	if (!MiscSettings)
	{
		return;
	}

	if (TSharedPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel> DataChannelPtr = DataChannelChanged.Pin())
	{
		MiscSettings->DataChannelEnabledState.FindOrAdd(DataChannelPtr->GetId().ToString()) = DataChannelPtr->IsChannelEnabled();
	}

	MiscSettings->SaveConfig();

#endif
}

void FChaosVDEngine::UpdateRecentFilesList(const FString& InFilename)
{
	UChaosVDMiscSettings* MiscSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMiscSettings>();
	if (!MiscSettings)
	{
		return;
	}

	FDateTime CurrentTime = FDateTime::UtcNow();
	if (FChaosVDRecentFile* RecentProject = MiscSettings->RecentFiles.FindByKey(InFilename))
	{
		RecentProject->LastOpenTime = CurrentTime;
	}
	else
	{
		FChaosVDRecentFile RecentFileEntry(InFilename, CurrentTime);
		MiscSettings->RecentFiles.Emplace(RecentFileEntry);
	}
	
	MiscSettings->RecentFiles.Sort(FChaosVDRecentFile::FRecentFilesSortPredicate());

	if (MiscSettings->RecentFiles.Num() > MiscSettings->MaxRecentFilesNum)
	{
		MiscSettings->RecentFiles.SetNum(MiscSettings->MaxRecentFilesNum);
	}

	MiscSettings->SaveConfig();
}

bool FChaosVDEngine::HasAnyLiveSessionActive() const
{
	return PlaybackController && PlaybackController->IsPlayingLiveSession();
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
const FChaosVDTraceSessionDescriptor& FChaosVDEngine::GetSessionDescriptor(FGuid) const
{
	static FChaosVDTraceSessionDescriptor InvalidSessionDescriptor;
	return InvalidSessionDescriptor;
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS

const UE::TraceBasedDebuggers::FTraceSessionDescriptor& FChaosVDEngine::GetTraceSessionDescriptor(FGuid RemoteSessionID) const
{
	using namespace UE::TraceBasedDebuggers;
	static FTraceSessionDescriptor InvalidSessionDescriptor;
	const FTraceSessionDescriptor* DescriptorFound = CurrentSessionDescriptors.FindByPredicate([RemoteSessionID](const FTraceSessionDescriptor& InDescriptor)
	{
		return RemoteSessionID == InDescriptor.RemoteSessionID;
	});

	return DescriptorFound ? *DescriptorFound : InvalidSessionDescriptor;
}

TArrayView<UE::TraceBasedDebuggers::FTraceSessionDescriptor> FChaosVDEngine::GetTraceSessionDescriptors()
{
	return CurrentSessionDescriptors;
}

void FChaosVDEngine::LoadRecording_Internal(const TFunction<FString(const TSharedPtr<FChaosVDRecording>&)>& LoadCallback, EChaosVDLoadRecordedDataMode LoadingMode)
{
	TSharedPtr<FChaosVDRecording> ExistingRecordingInstance = nullptr;
	if (LoadingMode == EChaosVDLoadRecordedDataMode::MultiSource && !CurrentSessionDescriptors.IsEmpty())
	{
		if (const TSharedPtr<const TraceServices::IAnalysisSession> TraceSession = FChaosVDModule::Get().GetTraceSessionsManager()->GetSession(CurrentSessionDescriptors[0].SessionName))
		{
			if (const FChaosVDTraceProvider* ChaosVDProvider = TraceSession->ReadProvider<FChaosVDTraceProvider>(FChaosVDTraceProvider::ProviderName))
			{
				ExistingRecordingInstance = ChaosVDProvider->GetRecordingForSession();
			}
		}
	}

	using namespace UE::TraceBasedDebuggers;
	FTraceSessionDescriptor NewSessionFromFileDescriptor;
	NewSessionFromFileDescriptor.SessionName = LoadCallback(ExistingRecordingInstance);
	NewSessionFromFileDescriptor.bIsLiveSession = false;

	if (NewSessionFromFileDescriptor.IsValid())
	{
		OpenSession(NewSessionFromFileDescriptor, LoadingMode);
	}
}

void FChaosVDEngine::LoadRecording(const FString& FilePath, EChaosVDLoadRecordedDataMode LoadingMode)
{
	if (LoadingMode == EChaosVDLoadRecordedDataMode::SingleSource)
	{
		CloseActiveTraceSessions();
	}

	if (FilePath.EndsWith(TEXT("cvdmulti")))
	{
		LoadCombinedMultiRecording(FilePath);
		return;
	}

	auto LoadSingleFilePathFromTraceManager = [&FilePath](const TSharedPtr<FChaosVDRecording>& ExistingRecordingInstance)
	{
		return FChaosVDModule::Get().GetTraceSessionsManager()->LoadTraceFile(FilePath, /*PreStartAnalysisCallback*/
			[ExistingRecordingInstance]
			{
				FChaosVDTraceManagerThreadContext::Get().PendingExternalRecordingWeakPtr = ExistingRecordingInstance;
			});
	};

	LoadRecording_Internal(LoadSingleFilePathFromTraceManager, LoadingMode);
	UpdateRecentFilesList(FilePath);
}

void FChaosVDEngine::LoadCombinedMultiRecording(const FString& FilePath)
{
	CloseActiveTraceSessions();

	if (!ensure(FilePath.EndsWith(TEXT("cvdmulti"))))
	{
		return;
	}

	using namespace Chaos::VisualDebugger;

	TArray<TUniquePtr<IFileHandle>> ExtractedHandles = CombinedTraceFile::GetInnerFileHandles(FilePath);
	
	if (!ensure(!ExtractedHandles.IsEmpty()))
	{
		return;
	}

	int32 CurrentFileIndex = 0;
	for (TUniquePtr<IFileHandle>& Handle : ExtractedHandles)
	{
		const FStringFormatOrderedArguments Args {FilePath, FString::FromInt(CurrentFileIndex)};
		FString SessionName = FString::Format(TEXT("{0}-{1}"), Args);
		auto LoadSingleFileHandleFromTraceManager = [&Handle, &SessionName](const TSharedPtr<FChaosVDRecording>& ExistingRecordingInstance)
		{
			return FChaosVDModule::Get().GetTraceSessionsManager()->LoadTraceFile(MoveTemp(Handle), SessionName,  /*PreStartAnalysisCallback*/
				[ExistingRecordingInstance]
				{
					FChaosVDTraceManagerThreadContext::Get().PendingExternalRecordingWeakPtr = ExistingRecordingInstance;
				});
		};

		LoadRecording_Internal(LoadSingleFileHandleFromTraceManager, EChaosVDLoadRecordedDataMode::MultiSource);
		CurrentFileIndex++;
	}
	
	UpdateRecentFilesList(FilePath);
}

bool FChaosVDEngine::ConnectToLiveSession(uint32 SessionID, const FString& InSessionAddress, EChaosVDLoadRecordedDataMode LoadingMode)
{
	if (LoadingMode == EChaosVDLoadRecordedDataMode::SingleSource)
	{
		CloseActiveTraceSessions();
	}
	
	using namespace UE::TraceBasedDebuggers;
	FTraceSessionDescriptor NewSessionFromFileDescriptor;

	NewSessionFromFileDescriptor.bIsLiveSession = true;
	
	TSharedPtr<FChaosVDRecording> ExistingRecordingInstance = nullptr;
	if (LoadingMode == EChaosVDLoadRecordedDataMode::MultiSource && PlaybackController)
	{
		ExistingRecordingInstance = PlaybackController->GetCurrentRecording().Pin();
	}

	auto PreStartAnalysisCallback = [ExistingRecordingInstance]
		{
			FChaosVDTraceManagerThreadContext::Get().PendingExternalRecordingWeakPtr = ExistingRecordingInstance;
		};

	NewSessionFromFileDescriptor.SessionName =
		FChaosVDModule::Get().GetTraceSessionsManager()->ConnectToLiveSession(InSessionAddress, SessionID, PreStartAnalysisCallback);

	bool bSuccess = false;

	if (NewSessionFromFileDescriptor.IsValid())
	{
		OpenSession(NewSessionFromFileDescriptor, LoadingMode);
		bSuccess = true;
	}

	return bSuccess;
}


bool FChaosVDEngine::ConnectToLiveSession_Direct(EChaosVDLoadRecordedDataMode LoadingMode)
{
	UE_LOGF(LogChaosVDEditor, Error, "[%s] This method is no longer supported. Please use the overload that takes a Remote Session ID.", __func__);
	return false;
}

bool FChaosVDEngine::ConnectToLiveSession_Direct(FGuid RemoteSessionID, EChaosVDLoadRecordedDataMode LoadingMode)
{
	if (LoadingMode == EChaosVDLoadRecordedDataMode::SingleSource)
	{
		CloseActiveTraceSessions();
	}

	using namespace UE::TraceBasedDebuggers;
	FTraceSessionDescriptor NewSessionFromFileDescriptor;

	NewSessionFromFileDescriptor.RemoteSessionID = RemoteSessionID;
	NewSessionFromFileDescriptor.bIsLiveSession = true;

	TSharedPtr<FChaosVDRecording> ExistingRecordingInstance = nullptr;
	if (LoadingMode == EChaosVDLoadRecordedDataMode::MultiSource && PlaybackController)
	{
		ExistingRecordingInstance = PlaybackController->GetCurrentRecording().Pin();
	}

	auto PreStartAnalysisCallback = [ExistingRecordingInstance]
		{
			FChaosVDTraceManagerThreadContext::Get().PendingExternalRecordingWeakPtr = ExistingRecordingInstance;
		};

	NewSessionFromFileDescriptor.SessionName =
		FChaosVDModule::Get().GetTraceSessionsManager()->ConnectToLiveSession_Direct(RemoteSessionID, NewSessionFromFileDescriptor.SessionPort, PreStartAnalysisCallback);

	bool bSuccess = false;

	if (NewSessionFromFileDescriptor.IsValid())
	{
		OpenSession(NewSessionFromFileDescriptor, LoadingMode);
		bSuccess = true;
	}

	return bSuccess;
}

bool FChaosVDEngine::ConnectToLiveSession_Relay(FGuid RemoteSessionID, EChaosVDLoadRecordedDataMode LoadingMode)
{
	if (LoadingMode == EChaosVDLoadRecordedDataMode::SingleSource)
	{
		CloseActiveTraceSessions();
	}

	using namespace UE::TraceBasedDebuggers;
	FTraceSessionDescriptor NewSessionFromFileDescriptor;

	NewSessionFromFileDescriptor.bIsLiveSession = true;

	TSharedPtr<FChaosVDRecording> ExistingRecordingInstance = nullptr;
	if (LoadingMode == EChaosVDLoadRecordedDataMode::MultiSource && PlaybackController)
	{
		ExistingRecordingInstance = PlaybackController->GetCurrentRecording().Pin();
	}

	auto PreStartAnalysisCallback = [ExistingRecordingInstance]
		{
			FChaosVDTraceManagerThreadContext::Get().PendingExternalRecordingWeakPtr = ExistingRecordingInstance;
		};

	NewSessionFromFileDescriptor.RemoteSessionID = RemoteSessionID;
	NewSessionFromFileDescriptor.SessionName =
		FChaosVDModule::Get().GetTraceSessionsManager()->ConnectToLiveSession_Relay(RemoteSessionID, PreStartAnalysisCallback);

	bool bSuccess = false;

	if (NewSessionFromFileDescriptor.IsValid())
	{
		OpenSession(NewSessionFromFileDescriptor, LoadingMode);
		bSuccess = true;
	}

	return bSuccess;
}

void FChaosVDEngine::OpenSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& SessionDescriptor, EChaosVDLoadRecordedDataMode LoadingMode)
{
	CurrentSessionDescriptors.Emplace(SessionDescriptor);

	if (CurrentSessionDescriptors.Num() <= 1)
	{
		PlaybackController->LoadChaosVDRecordingFromTraceSession(SessionDescriptor);
	}

	OnTraceSessionOpened().Broadcast(SessionDescriptor);
}

bool FChaosVDEngine::SaveOpenSessionToCombinedFile(const FString& InTargetFilePath)
{
	if (!ensure(CurrentSessionDescriptors.Num() > 0))
	{
		return false;
	}

	IPlatformFile& FileSystem = IPlatformFile::GetPlatformPhysical();

	TArray<TUniquePtr<IFileHandle>> FilePathsToCombine;
	FilePathsToCombine.Reserve(CurrentSessionDescriptors.Num());

	using namespace UE::TraceBasedDebuggers;
	for (FTraceSessionDescriptor& SessionDescriptor : CurrentSessionDescriptors)
	{
		if (SessionDescriptor.SessionName.EndsWith(TEXT("cvdmulti")))
		{
			UE_LOGF(LogChaosVDEditor, Error, "Failed to combine files, because one of the files is a multi cvd file and that is not supported yet | [%ls]", *SessionDescriptor.SessionName)
			return false;
		}
	
		IFileHandle* ContainerFileHandle = FileSystem.OpenRead(*SessionDescriptor.SessionName);
		if (!ContainerFileHandle)
		{
			UE_LOGF(LogChaosVDEditor, Error, "Failed to combine files, because one fo the files cannot be open | [%ls]", *SessionDescriptor.SessionName)
			return false;
		}

		FilePathsToCombine.Emplace(TUniquePtr<IFileHandle>(ContainerFileHandle));
	}

	FString FinalFilePath = InTargetFilePath;

	if (FinalFilePath.IsEmpty())
	{
		FString PathPart;
		FString FilenamePart;
		FString ExtensionPart;
		FPaths::Split(CurrentSessionDescriptors[0].SessionName, PathPart, FilenamePart, ExtensionPart);

		FStringFormatOrderedArguments NameArgs { FilenamePart, CurrentSessionDescriptors.Num(),FDateTime::Now().ToString(TEXT("%Y%m%d_%H%M%S")) };
		FinalFilePath = FPaths::Combine(PathPart, FString::Format(TEXT("ChaosVD-{0}-Combined-{1}-Sessions-{2}.cvdmulti"), NameArgs));
	}
	
	using namespace Chaos::VisualDebugger::CombinedTraceFile;

	return ensure(CombineFiles(FilePathsToCombine, FinalFilePath));
}

bool FChaosVDEngine::CanCombineOpenSessions() const
{
	if (CurrentSessionDescriptors.Num() > 1)
	{
		using namespace UE::TraceBasedDebuggers;
		for (const FTraceSessionDescriptor& Session : CurrentSessionDescriptors)
		{
			if (Session.SessionName.EndsWith(TEXT("cvdmulti")))
			{
				return false;
			}
		}

		return true;
	}

	return false;
}

bool FChaosVDEngine::Tick(float DeltaTime)
{
	return true;
}
