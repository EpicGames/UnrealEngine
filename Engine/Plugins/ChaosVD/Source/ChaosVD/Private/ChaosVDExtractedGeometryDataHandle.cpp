// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDExtractedGeometryDataHandle.h"

#include "Chaos/ImplicitObject.h"

void FChaosVDExtractedGeometryDataHandle::SetGeometryKey(const uint32 Key)
{
	GeometryKey = Key;
}

FName FChaosVDExtractedGeometryDataHandle::GetTypeName() const
{
	using namespace Chaos;

	static FName InvalidName = TEXT("Invalid");

	return ImplicitObject ? GetImplicitObjectTypeName(GetInnerType(ImplicitObject->GetType())) : InvalidName;
}
