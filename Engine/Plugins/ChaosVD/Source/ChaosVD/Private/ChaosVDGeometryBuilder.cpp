// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGeometryBuilder.h"

#include "Chaos/HeightField.h"
#include "ChaosVDConvexMeshGenerator.h"
#include "ChaosVDGeometryDataComponent.h"
#include "ChaosVDModule.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDTriMeshGenerator.h"
#include "DynamicMeshToMeshDescription.h"
#include "Generators/CapsuleGenerator.h"
#include "Generators/MinimalBoxMeshGenerator.h"
#include "Generators/SphereGenerator.h"
#include "MeshConstraints.h"
#include "MeshConstraintsUtil.h"
#include "MeshSimplification.h"
#include "Misc/ScopedSlowTask.h"
#include "StaticMeshAttributes.h"
#include "UDynamicMesh.h"
#include "UObject/UObjectGlobals.h"

namespace Chaos::VisualDebugger
{
	namespace Cvars
	{
		static bool bUseCVDDynamicMeshGenerator = true;
		static FAutoConsoleVariableRef CVarUseCVDDynamicMeshGenerator(
			TEXT("p.Chaos.VD.Tool.UseCVDDynamicMeshGenerator"),
			bUseCVDDynamicMeshGenerator,
			TEXT("If true, when creating a dynamic mesh from a mesh generator, CVD will use it's own mesh creation logic which included error handling that tries to repair broken geometry"));

		static bool bDisableUVsSupport = true;
		static FAutoConsoleVariableRef CVarDisableUVsSupport(
			TEXT("p.Chaos.VD.Tool.DisableUVsSupport"),
			bDisableUVsSupport,
			TEXT("If true, the generated meshes will not have UV data"));

		static float GeometryGenerationTaskLaunchBudgetSeconds = 0.005f;
		static FAutoConsoleVariableRef CVarGeometryGenerationTaskLaunchBudgetSeconds(
			TEXT("p.Chaos.VD.Tool.GeometryGenerationTaskLaunchBudgetSeconds"),
			GeometryGenerationTaskLaunchBudgetSeconds,
			TEXT("How much time we can spend on the Geoemtry builder tick launching Geometry Generation Tasks"));
	
		static bool bDeduplicateSimpleGeometry = true;
		static FAutoConsoleVariableRef CVarDeduplicateSimpleGeometry(
			TEXT("p.Chaos.VD.Tool.DeduplicateSimpleGeometry"),
			bDeduplicateSimpleGeometry,
			TEXT("If set to true, Box and Spheres will be represented with pre-made static meshes with a calculated scale based on the implicit object data"));
	}

	void SetTriangleAttributes(const UE::Geometry::FMeshShapeGenerator& Generator, FDynamicMesh3& OutDynamicMesh, int32 AppendedTriangleID, int32 GeneratorTriangleIndex)
	{
		UE::Geometry::FDynamicMeshUVOverlay* UVOverlay = OutDynamicMesh.Attributes()->PrimaryUV();
		UE::Geometry::FDynamicMeshNormalOverlay* NormalOverlay = OutDynamicMesh.Attributes()->PrimaryNormals();

		if (UVOverlay &&Generator.TriangleUVs.IsValidIndex(GeneratorTriangleIndex))
		{
			UVOverlay->SetTriangle(AppendedTriangleID, Generator.TriangleUVs[GeneratorTriangleIndex]);
		}
		
		if (ensure(NormalOverlay && Generator.TriangleUVs.IsValidIndex(GeneratorTriangleIndex)))
		{
			NormalOverlay->SetTriangle(AppendedTriangleID, Generator.TriangleNormals[GeneratorTriangleIndex]);
		}
	}

	void HandleTriangleAddedToDynamicMesh(const UE::Geometry::FMeshShapeGenerator& Generator, FDynamicMesh3& OutDynamicMesh, int32 TriangleIDResult, int32 GroupID, int32 GeneratorTriangleIndex, int32& OutSkippedTriangles, bool bAttemptToFixNoManifoldError = true)
	{
		// If we get a triangle ID greater than 0 means the add triangle operation didn't generate an error itself
		// But we still need to take into account skipped triangles to verify that we have valid data for this triangle in the mesh generator
		const bool bHasUnhandledError = TriangleIDResult < 0 ? true : (TriangleIDResult + OutSkippedTriangles) != GeneratorTriangleIndex;

		if (!bHasUnhandledError)
		{
			SetTriangleAttributes(Generator, OutDynamicMesh, TriangleIDResult, GeneratorTriangleIndex);
			return;
		}

		if (TriangleIDResult == FDynamicMesh3::NonManifoldID && bAttemptToFixNoManifoldError)
		{
			// If we get to here, it means we have more than two triangles sharing the same edge.
			// So lets try to conserve the original geometry by cloning the vertices and creating a new triangle with these
			// Visually should be mostly ok, although technically this triangle will be "detached"
			const UE::Geometry::FIndex3i& TriangleData = Generator.Triangles[GeneratorTriangleIndex];
			UE::Geometry::FIndex3i DuplicatedVertices(
				OutDynamicMesh.AppendVertex(OutDynamicMesh.GetVertex(TriangleData.A)),
				OutDynamicMesh.AppendVertex(OutDynamicMesh.GetVertex(TriangleData.B)),
				OutDynamicMesh.AppendVertex(OutDynamicMesh.GetVertex(TriangleData.C))
			);

			const int32 RepairedTriangleID = OutDynamicMesh.AppendTriangle(DuplicatedVertices, GroupID);

			UE_LOGF(LogChaosVDEditor, Verbose, "Failed to add triangle | [%d] but expected [%d] | Attempting to fix it ... Repaired triangle ID [%d]", TriangleIDResult, GeneratorTriangleIndex, RepairedTriangleID);

			// Only attempt to fix once
			constexpr bool bShouldAttemptToFixNoManifoldError = false;
			HandleTriangleAddedToDynamicMesh(Generator, OutDynamicMesh, RepairedTriangleID, GroupID, GeneratorTriangleIndex, OutSkippedTriangles, bShouldAttemptToFixNoManifoldError);
			return;
		}

		if (TriangleIDResult == FDynamicMesh3::DuplicateTriangleID)
		{
			OutSkippedTriangles++;
			UE_LOGF(LogTemp, Verbose, "Failed to add triangle | [%d] but expected [%d] | Ignoring Duplicated triangle.", TriangleIDResult, GeneratorTriangleIndex);
			return;
		}

		OutSkippedTriangles++;
		UE_LOGF(LogTemp, Error, "Failed to add triangle | [%d] but expected [%d]. This geometry will have missing triangles.", TriangleIDResult, GeneratorTriangleIndex);

		ensure(!bHasUnhandledError);
	}

	void GenerateDynamicMeshFromGenerator(const UE::Geometry::FMeshShapeGenerator& Generator, FDynamicMesh3& OutDynamicMesh)
	{
		OutDynamicMesh.Clear();

		OutDynamicMesh.EnableTriangleGroups();

		if (ensure(Generator.HasAttributes()))
		{
			OutDynamicMesh.EnableAttributes();
		}
		else
		{
			UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Attempted to created a mesh using a generator without attributes. CVD Meshes requiere attributes, this should have not happened.", ANSI_TO_TCHAR(__FUNCTION__));
			return;
		}

		const int32 NumVerts = Generator.Vertices.Num();
		for (int32 VertexIndex = 0; VertexIndex < NumVerts; ++VertexIndex)
		{
			OutDynamicMesh.AppendVertex(Generator.Vertices[VertexIndex]);
		}

		if (Cvars::bDisableUVsSupport)
		{
			// Remove the default UV Layer
			OutDynamicMesh.Attributes()->SetNumUVLayers(0);
		}
		else if (UE::Geometry::FDynamicMeshUVOverlay* UVOverlay = OutDynamicMesh.Attributes()->PrimaryUV())
		{
			const int32 NumUVs = Generator.UVs.Num();
			for (int32 UVIndex = 0; UVIndex < NumUVs; ++UVIndex)
			{
				UVOverlay->AppendElement(Generator.UVs[UVIndex]);
			}
		}

		if (UE::Geometry::FDynamicMeshNormalOverlay* NormalOverlay = OutDynamicMesh.Attributes()->PrimaryNormals())
		{
			const int32 NumNormals = Generator.Normals.Num();
            for (int32 NormalIndex = 0; NormalIndex < NumNormals; ++NormalIndex)
            {
            	NormalOverlay->AppendElement(Generator.Normals[NormalIndex]);
            }
		}

		int32 SkippedTriangles = 0;
		const int32 NumTris = Generator.Triangles.Num();
		for (int32 GeneratorTriangleIndex = 0; GeneratorTriangleIndex < NumTris; ++GeneratorTriangleIndex)
		{
			const int32 PolygonGroupID = Generator.TrianglePolygonIDs.Num() > 0 ? 1 + Generator.TrianglePolygonIDs[GeneratorTriangleIndex] : 0;
			const int32 ResultingTriangleID = OutDynamicMesh.AppendTriangle(Generator.Triangles[GeneratorTriangleIndex], PolygonGroupID);
		
			constexpr bool bShouldAttemptToFixNoManifoldError = true;
			HandleTriangleAddedToDynamicMesh(Generator, OutDynamicMesh, ResultingTriangleID, PolygonGroupID, GeneratorTriangleIndex, SkippedTriangles, bShouldAttemptToFixNoManifoldError);	
		}
	}
}

FChaosVDGeometryBuilder::~FChaosVDGeometryBuilder()
{
	DeInitialize();
}

void FChaosVDGeometryBuilder::CachePreBuiltMeshes()
{
	using namespace Chaos;
	if (UChaosVDCoreSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCoreSettings>())
	{
		// Note: Sphere and Box are 0 and 1 respectively, which is the key value we would get if we hash them. I expect this to not cause a collision, but if it does, we will need
		// to create buckets per implicit object type, or we could just create a new instance of this geometry builder per type
		// CVD already supports multiple builders.
		// If this unlikely scenario happens, there is an ensure that should trigger in FChaosVDGeometryBuilder::ExtractGeometryDataForImplicit
		StaticMeshCacheMap.Add(ImplicitObjectType::Box, Settings->BoxMesh.Get());
		StaticMeshCacheMap.Add(ImplicitObjectType::Sphere, Settings->SphereMesh.Get());
	}
}

void FChaosVDGeometryBuilder::Initialize(const TWeakPtr<FChaosVDScene>& ChaosVDScene)
{
	if (!ChaosVDScene.Pin())
	{
		return;
	}

	SceneWeakPtr = ChaosVDScene;

	auto ProcessMeshComponent = [WeakThis = AsWeak()](uint32 GeometryKey, const TWeakObjectPtr<UMeshComponent> Object)
	{
		const TSharedPtr<FChaosVDGeometryBuilder> GeometryBuilder = WeakThis.Pin();
		if (!GeometryBuilder)
		{
			UE_LOGF(LogChaosVDEditor, Verbose, " [%ls] Failed to update mesh for Handle | Geometry Key [%u] | Geometry Builder is invalid", ANSI_TO_TCHAR(__FUNCTION__), GeometryKey);

			// If the builder is no longer valid, just consume the request
			return true;
		}

		return GeometryBuilder->ApplyMeshToComponentFromKey(Object, GeometryKey);
	};

	auto ShouldProcessObjectsForKey = [WeakThis = AsWeak()](uint32 GeometryKey)
	{
		if (const TSharedPtr<FChaosVDGeometryBuilder> GeometryBuilder = WeakThis.Pin())
		{
			return GeometryBuilder->HasGeometryInCache(GeometryKey);
		}

		return false;
	};
	
	auto UpdateMeshMaterialForComponent = [WeakThis = AsWeak()](const TWeakObjectPtr<UMeshComponent> Object)
	{
		const TSharedPtr<FChaosVDGeometryBuilder> GeometryBuilder = WeakThis.Pin();
		if (!GeometryBuilder)
		{
			UE_LOGF(LogChaosVDEditor, Verbose, " [%ls] Failed to Create Material for Mesh | Geometry builder is no longer valid ", ANSI_TO_TCHAR(__FUNCTION__));

			// If the builder is no longer valid, just consume the request
			return true;
		}

		UMeshComponent* MeshComponent = Object.Get();
		if (IChaosVDGeometryComponent* CVDMeshComponent = Cast<IChaosVDGeometryComponent>(MeshComponent))
		{
			GeometryBuilder->SetMeshComponentMaterial(CVDMeshComponent);
		}
		return true;
	};

	auto LaunchGeometryGenerationTaskDeferred = [](TSharedPtr<FChaosVDGeometryGenerationTask> GeometryGenerationTask)
	{
		if(GeometryGenerationTask)
		{
			GeometryGenerationTask->TaskHandle = UE::Tasks::Launch(TEXT("GeometryGeneration"),
			[GeometryGenerationTask]()
			{
				if (GeometryGenerationTask)
				{
					if (GeometryGenerationTask->IsCanceled())
					{
						return;
					}
					GeometryGenerationTask->GenerateGeometry();
				}
			});
		}
	
		return true;
	};

	MeshComponentsWaitingForGeometry = MakeUnique<FObjectsWaitingGeometryList<FMeshComponentWeakPtr>>(ProcessMeshComponent, NSLOCTEXT("ChaosVisualDebugger", "GeometryGenNotification","Mesh Components"), ShouldProcessObjectsForKey);
	MeshComponentsWaitingForMaterial = MakeUnique<FObjectsWaitingProcessingQueue<FMeshComponentWeakPtr>>(UpdateMeshMaterialForComponent, NSLOCTEXT("ChaosVisualDebugger", "GeometryMaterialNotification","Component Materials"));
	GeometryTasksPendingLaunch = MakeUnique<FObjectsWaitingProcessingQueue<TSharedPtr<FChaosVDGeometryGenerationTask>>>(LaunchGeometryGenerationTaskDeferred, NSLOCTEXT("ChaosVisualDebugger", "GeometryTaskLauchNotification","Static Meshes"));

	GameThreadTickDelegate = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(this, &FChaosVDGeometryBuilder::GameThreadTick));

	constexpr int32 MeshPendingDisposalContainerDefaultSize = 500; 
	MeshComponentsPendingDisposal.Reserve(MeshPendingDisposalContainerDefaultSize);

	CachePreBuiltMeshes();

	bInitialized = true;
}

void FChaosVDGeometryBuilder::DeInitialize()
{
	if (!bInitialized)
	{
		return;
	}

	constexpr float MaxAmountOfWork = 1.0f;
	const int32 WorkRemaining = GeometryBeingGeneratedByKey.Num() + StaticMeshCacheMap.Num();
	const float PercentagePerElement = 1.0f / static_cast<float>(WorkRemaining);

	FScopedSlowTask CleaningGeometrySlowTask(MaxAmountOfWork, NSLOCTEXT("ChaosVisualDebugger", "DeInitializeGeometrybuilderSlowTask", "Deinitialiing GeometryBuilder"));

	FTSTicker::RemoveTicker(GameThreadTickDelegate);

	int32 TasksFailedToCancelNum = 0;

	for (const TPair<uint32, TSharedPtr<FChaosVDGeometryGenerationTask>>& TaskWithID : GeometryBeingGeneratedByKey)
	{
		if (TaskWithID.Value)
		{
			TaskWithID.Value->CancelTask();
		}

		if (!TaskWithID.Value->TaskHandle.Wait(FTimespan::FromSeconds(10.0f)))
		{
			TasksFailedToCancelNum++;
		}

		CleaningGeometrySlowTask.EnterProgressFrame(PercentagePerElement);
	}
	
	if (TasksFailedToCancelNum > 0)
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Failed to cancel [%d] tasks", ANSI_TO_TCHAR(__FUNCTION__), TasksFailedToCancelNum);
	}
	
	GeometryBeingGeneratedByKey.Reset();

	FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.RemoveAll(this);

	for (const TPair<uint32, TObjectPtr<UStaticMesh>>& StaticMeshByKey : StaticMeshCacheMap)
	{
		const TObjectPtr<UStaticMesh>& StaticMesh = StaticMeshByKey.Value;
		if (StaticMesh && !StaticMesh->IsAsset())
		{
			StaticMesh->ClearFlags(RF_Standalone);
			StaticMesh->MarkAsGarbage();
		}
		CleaningGeometrySlowTask.EnterProgressFrame(PercentagePerElement);
	}

	MeshComponentsPendingDisposal.Reset();
	TranslucentMirroredInstancedMeshComponentByGeometryKey.Reset();
	MirroredInstancedMeshComponentByGeometryKey.Reset();
	TranslucentInstancedMeshComponentByGeometryKey.Reset();
	InstancedMeshComponentByGeometryKey.Reset();
	GeometryTasksPendingLaunch.Reset();
	MeshComponentsWaitingForMaterial.Reset();
	MeshComponentsWaitingForGeometry.Reset();
	StaticMeshCacheMap.Reset();
	SourceGeometryCache.Reset();

	bInitialized = false;
}

void FChaosVDGeometryBuilder::CreateMeshesFromImplicitObject(const Chaos::FImplicitObject* InImplicitObject, TArray<TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>>& OutMeshDataHandles, Chaos::VD::FChaosVDImplicitObjectView* Root, int32 AvailableShapeDataNum , const int32 DesiredLODCount, const Chaos::FRigidTransform3& InTransform, const int32 MeshIndex)
{
	// To start set the leaf and the root to the same ptr. If the object is an union, in the subsequent recursive call the leaf will be set correctly
	CreateMeshesFromImplicit_Internal(InImplicitObject, InImplicitObject, OutMeshDataHandles, Root, DesiredLODCount, InTransform, MeshIndex, AvailableShapeDataNum);
}

void FChaosVDGeometryBuilder::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddStableReferenceMap(StaticMeshCacheMap);
	Collector.AddReferencedObjects(MeshComponentsPendingDisposal);
}

bool FChaosVDGeometryBuilder::DoesImplicitContainType(const Chaos::FImplicitObject* InImplicitObject, const Chaos::EImplicitObjectType ImplicitTypeToCheck)
{
	using namespace Chaos;
	
	if (!InImplicitObject)
	{
		return false;
	}

	const EImplicitObjectType InnerType = GetInnerType(InImplicitObject->GetType());

	switch (InnerType)
	{
		case ImplicitObjectType::Union:
		case ImplicitObjectType::UnionClustered:
			{
				if (const FImplicitObjectUnion* Union = InImplicitObject->template AsA<FImplicitObjectUnion>())
				{
					const TArray<Chaos::FImplicitObjectPtr>& UnionObjects = Union->GetObjects();
					for (const FImplicitObjectPtr& UnionImplicit : UnionObjects)
					{
						if (DoesImplicitContainType(UnionImplicit.GetReference(), ImplicitTypeToCheck))
						{
							return true;
						}
					}
				}
				return false;
			}
		case ImplicitObjectType::Transformed:
			{
				const TImplicitObjectTransformed<FReal, 3>* Transformed = InImplicitObject->template GetObject<TImplicitObjectTransformed<FReal, 3>>();
				return DoesImplicitContainType(Transformed->GetTransformedObject(), ImplicitTypeToCheck);
			}
	default:
		return InnerType == ImplicitTypeToCheck;
	}
}

bool FChaosVDGeometryBuilder::HasNegativeScale(const Chaos::FRigidTransform3& InTransform)
{
	const FVector ScaleSignVector = InTransform.GetScale3D().GetSignVector();
	return ScaleSignVector.X * ScaleSignVector.Y * ScaleSignVector.Z < 0;
}

void FChaosVDGeometryBuilder::CreateMeshesFromImplicit_Internal(const Chaos::FImplicitObject* InRootImplicitObject, const Chaos::FImplicitObject* InLeafImplicitObject, TArray<TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>>& OutMeshDataHandles, Chaos::VD::FChaosVDImplicitObjectView* Root, const int32 DesiredLODCount, const Chaos::FRigidTransform3& InTransform, const int32 ParentShapeInstanceIndex, int32 AvailableShapeDataNum)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FChaosVDGeometryBuilder::CreateMeshesFromImplicit_Internal);

	using namespace Chaos;

	const EImplicitObjectType InnerType = GetInnerType(InLeafImplicitObject->GetType());

	TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView> ImplicitView = MakeShared<Chaos::VD::FChaosVDImplicitObjectView>();
	ImplicitView->ImplicitObjectType = InnerType;

	Root->AddChild(ImplicitView);
	
	if (InnerType == ImplicitObjectType::Union || InnerType == ImplicitObjectType::UnionClustered)
	{
		if (const FImplicitObjectUnion* Union = InLeafImplicitObject->template AsA<FImplicitObjectUnion>())
		{
			const bool bIsRootUnion = InRootImplicitObject == InLeafImplicitObject;

			const bool bIsCluster = InnerType == ImplicitObjectType::UnionClustered;

			for (int32 ObjectIndex = 0; ObjectIndex < Union->GetObjects().Num(); ++ObjectIndex)
			{
				const FImplicitObjectPtr& UnionImplicit = Union->GetObjects()[ObjectIndex];

				int32 CurrentShapeInstanceIndex = ParentShapeInstanceIndex;

				if (bIsRootUnion)
				{
					if (bIsCluster)
					{
						// Geometry Collections might break the usual rule of how may shape data instances we have per geometry
						// Sometimes they can create clusters where all particles share a single instance
						constexpr int32 SingleShapeInstanceDataIndex = 0;
						CurrentShapeInstanceIndex = AvailableShapeDataNum == 1 ? SingleShapeInstanceDataIndex : ParentShapeInstanceIndex;
					}
					else
					{
						// If this union it is not the root implicit object, and it is not a cluster, then all its objects will share the same Instance index
						CurrentShapeInstanceIndex = ObjectIndex;
					}
				}

				CreateMeshesFromImplicit_Internal(InRootImplicitObject, UnionImplicit.GetReference(), OutMeshDataHandles, ImplicitView.Get(), DesiredLODCount, InTransform, CurrentShapeInstanceIndex, AvailableShapeDataNum);
			}
		}

		return;
	}

	if (InnerType == ImplicitObjectType::Transformed)
	{
		if (const TImplicitObjectTransformed<FReal, 3>* Transformed = InLeafImplicitObject->template GetObject<TImplicitObjectTransformed<FReal, 3>>())
		{
			// For transformed objects, the Instance index is the same so we pass it in without changing it
			CreateMeshesFromImplicit_Internal(InRootImplicitObject, Transformed->GetTransformedObject(), OutMeshDataHandles, ImplicitView.Get(), DesiredLODCount, Transformed->GetTransform(), ParentShapeInstanceIndex, AvailableShapeDataNum);
		}
		
		return;
	}

	if (const TSharedPtr<FChaosVDExtractedGeometryDataHandle> MeshDataHandle = ExtractGeometryDataForImplicit(InLeafImplicitObject, InTransform))
	{
		MeshDataHandle->SetImplicitObject(InLeafImplicitObject);
		MeshDataHandle->SetShapeInstanceIndex(ParentShapeInstanceIndex);
		MeshDataHandle->SetRootImplicitObject(InRootImplicitObject);

		OutMeshDataHandles.Add(TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>(MeshDataHandle, ImplicitView));
	}
}

bool FChaosVDGeometryBuilder::HasGeometryInCache(uint32 GeometryKey)
{
	FReadScopeLock ReadLock(GeometryCacheRWLock);
	return HasGeometryInCache_AssumesLocked(GeometryKey);
}

bool FChaosVDGeometryBuilder::HasGeometryInCache_AssumesLocked(uint32 GeometryKey) const
{
	return StaticMeshCacheMap.Contains(GeometryKey);
}

UStaticMesh* FChaosVDGeometryBuilder::GetCachedMeshForImplicit(const uint32 GeometryCacheKey)
{
	if (const TObjectPtr<UStaticMesh>* MeshPtrPtr = StaticMeshCacheMap.Find(GeometryCacheKey))
	{
		return MeshPtrPtr->Get();
	}

	return nullptr;
}

UStaticMesh* FChaosVDGeometryBuilder::CreateAndCacheStaticMesh(const uint32 GeometryCacheKey, UE::Geometry::FMeshShapeGenerator& MeshGenerator, const int32 LODsToGenerateNum)
{
	{
		FReadScopeLock ReadLock(GeometryCacheRWLock);
		if (TObjectPtr<UStaticMesh>* StaticMeshPtrPtr = StaticMeshCacheMap.Find(GeometryCacheKey))
		{
			return *StaticMeshPtrPtr;
		}
	}

	//TODO: Instead of generating a dynamic mesh and discard it, we should
	// Create a Mesh description directly when no LODs are required.
	// We could create a base class for our mesh Generators and add a Generate method that generates these mesh descriptions
	UStaticMesh* MainStaticMesh = NewObject<UStaticMesh>();
	MainStaticMesh->GetStaticMaterials().Add(FStaticMaterial());

	const int32 MeshDescriptionsToGenerate = LODsToGenerateNum + 1;

	TArray<const FMeshDescription*> LODDescriptions;
	LODDescriptions.Reserve(MeshDescriptionsToGenerate);

	MainStaticMesh->SetNumSourceModels(MeshDescriptionsToGenerate);

	FDynamicMesh3 DynamicMesh;

	if (Chaos::VisualDebugger::Cvars::bUseCVDDynamicMeshGenerator)
	{
		Chaos::VisualDebugger::GenerateDynamicMeshFromGenerator(MeshGenerator.Generate(), DynamicMesh);
	}
	else
	{
		DynamicMesh.Copy(&MeshGenerator.Generate());
	}

	for (int32 i = 0; i < MeshDescriptionsToGenerate; i++)
	{
		if (i > 0)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(FChaosVDGeometryBuilder::CreateAndCacheStaticMesh_LOD);
			//TODO: Come up with a better algo for this.
			const int32 DesiredTriangleCount = DynamicMesh.TriangleCount() / (i * 2);
			// Simplify
			UE::Geometry::FMeshConstraints Constraints;
			UE::Geometry::FMeshConstraintsUtil::ConstrainAllBoundariesAndSeams(Constraints, DynamicMesh,
																			   UE::Geometry::EEdgeRefineFlags::NoFlip, UE::Geometry::EEdgeRefineFlags::NoConstraint, UE::Geometry::EEdgeRefineFlags::NoConstraint,
																			   false, false, true);
			// Reduce the same previous LOD Mesh on each iteration
			UE::Geometry::FQEMSimplification Simplifier(&DynamicMesh);
			Simplifier.SetExternalConstraints(MoveTemp(Constraints));
			Simplifier.SimplifyToTriangleCount(DesiredTriangleCount);
		}

		FMeshDescription* MeshDescription = new FMeshDescription();
		FStaticMeshAttributes Attributes(*MeshDescription);
		Attributes.Register();

		FDynamicMeshToMeshDescription Converter;
		Converter.Convert(&DynamicMesh, *MeshDescription, true);
		LODDescriptions.Add(MeshDescription);
	}

	{
		TRACE_CPUPROFILER_EVENT_SCOPE(FChaosVDGeometryBuilder::CreateAndCacheStaticMesh_BUILD);
		UStaticMesh::FBuildMeshDescriptionsParams Params;
		Params.bUseHashAsGuid = true;
		Params.bMarkPackageDirty = false;
		Params.bBuildSimpleCollision = false;
		Params.bCommitMeshDescription = false;
		Params.bFastBuild = true;

		MainStaticMesh->GetNaniteSettings().bEnabled = true;
		MainStaticMesh->BuildFromMeshDescriptions(LODDescriptions, Params);

		MainStaticMesh->SetAutoComputeLODScreenSize(true);
	}

	{
		FWriteScopeLock WriteLock(GeometryCacheRWLock);
		StaticMeshCacheMap.Add(GeometryCacheKey, MainStaticMesh);
	}

	for (const FMeshDescription* Desc : LODDescriptions)
	{
		delete Desc;
		Desc = nullptr;
	}

	LODDescriptions.Reset();

	return MainStaticMesh;
}
void FChaosVDGeometryBuilder::SetMeshComponentMaterial(IChaosVDGeometryComponent* GeometryComponent)
{
	if (!GeometryComponent)
	{
		return;
	}

	// The component could have been set back to the pool before it was processed.
	// which can happen if a recording is scrub back and forth too fast
	if (GeometryComponent->GetIsDestroyed())
	{
		return;
	}

	UMaterialInterface* Material = ComponentMeshPool.GetMaterialForType(GeometryComponent->GetMaterialType());
	ensure(Material);

	if (UMeshComponent* AsMeshComponent = Cast<UMeshComponent>(GeometryComponent))
	{
		AsMeshComponent->SetMaterial(0, Material);
	}
}

void FChaosVDGeometryBuilder::HandleNewGeometryData(const Chaos::FConstImplicitObjectPtr& Geometry, const uint32 GeometryID)
{
	if (!Geometry)
	{
		return;
	}

	// We use implicit object hashes to tie them to generated static meshes.
	// Calculating the hash each time we need to create a static mesh for it is too expensive
	// So we do it here on load as this happens on the Trace Analysis thread.
	// We intentionally do this with the inner objects as that is what we use to generate meshes

	//TODO: At some point this will slow down too much the trace analysis thread, affecting live debugging and loading times
	// If we reach that point we should implement a background tasks that primes the hash cache instead
	
	Geometry->VisitObjects([this](const Chaos::FImplicitObject* Implicit, const Chaos::FRigidTransform3& Transform, const int32 RootObjectIndex, const int32 ObjectIndex, const int32 LeafObjectIndex)
	{
		auto CacheImplicitObjectHashIfNeeded = [this](const Chaos::FImplicitObject* ImplicitObject)
		{
			if (!ImplicitObject)
			{
				return;
			}

			if (!SourceGeometryCache.HasGeometryInHashCache(ImplicitObject))
			{
				SourceGeometryCache.CacheImplicitObjectHash(ImplicitObject, ImplicitObject->GetTypeHash());
			}
		};

		CacheImplicitObjectHashIfNeeded(Implicit);
	
		if (ImplicitObjectNeedsUnpacking(Implicit))
		{
			Chaos::FRigidTransform3 ExtractedTransform;
			CacheImplicitObjectHashIfNeeded(UnpackImplicitObject(Implicit, ExtractedTransform));
		}
		return true;
	});
}

void FChaosVDGeometryBuilder::DestroyMeshComponent(UMeshComponent* MeshComponent)
{
	if (IChaosVDGeometryComponent* AsCVDGeometryComponent = Cast<IChaosVDGeometryComponent>(MeshComponent))
	{
		if (Cast<UChaosVDInstancedStaticMeshComponent>(MeshComponent))
		{
			EChaosVDMeshAttributesFlags MeshAttributes = AsCVDGeometryComponent->GetMeshComponentAttributeFlags();
			TMap<uint32, UChaosVDInstancedStaticMeshComponent*>& InstancedMeshComponentCache = GetInstancedStaticMeshComponentCacheMap(MeshAttributes);
			InstancedMeshComponentCache.Remove(AsCVDGeometryComponent->GetGeometryKey());
		}
		
		RemoveMeshComponentWaitingForGeometry(AsCVDGeometryComponent->GetGeometryKey(), MeshComponent);	
        AsCVDGeometryComponent->OnComponentEmpty()->RemoveAll(this);

		// Mark destroyed right away to avoid other system using the component by mistake
		AsCVDGeometryComponent->SetIsDestroyed(true);
	}

	MeshComponentsPendingDisposal.Add(MeshComponent);
}

void FChaosVDGeometryBuilder::RequestMaterialUpdate(UMeshComponent* MeshComponent)
{
	MeshComponentsWaitingForMaterial->EnqueueObject(MeshComponent);
}

TMap<uint32, UChaosVDInstancedStaticMeshComponent*>& FChaosVDGeometryBuilder::GetInstancedStaticMeshComponentCacheMap(EChaosVDMeshAttributesFlags MeshAttributeFlags)
{
	if (EnumHasAnyFlags(MeshAttributeFlags, EChaosVDMeshAttributesFlags::MirroredGeometry))
	{
		if (EnumHasAnyFlags(MeshAttributeFlags, EChaosVDMeshAttributesFlags::TranslucentGeometry))
		{
			return TranslucentMirroredInstancedMeshComponentByGeometryKey;
		}
		else
		{
			return MirroredInstancedMeshComponentByGeometryKey;
		}
	}
	else
	{
		if (EnumHasAnyFlags(MeshAttributeFlags, EChaosVDMeshAttributesFlags::TranslucentGeometry))
		{
			return TranslucentInstancedMeshComponentByGeometryKey;
		}
		else
		{
			return InstancedMeshComponentByGeometryKey;
		}
	}
}

bool FChaosVDGeometryBuilder::ApplyMeshToComponentFromKey(TWeakObjectPtr<UMeshComponent> MeshComponent, const uint32 GeometryKey)
{
	bool bApplyMeshRequestProcessed = false;
	if (!MeshComponent.IsValid())
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Failed to apply geometry with key [%d] | Mesh Component is invalid", ANSI_TO_TCHAR(__FUNCTION__), GeometryKey);

		// If the component is no longer valid, just consume the request
		bApplyMeshRequestProcessed = true;
		return bApplyMeshRequestProcessed;
	}

	IChaosVDGeometryComponent* DataComponent = Cast<IChaosVDGeometryComponent>(MeshComponent.Get());
	if (!DataComponent)
	{
		// If the component is valid but not of the correct type, just consume the request and log the error

		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed to apply geometry with key [%d] | Mesh component is not a ChaosVDGeometryDataComponent", ANSI_TO_TCHAR(__FUNCTION__), GeometryKey);

		bApplyMeshRequestProcessed = true;
		return bApplyMeshRequestProcessed;
	}

	if (HasGeometryInCache(GeometryKey))
	{
		if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(MeshComponent))
		{
			StaticMeshComponent->SetStaticMesh(GetCachedMeshForImplicit(GeometryKey));
		}

		DataComponent->SetIsMeshReady(true);
		DataComponent->OnMeshReady()->Broadcast(*DataComponent);
		bApplyMeshRequestProcessed = true;
	}
	else
	{
		UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] Failed to apply geometry with key [%u] | Geometry was not ready", ANSI_TO_TCHAR(__FUNCTION__), GeometryKey);
	}

	return bApplyMeshRequestProcessed;
}

TSharedPtr<UE::Geometry::FMeshShapeGenerator> FChaosVDGeometryBuilder::CreateMeshGeneratorForImplicitObject(const Chaos::FImplicitObject* InImplicit, float SimpleShapesComplexityFactor)
{
	using namespace Chaos;

	switch (GetInnerType(InImplicit->GetType()))
	{
		case ImplicitObjectType::Sphere:
		{
			if (const Chaos::FSphere* Sphere = InImplicit->template GetObject<Chaos::FSphere>())
			{
				TSharedPtr<UE::Geometry::FSphereGenerator> SphereGen = MakeShared<UE::Geometry::FSphereGenerator>();
				SphereGen->Radius = Sphere->GetRadiusf();
				SphereGen->NumTheta = static_cast<int32>(25 * SimpleShapesComplexityFactor);
				SphereGen->NumPhi =  static_cast<int32>(25 * SimpleShapesComplexityFactor);
				SphereGen->bPolygroupPerQuad = false;

				return SphereGen;	
			}
			break;
		}
		case ImplicitObjectType::Box:
		{
			if (const Chaos::TBox<FReal, 3>* Box = InImplicit->template GetObject<Chaos::TBox<FReal, 3>>())
			{
				TSharedPtr<UE::Geometry::FMinimalBoxMeshGenerator> BoxGen = MakeShared<UE::Geometry::FMinimalBoxMeshGenerator>();
				UE::Geometry::FOrientedBox3d OrientedBox;
				OrientedBox.Frame = UE::Geometry::FFrame3d(Box->Center());
				OrientedBox.Extents = Box->Extents() * 0.5;
				BoxGen->Box = OrientedBox;
				return BoxGen;
			}
			break;
		}
		case ImplicitObjectType::Capsule:
		{
			if (const Chaos::FCapsule* Capsule = InImplicit->template GetObject<Chaos::FCapsule>())
			{
				TSharedPtr<UE::Geometry::FCapsuleGenerator> CapsuleGenerator = MakeShared<UE::Geometry::FCapsuleGenerator>();
				CapsuleGenerator->Radius = FMath::Max(FMathf::ZeroTolerance, Capsule->GetRadiusf());
				CapsuleGenerator->SegmentLength = FMath::Max(FMathf::ZeroTolerance, Capsule->GetSegment().GetLength());
				CapsuleGenerator->NumHemisphereArcSteps = static_cast<int32>(12 * SimpleShapesComplexityFactor);
				CapsuleGenerator->NumCircleSteps = static_cast<int32>(12 * SimpleShapesComplexityFactor);

				return CapsuleGenerator;
			}

			break;
		}
		case ImplicitObjectType::Convex:
		{
			if (const Chaos::FConvex* Convex = InImplicit->template GetObject<Chaos::FConvex>())
			{
				TSharedPtr<FChaosVDConvexMeshGenerator> ConvexMeshGen = MakeShared<FChaosVDConvexMeshGenerator>();
				ConvexMeshGen->GenerateFromConvex(*Convex);
				return ConvexMeshGen;
			}
				
			break;
		}
		case ImplicitObjectType::TriangleMesh:
		{
			if (const Chaos::FTriangleMeshImplicitObject* TriangleMesh = InImplicit->template GetObject<Chaos::FTriangleMeshImplicitObject>())
			{
				TSharedPtr<FChaosVDTriMeshGenerator> TriMeshGen = MakeShared<FChaosVDTriMeshGenerator>();
				TriMeshGen->bReverseOrientation = true;
				TriMeshGen->GenerateFromTriMesh(*TriangleMesh);
				return TriMeshGen;
			}

			break;
		}
		case ImplicitObjectType::HeightField:
		{
			if (const Chaos::FHeightField* HeightField = InImplicit->template GetObject<Chaos::FHeightField>())
			{
				TSharedPtr<FChaosVDHeightFieldMeshGenerator> HeightFieldMeshGen = MakeShared<FChaosVDHeightFieldMeshGenerator>();
				HeightFieldMeshGen->bReverseOrientation = false;
				HeightFieldMeshGen->GenerateFromHeightField(*HeightField);
				return HeightFieldMeshGen;
			}
		
			break;
		}
		case ImplicitObjectType::Plane:
		case ImplicitObjectType::LevelSet:
		case ImplicitObjectType::TaperedCylinder:
		case ImplicitObjectType::Cylinder:
		{
			//TODO: Implement
			break;
		}
		default:
			break;
	}

	return nullptr;
}

const Chaos::FImplicitObject* FChaosVDGeometryBuilder::UnpackImplicitObject(const Chaos::FImplicitObject* InImplicitObject, Chaos::FRigidTransform3& InOutTransform) const
{
	using namespace Chaos;

	const EImplicitObjectType InnerType = GetInnerType(InImplicitObject->GetType());
	switch (InnerType)
	{
		case ImplicitObjectType::Convex:
			{
				return GetGeometryBasedOnPackedType<FConvex>(InImplicitObject, InOutTransform, InImplicitObject->GetType());
			}
		case ImplicitObjectType::TriangleMesh:
			{
				return GetGeometryBasedOnPackedType<FTriangleMeshImplicitObject>(InImplicitObject, InOutTransform, InImplicitObject->GetType());
			}
		case ImplicitObjectType::HeightField:
			{
				return GetGeometryBasedOnPackedType<FHeightField>(InImplicitObject, InOutTransform, InImplicitObject->GetType());
			}
	default:
			ensureMsgf(false, TEXT("Unpacking [%s] is not supported"), *GetImplicitObjectTypeName(InnerType).ToString());
			break;
	}

	return nullptr;
}

void FChaosVDGeometryBuilder::AdjustedTransformForImplicit(const Chaos::FImplicitObject* InImplicit, FTransform& OutAdjustedTransform, EChaosVDGeometryTransformGeneratorFlags Options)
{
	using namespace Chaos;
	
	EImplicitObjectType InnerType = GetInnerType(InImplicit->GetType());
	switch (InnerType)
	{
		// Currently, only capsules and spheres transforms needs to be re-adjusted to take into account non-zero center locations
		case ImplicitObjectType::Capsule:
		{
			if (const FCapsule* Capsule = InImplicit->template GetObject<FCapsule>())
			{
				// Re-adjust the location so the pivot is not the center of the capsule, and transform it based on the provided transform
				const FVector FinalLocation = OutAdjustedTransform.TransformPosition(FVector(Capsule->GetCenterf()) - FVector(Capsule->GetAxisf()) * Capsule->GetSegment().GetLength() * 0.5f);
				const FQuat Rotation = FRotationMatrix::MakeFromZ(FVector(Capsule->GetAxisf())).Rotator().Quaternion();

				OutAdjustedTransform.SetRotation(OutAdjustedTransform.GetRotation() * Rotation);
				OutAdjustedTransform.SetLocation(FinalLocation);
			}
			break;
		}
		
		default:
			break;
	}

	if (EnumHasAnyFlags(Options, EChaosVDGeometryTransformGeneratorFlags::UseScaleForSize))
	{
		switch (InnerType)
		{
			case ImplicitObjectType::Sphere:
				{
					if (const Chaos::FSphere* Sphere = InImplicit->template GetObject<Chaos::FSphere>())
					{
						const FVector FinalLocation = OutAdjustedTransform.TransformPosition(FVector(Sphere->GetCenterf()));
						const FVector FinalScale = OutAdjustedTransform.GetScale3D() * (Sphere->GetRadiusf());
						OutAdjustedTransform.SetScale3D(FinalScale);
						OutAdjustedTransform.SetLocation(FinalLocation);

					}
					break;
				}
			case ImplicitObjectType::Box:
				{
					if (const Chaos::TBox<FReal, 3>* Box = InImplicit->template GetObject<Chaos::TBox<FReal, 3>>())
					{
						const FVector FinalLocation = OutAdjustedTransform.TransformPosition(FVector(Box->GetCenter()));
						const FVector FinalScale = OutAdjustedTransform.GetScale3D() * Box->Extents();
						OutAdjustedTransform.SetLocation(FinalLocation);
						OutAdjustedTransform.SetScale3D(FinalScale);
					}
				}
			default:
				break;
		}
	}
	else
	{
		switch (InnerType)
		{
			case ImplicitObjectType::Sphere:
				{
					if (const Chaos::FSphere* Sphere = InImplicit->template GetObject<Chaos::FSphere>())
					{
						const FVector FinalLocation = OutAdjustedTransform.TransformPosition(FVector(Sphere->GetCenterf()));
						OutAdjustedTransform.SetLocation(FinalLocation);
					}
					break;
				}
			default:
				break;
		}
	}
}

TSharedPtr<FChaosVDExtractedGeometryDataHandle> FChaosVDGeometryBuilder::ExtractGeometryDataForImplicit(const Chaos::FImplicitObject* InImplicitObject, const Chaos::FRigidTransform3& InTransform)
{
	const uint32 ImplicitObjectHash = SourceGeometryCache.GetAndCacheGeometryHash(InImplicitObject);

	Chaos::FRigidTransform3 ExtractedTransform = InTransform;
	const bool bNeedsUnpack = ImplicitObjectNeedsUnpacking(InImplicitObject);
	if (const Chaos::FImplicitObject* ImplicitObjectToProcess = bNeedsUnpack ? UnpackImplicitObject(InImplicitObject, ExtractedTransform) : InImplicitObject)
	{
		Chaos::EImplicitObjectType ImplicitObjectType = Chaos::GetInnerType(ImplicitObjectToProcess->GetType());
		TSharedPtr<FChaosVDExtractedGeometryDataHandle> MeshDataHandle = MakeShared<FChaosVDExtractedGeometryDataHandle>();

		uint32 GeometryKey = 0;
		if (UsesPreBuiltGeometry(ImplicitObjectType))
		{
			GeometryKey = GetTypeHash(ImplicitObjectType);

			MeshDataHandle->SetGeometryKey(GeometryKey);

			// For the Component data key, we need the hash of the implicit as it is (packed) because we will need to match it when looking for shape data
			MeshDataHandle->SetDataComponentKey(bNeedsUnpack ? ImplicitObjectHash : SourceGeometryCache.GetAndCacheGeometryHash(ImplicitObjectToProcess));
		}
		else
		{
			GeometryKey = SourceGeometryCache.GetAndCacheGeometryHash(ImplicitObjectToProcess);

			if (Chaos::VisualDebugger::Cvars::bDeduplicateSimpleGeometry &&
				!ensureMsgf(GeometryKey != Chaos::ImplicitObjectType::Box && GeometryKey != Chaos::ImplicitObjectType::Sphere, TEXT("A calculate geometry key is colliding with a deduplicated geometry key (either Box or Sphere). This should not happen. | Geometry key [%u] | Type [%u]"), GeometryKey, ImplicitObjectType))
			{
				UE_LOGF(LogChaosVDEditor, Verbose, "Geometry key [%u] | Implicit Object Type [%ls]", GeometryKey, *Chaos::GetImplicitObjectTypeName(ImplicitObjectType).ToString())
			}
			
			MeshDataHandle->SetGeometryKey(GeometryKey);

			// For the Component data key, we need the hash of the implicit as it is (packed) because we will need to match it when looking for shape data
			MeshDataHandle->SetDataComponentKey(bNeedsUnpack ? ImplicitObjectHash : GeometryKey);
		}

		if (!HasGeometryInCache(GeometryKey))
		{
			DispatchCreateAndCacheMeshForImplicitAsync(GeometryKey, ImplicitObjectToProcess);
		}

		EChaosVDGeometryTransformGeneratorFlags TransformUpdateFlags = Chaos::VisualDebugger::Cvars::bDeduplicateSimpleGeometry ? EChaosVDGeometryTransformGeneratorFlags::UseScaleForSize : EChaosVDGeometryTransformGeneratorFlags::None;
		AdjustedTransformForImplicit(ImplicitObjectToProcess, ExtractedTransform, TransformUpdateFlags);
		MeshDataHandle->SetTransform(ExtractedTransform);

		return MeshDataHandle;
	}

	return nullptr;
}

bool FChaosVDGeometryBuilder::ImplicitObjectNeedsUnpacking(const Chaos::FImplicitObject* InImplicitObject) const
{
	using namespace Chaos;
	const EImplicitObjectType InnerType = GetInnerType(InImplicitObject->GetType());

	return InnerType == ImplicitObjectType::Convex || InnerType == ImplicitObjectType::TriangleMesh ||  InnerType == ImplicitObjectType::HeightField;
}

bool FChaosVDGeometryBuilder::GameThreadTick(float DeltaTime)
{
	const float BudgetPerCategory = Chaos::VisualDebugger::Cvars::GeometryGenerationTaskLaunchBudgetSeconds / 3;

	if (GeometryTasksPendingLaunch)
	{
		GeometryTasksPendingLaunch->ProcessWaitingTasks(BudgetPerCategory);
	}

	if (MeshComponentsWaitingForGeometry)
	{
		MeshComponentsWaitingForGeometry->ProcessWaitingObjects(BudgetPerCategory);
	}

	if (MeshComponentsWaitingForMaterial)
	{
		MeshComponentsWaitingForMaterial->ProcessWaitingTasks(BudgetPerCategory);
	}

	for (TObjectPtr<UMeshComponent>& MeshComponentPtr : MeshComponentsPendingDisposal)
	{
		if (MeshComponentPtr)
		{
			ComponentMeshPool.DisposeMeshComponent(MeshComponentPtr);
		}
	}

	MeshComponentsPendingDisposal.Reset();

	return true;
}

void FChaosVDGeometryBuilder::AddMeshComponentWaitingForGeometry(uint32 GeometryKey, TWeakObjectPtr<UMeshComponent> MeshComponent) const
{
	if (!MeshComponent.IsValid())
	{
		UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] Failed to add mesh component update for geometry key [%d] | Mesh component is invalid", ANSI_TO_TCHAR(__FUNCTION__), GeometryKey);
		return;
	}

	if (!ensure(MeshComponentsWaitingForGeometry.IsValid()))
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed to add mesh component update for geometry key [%d] | WaitingListObject is invalid", ANSI_TO_TCHAR(__FUNCTION__), GeometryKey);
		return;
	}

	MeshComponentsWaitingForGeometry->AddObject(GeometryKey, MeshComponent);
}

void FChaosVDGeometryBuilder::RemoveMeshComponentWaitingForGeometry(uint32 GeometryKey, TWeakObjectPtr<UMeshComponent> MeshComponent) const
{
	if (!MeshComponent.IsValid())
    {
    	UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] Failed to remove mesh component update for geometry key [%d] | Mesh component is invalid", ANSI_TO_TCHAR(__FUNCTION__), GeometryKey);
    	return;
    }

	if (!ensure(MeshComponentsWaitingForGeometry.IsValid()))
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed to remove mesh component update for geometry key [%d] | WaitingListObject is invalid", ANSI_TO_TCHAR(__FUNCTION__), GeometryKey);
		return;
	}

	MeshComponentsWaitingForGeometry->RemoveObject(GeometryKey, MeshComponent);
}

void FChaosVDGeometryBuilder::RequestMeshForComponent(const TSharedRef<FChaosVDExtractedGeometryDataHandle>& SourceGeometry, UMeshComponent* MeshComponent)
{
	using namespace Chaos;

	AddMeshComponentWaitingForGeometry(SourceGeometry->GetGeometryKey(), MeshComponent);
}

bool FChaosVDGeometryBuilder::UsesPreBuiltGeometry(Chaos::EImplicitObjectType ObjetType) const
{
	return Chaos::VisualDebugger::Cvars::bDeduplicateSimpleGeometry ? ObjetType == Chaos::ImplicitObjectType::Box || ObjetType == Chaos::ImplicitObjectType::Sphere : false;
}

void FChaosVDGeometryBuilder::HandleStaticMeshComponentInstanceIndexUpdated(UInstancedStaticMeshComponent* InComponent, TArrayView<const FInstancedStaticMeshDelegates::FInstanceIndexUpdateData> InIndexUpdates)
{
	if (UChaosVDInstancedStaticMeshComponent* CVDInstancedStaticMeshComponent = Cast<UChaosVDInstancedStaticMeshComponent>(InComponent))
	{	
		CVDInstancedStaticMeshComponent->HandleInstanceIndexUpdated(InIndexUpdates);
	}
}

void FChaosVDGeometryBuilder::DispatchCreateAndCacheMeshForImplicitAsync(const uint32 GeometryKey,  const Chaos::FImplicitObject* ImplicitObject, const int32 LODsToGenerateNum)
{
	ensure(IsInGameThread());

	{
		FReadScopeLock ReadLock(GeometryCacheRWLock);
		if (GeometryBeingGeneratedByKey.Contains(GeometryKey))
		{
			return;
		}
	}

	TSharedPtr<FChaosVDGeometryGenerationTask> GenerationTask = MakeShared<FChaosVDGeometryGenerationTask>(AsWeak(), GeometryKey, ImplicitObject, LODsToGenerateNum);

	{
		FWriteScopeLock WriteLock(GeometryCacheRWLock);
		GeometryBeingGeneratedByKey.Add(GeometryKey, GenerationTask);
	}

	GeometryTasksPendingLaunch->EnqueueObject(GenerationTask);
}

void FChaosVDGeometryGenerationTask::GenerateGeometry()
{
	if (const TSharedPtr<FChaosVDGeometryBuilder> BuilderPtr = Builder.Pin())
	{
		if (TSharedPtr<UE::Geometry::FMeshShapeGenerator> MeshGenerator = BuilderPtr->CreateMeshGeneratorForImplicitObject(ImplicitObject))
		{
			BuilderPtr->CreateAndCacheStaticMesh(GeometryKey, *MeshGenerator.Get(), LODsToGenerateNum);
			{
				FWriteScopeLock WriteLock(BuilderPtr->GeometryCacheRWLock);
				BuilderPtr->GeometryBeingGeneratedByKey.Remove(GeometryKey);
			}
		}
	}
}
