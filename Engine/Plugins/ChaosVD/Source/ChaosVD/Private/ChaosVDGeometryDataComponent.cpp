// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGeometryDataComponent.h"

#include "Settings/ChaosVDCoreSettings.h"
#include "ChaosVDGeometryBuilder.h"
#include "ChaosVDModule.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDSettingsManager.h"
#include "Components/MeshComponent.h"
#include "Materials/Material.h"
#include "Materials/MaterialInterface.h"
#include "Settings/ChaosVDParticleVisualizationSettings.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "Visualizers/ChaosVDParticleDataComponentVisualizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDGeometryDataComponent)

TObjectPtr<UChaosVDParticleVisualizationColorSettings> FChaosVDGeometryComponentUtils::CachedParticleColorSettings = nullptr;
TObjectPtr<UChaosVDParticleVisualizationSettings> FChaosVDGeometryComponentUtils::CachedParticleVisualizationSettings = nullptr;

FChaosVDInstancedMeshData::FChaosVDInstancedMeshData(int32 InInstanceIndex, UMeshComponent* InMeshComponent, int32 InParticleID, int32 InSolverID, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InSourceGeometryHandle)
: ExtractedGeometryHandle(InSourceGeometryHandle)
{
	InstanceState.MeshComponent = InMeshComponent;
	InstanceState.MeshInstanceIndex = InInstanceIndex;
	InstanceState.OwningParticleID = InParticleID;
	InstanceState.OwningSolverID = InSolverID;

	if (Cast<UInstancedStaticMeshComponent>(InstanceState.MeshComponent))
	{
		InstanceState.MeshComponentType = EChaosVDMeshComponent::InstancedStatic;
	}
	else if (Cast<UStaticMeshComponent>(InstanceState.MeshComponent))
	{
		InstanceState.MeshComponentType = EChaosVDMeshComponent::Static;
	}
	else
	{
		InstanceState.MeshComponentType = EChaosVDMeshComponent::Dynamic;
	}

	ExtractedGeometryHandle = InSourceGeometryHandle;

	InstanceState.ImplicitObjectInfo.bIsRootObject = ExtractedGeometryHandle->GetRootImplicitObject() == ExtractedGeometryHandle->GetImplicitObject();
	InstanceState.ImplicitObjectInfo.ShapeInstanceIndex = ExtractedGeometryHandle->GetShapeInstanceIndex();
	InstanceState.ImplicitObjectInfo.ImplicitObjectType = ExtractedGeometryHandle->GetTypeName();
	InstanceState.ImplicitObjectInfo.ImplicitObjectTypeEnum = Chaos::GetInnerType(ExtractedGeometryHandle->GetImplicitObject()->GetType());
	InstanceState.ImplicitObjectInfo.RelativeTransform = ExtractedGeometryHandle->GetRelativeTransform();	
}

void FChaosVDInstancedMeshData::SetWorldTransform(const FTransform& InTransform)
{
	const FTransform ExtractedRelativeTransform = ExtractedGeometryHandle->GetRelativeTransform();

	InstanceState.CurrentWorldTransform.SetLocation(InTransform.TransformPosition(ExtractedRelativeTransform.GetLocation()));
	InstanceState.CurrentWorldTransform.SetRotation(InTransform.TransformRotation(ExtractedRelativeTransform.GetRotation()));
	InstanceState.CurrentWorldTransform.SetScale3D(ExtractedRelativeTransform.GetScale3D());
	
	if (IChaosVDGeometryComponent* CVDGeometryComponent = Cast<IChaosVDGeometryComponent>(GetMeshComponent()))
	{
		CVDGeometryComponent->UpdateWorldTransformForInstance(AsShared());
	}
}

void FChaosVDInstancedMeshData::SetInstanceColor(const FLinearColor& NewColor)
{
	if (InstanceState.CurrentGeometryColor != NewColor)
	{
		if (IChaosVDGeometryComponent* CVDGeometryComponent = Cast<IChaosVDGeometryComponent>(GetMeshComponent()))
		{
			InstanceState.CurrentGeometryColor = NewColor;
			CVDGeometryComponent->UpdateColorForInstance(AsShared());
		}
	}
}

void FChaosVDInstancedMeshData::UpdateMeshComponentForCollisionData(const FChaosVDShapeCollisionData& InCollisionData)
{
	if (InCollisionData.bIsValid && InstanceState.CollisionData != InCollisionData)
	{
		if (const TSharedPtr<FChaosVDGeometryBuilder> GeometryBuilderPtr = GeometryBuilderInstance.Pin())
		{
			EChaosVDMeshAttributesFlags RequiredMeshAttributes = EChaosVDMeshAttributesFlags::None;

			// If this is a query only type of geometry, we need a translucent mesh
			if (InCollisionData.bQueryCollision && !InCollisionData.bSimCollision)
			{
				EnumAddFlags(RequiredMeshAttributes, EChaosVDMeshAttributesFlags::TranslucentGeometry);
			}

			// Mirrored geometry needs to be on a instanced mesh component with reversed culling
			if (GeometryBuilderPtr->HasNegativeScale(ExtractedGeometryHandle->GetRelativeTransform()))
			{
				EnumAddFlags(RequiredMeshAttributes, EChaosVDMeshAttributesFlags::MirroredGeometry);
			}

			// If the current mesh component does not meet the required mesh attributes, we need to move to a new mesh component that it does
			bool bMeshComponentWasUpdated = false;
			if (IChaosVDGeometryComponent* CVDOldGeometryComponent = Cast<IChaosVDGeometryComponent>(GetMeshComponent()))
			{
				if (RequiredMeshAttributes != CVDOldGeometryComponent->GetMeshComponentAttributeFlags())
				{
					CVDOldGeometryComponent->RemoveMeshInstance(AsShared(), IChaosVDGeometryComponent::ERemovalMode::Instant);

					GeometryBuilderPtr->UpdateMeshDataInstance<UChaosVDInstancedStaticMeshComponent>(AsShared(), RequiredMeshAttributes);

					bMeshComponentWasUpdated = true;
				}
			}

			if (bMeshComponentWasUpdated)
			{
				if (IChaosVDGeometryComponent* CVDNewGeometryComponent = Cast<IChaosVDGeometryComponent>(GetMeshComponent()))
				{
					// Reset the color so it is updated in the next Update color calls (which always happens after updating the shape instance data)
					InstanceState.CurrentGeometryColor = FLinearColor(ForceInitToZero);
		
					CVDNewGeometryComponent->UpdateVisibilityForInstance(AsShared());
					CVDNewGeometryComponent->UpdateSelectionStateForInstance(AsShared());
				}
			}
		}
	}
}

void FChaosVDInstancedMeshData::SetGeometryCollisionData(const FChaosVDShapeCollisionData&& InCollisionData)
{
	// If this is a static mesh component, we can't just update change the material. We need to remove this instance from the current component and move it to a
	// component that has the correct translucent mesh
	if (GetMeshComponentType() == EChaosVDMeshComponent::InstancedStatic)
	{
		UpdateMeshComponentForCollisionData(InCollisionData);
	}

	InstanceState.CollisionData = InCollisionData;
}

void FChaosVDInstancedMeshData::SetIsSelected(bool bInIsSelected)
{
	InstanceState.bIsSelected = bInIsSelected;
	if (IChaosVDGeometryComponent* CVDGeometryComponent = Cast<IChaosVDGeometryComponent>(GetMeshComponent()))
	{
		CVDGeometryComponent->UpdateSelectionStateForInstance(AsShared());
	}
}

void FChaosVDInstancedMeshData::SetVisibility(bool bInIsVisible)
{
	if (InstanceState.bIsVisible != bInIsVisible)
	{
		InstanceState.bIsVisible = bInIsVisible;

		if (IChaosVDGeometryComponent* CVDGeometryComponent = Cast<IChaosVDGeometryComponent>(GetMeshComponent()))
		{
			CVDGeometryComponent->UpdateVisibilityForInstance(AsShared());
		}
	}
}

void FChaosVDGeometryComponentUtils::UpdateCollisionDataFromShapeArray(TArrayView<FChaosVDShapeCollisionData> InShapeArray, const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	FChaosVDMeshDataInstanceState& InstanceState = InInstanceHandle->GetState();

	if (UNLIKELY(!InShapeArray.IsValidIndex(InstanceState.ImplicitObjectInfo.ShapeInstanceIndex)))
	{
		const FName ImplicitObjectTypeName = InInstanceHandle->GetState().ImplicitObjectInfo.ImplicitObjectType;
		const TSharedRef<FChaosVDExtractedGeometryDataHandle>& ExtractedGeometryHandle = InInstanceHandle->GetGeometryHandle();
		const Chaos::FImplicitObject* RootImplicitObject = ExtractedGeometryHandle->GetRootImplicitObject();
		const FName RootImplicitObjectTypeName = !InInstanceHandle->GetState().ImplicitObjectInfo.bIsRootObject && RootImplicitObject ? Chaos::GetImplicitObjectTypeName(Chaos::GetInnerType(RootImplicitObject->GetType())) : TEXT("None");
		
		FString ErrorMessage = FString::Printf(TEXT("[%s] Failed to find shape instance data at Index [%d] | Particle ID[%d] | Available Shape instance Data Num [%d] | Implicit Type [%s] - Root Implicit Type [%s] | This geometry will be hidden..."), ANSI_TO_TCHAR(__FUNCTION__), InstanceState.ImplicitObjectInfo.ShapeInstanceIndex, InInstanceHandle->GetOwningParticleID(), InShapeArray.Num(), *ImplicitObjectTypeName.ToString(), *RootImplicitObjectTypeName.ToString());
		
		UE_LOGF(LogChaosVDEditor, Verbose, "[%ls]", *ErrorMessage);

		ensureMsgf(false, TEXT("[%s]"), *ErrorMessage);

		InInstanceHandle->bFailedToUpdateShapeInstanceData = true;
		return;
	}
	else if (InInstanceHandle->bFailedToUpdateShapeInstanceData)
	{
		InInstanceHandle->bFailedToUpdateShapeInstanceData = false;
		UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] Recovered from failing to find shape instance data at Index [%d] | Particle ID[%d] | Available Shape instance Data Num [%d] | This geometry will be shown again...", ANSI_TO_TCHAR(__FUNCTION__), InstanceState.ImplicitObjectInfo.ShapeInstanceIndex, InInstanceHandle->GetOwningParticleID(), InShapeArray.Num());
	}

	FChaosVDShapeCollisionData CollisionDataToUpdate = InShapeArray[InstanceState.ImplicitObjectInfo.ShapeInstanceIndex];
	CollisionDataToUpdate.bIsComplex = InstanceState.ImplicitObjectInfo.ImplicitObjectTypeEnum == Chaos::ImplicitObjectType::HeightField || InstanceState.ImplicitObjectInfo.ImplicitObjectTypeEnum == Chaos::ImplicitObjectType::TriangleMesh;
	CollisionDataToUpdate.bIsValid = true;

	InInstanceHandle->SetGeometryCollisionData(MoveTemp(CollisionDataToUpdate));
}

void FChaosVDGeometryComponentUtils::UpdateMeshColor(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle, const FChaosVDParticleDataWrapper& InParticleData, bool bIsServer)
{
	const FChaosVDShapeCollisionData& ShapeData = InInstanceHandle->GetGeometryCollisionData();
	const bool bIsQueryOnly = ShapeData.bQueryCollision && !ShapeData.bSimCollision;

	if (ShapeData.bIsValid)
	{
		FLinearColor ColorToApply = GetGeometryParticleColor(InInstanceHandle->GetGeometryHandle(), InParticleData, bIsServer);

		constexpr float QueryOnlyShapeOpacity = 0.6f;
		ColorToApply.A = bIsQueryOnly ? QueryOnlyShapeOpacity : 1.0f;

		InInstanceHandle->SetInstanceColor(ColorToApply);
	}
}

void FChaosVDGeometryComponentUtils::UpdateMeshVisibility(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle, const FChaosVDParticleDataWrapper& InParticleData, bool bIsActive)
{
	if (!bIsActive)
	{
		InInstanceHandle->SetVisibility(bIsActive);
		return;
	}

	if (const UChaosVDParticleVisualizationSettings* ParticleVisualizationSettings = GetParticleVisualizationSettings())
	{
		const EChaosVDGeometryVisibilityFlags CurrentVisibilityFlags = static_cast<EChaosVDGeometryVisibilityFlags>(ParticleVisualizationSettings->GeometryVisibilityFlags);
		
		bool bShouldGeometryBeVisible = false;

		if (!EnumHasAnyFlags(CurrentVisibilityFlags, EChaosVDGeometryVisibilityFlags::ShowDisabledParticles))
		{
			if (InParticleData.ParticleDynamicsMisc.HasValidData() && InParticleData.ParticleDynamicsMisc.bDisabled)
			{
				InInstanceHandle->SetVisibility(bShouldGeometryBeVisible);
				return;
			}
		}

		// TODO: Re-visit the way we determine visibility of the meshes.
		// Now that the options have grown and they will continue to do so, these checks are becoming hard to read and extend

		const bool bIsHeightfield = InInstanceHandle->GetState().ImplicitObjectInfo.ImplicitObjectTypeEnum == Chaos::ImplicitObjectType::HeightField;

		if (bIsHeightfield && EnumHasAnyFlags(CurrentVisibilityFlags, EChaosVDGeometryVisibilityFlags::ShowHeightfields))
		{
			bShouldGeometryBeVisible = true;
		}
		else
		{
			const FChaosVDShapeCollisionData& InstanceShapeData = InInstanceHandle->GetGeometryCollisionData();

			if (InstanceShapeData.bIsValid)
			{
				// Complex vs Simple takes priority although this is subject to change
				const bool bShouldBeVisibleIfComplex = InstanceShapeData.bIsComplex && EnumHasAnyFlags(CurrentVisibilityFlags, EChaosVDGeometryVisibilityFlags::Complex);
				const bool bShouldBeVisibleIfSimple = !InstanceShapeData.bIsComplex && EnumHasAnyFlags(CurrentVisibilityFlags, EChaosVDGeometryVisibilityFlags::Simple);
		
				if (bShouldBeVisibleIfComplex || bShouldBeVisibleIfSimple)
				{
					bShouldGeometryBeVisible = (InstanceShapeData.bSimCollision && EnumHasAnyFlags(CurrentVisibilityFlags, EChaosVDGeometryVisibilityFlags::Simulated))
					|| (InstanceShapeData.bQueryCollision && EnumHasAnyFlags(CurrentVisibilityFlags, EChaosVDGeometryVisibilityFlags::Query));
				}
			}
		}

		InInstanceHandle->SetVisibility(bShouldGeometryBeVisible);
	}
}

FLinearColor FChaosVDGeometryComponentUtils::GetGeometryParticleColor(const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InGeometryHandle, const FChaosVDParticleDataWrapper& InParticleData, bool bIsServer)
{
	constexpr FLinearColor DefaultColor(0.088542f, 0.088542f, 0.088542f);
	FLinearColor ColorToApply = DefaultColor;

	const UChaosVDParticleVisualizationColorSettings* VisualizationSettings = GetParticleColorSettings();
	if (!VisualizationSettings)
	{
		return ColorToApply;
	}

	switch (VisualizationSettings->ParticleColorMode)
	{
	case EChaosVDParticleDebugColorMode::ShapeType:
		{
			ColorToApply = InGeometryHandle->GetImplicitObject() ? VisualizationSettings->ColorsByShapeType.GetColorFromShapeType(Chaos::GetInnerType(InGeometryHandle->GetImplicitObject()->GetType())) : DefaultColor;
			break;
		}
	case EChaosVDParticleDebugColorMode::State:
		{
			if (InParticleData.Type == EChaosVDParticleType::Static)
			{
				ColorToApply = VisualizationSettings->ColorsByParticleState.GetColorFromState(EChaosVDObjectStateType::Static);
			}
			else
			{
				ColorToApply = VisualizationSettings->ColorsByParticleState.GetColorFromState(InParticleData.ParticleDynamicsMisc.MObjectState);
			}
			break;
		}
	case EChaosVDParticleDebugColorMode::ClientServer:
		{
			if (InParticleData.Type == EChaosVDParticleType::Static)
			{
				ColorToApply = VisualizationSettings->ColorsByClientServer.GetColorFromState(bIsServer, EChaosVDObjectStateType::Static);
			}
			else
			{
				ColorToApply = VisualizationSettings->ColorsByClientServer.GetColorFromState(bIsServer, InParticleData.ParticleDynamicsMisc.MObjectState);
			}
			break;
		}

	case EChaosVDParticleDebugColorMode::None:
	default:
		// Nothing to do here. Color to apply is already set to the default
		break;
	}

	return ColorToApply;
}

const UChaosVDParticleVisualizationColorSettings* FChaosVDGeometryComponentUtils::GetParticleColorSettings()
{
	if (CachedParticleColorSettings)
	{
		return CachedParticleColorSettings.Get();
	}

	CachedParticleColorSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationColorSettings>();

	return CachedParticleColorSettings.Get();
}

const UChaosVDParticleVisualizationSettings* FChaosVDGeometryComponentUtils::GetParticleVisualizationSettings()
{
	if (CachedParticleVisualizationSettings)
	{
		return CachedParticleVisualizationSettings.Get();
	}

	CachedParticleVisualizationSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationSettings>();

	return CachedParticleVisualizationSettings.Get();
}

UMaterialInterface* FChaosVDGeometryComponentUtils::GetBaseMaterialForType(EChaosVDMaterialType Type)
{
	const UChaosVDCoreSettings* EditorSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCoreSettings>();
	if (!EditorSettings)
	{
		return nullptr;
	}

	switch(Type)
	{
		case EChaosVDMaterialType::SMTranslucent:
				return EditorSettings->QueryOnlyMeshesMaterial.Get();
		case EChaosVDMaterialType::SMOpaque:
				return EditorSettings->SimOnlyMeshesMaterial.Get();
		case EChaosVDMaterialType::ISMCOpaque:
				return EditorSettings->InstancedMeshesMaterial.Get();
		case EChaosVDMaterialType::ISMCTranslucent:
				return EditorSettings->InstancedMeshesQueryOnlyMaterial.Get();
		default:
			return nullptr;
	}	
}

void Chaos::VisualDebugger::SelectParticleWithGeometryInstance(const TSharedRef<FChaosVDScene>& InScene, FChaosVDSceneParticle* Particle, const TSharedPtr<FChaosVDInstancedMeshData>& InMeshDataHandle)
{
	InScene->SetSelectedObject(nullptr);

	if (Particle)
	{
		Particle->SetSelectedMeshInstance(InMeshDataHandle);
		InScene->SetSelected(VD::TypedElementDataUtil::AcquireTypedElementHandleForStruct(Particle, true));	
	}
}
