// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDHeightfieldMeshGenerator.h"

#include "Chaos/HeightField.h"

void FChaosVDHeightFieldMeshGenerator::AppendTriangle(const Chaos::TVec2<Chaos::FReal>& InCellCoordinates, const UE::Geometry::FIndex3i& InTriangle, const Chaos::FHeightField& InHeightField, int32 PolygonID, int32 TriangleIndex)
{
	int32 StartNormalIndex = TriangleIndex * 3;
	FVector3f CellNormal(InHeightField.GetNormalAt(InCellCoordinates));

	for (int32 LocalVertexIndex = 0; LocalVertexIndex < 3; ++LocalVertexIndex)
	{
		Normals[StartNormalIndex + LocalVertexIndex] = CellNormal;
	}

	SetTriangle(TriangleIndex, InTriangle);
	SetTrianglePolygon(TriangleIndex, PolygonID);
	SetTriangleNormals(TriangleIndex, StartNormalIndex, StartNormalIndex + 1, StartNormalIndex + 2);
}

void FChaosVDHeightFieldMeshGenerator::GenerateFromHeightField(const Chaos::FHeightField& InHeightField)
{
	using namespace UE::Geometry;

	const int32 VertexGridNumRows = InHeightField.GetNumRows();
	const int32 VertexGridNumColumns = InHeightField.GetNumCols();
	const int32 VertexCount = VertexGridNumRows * VertexGridNumColumns;

	// If we consider each polygon/quad we create a cell of a new grid, the resulting grid will be one element smaller in both dimensions
	const int32 PolygonsGridNumRows = VertexGridNumRows -1;
	const int32 PolygonsGridNumColumns = VertexGridNumColumns -1;

	const int32 NumQuads = PolygonsGridNumRows * PolygonsGridNumColumns;
	const int32 NumTris = NumQuads * 2;

	const int32 NumNormals = NumTris * 3;
	constexpr int32 NumUVs = 0;
	
	SetBufferSizes(VertexCount, NumTris, NumUVs, NumNormals);

	// TODO: This value is not tuned yet
	constexpr int32 MaxRowsNumToProcessInSingleThread = 8;

	EParallelForFlags VerticesProcessingLoopFlags = VertexGridNumRows > MaxRowsNumToProcessInSingleThread ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None;

	// Fill the vertex buffer with the height data
	ParallelFor(VertexGridNumRows, [this, VertexGridNumColumns, &InHeightField](int32 RowIndex)
	{
		for (int32 ColumnIndex = 0; ColumnIndex < VertexGridNumColumns; ++ColumnIndex)
		{
			const int32 SampleIndex = RowIndex * VertexGridNumColumns + ColumnIndex;
			Vertices[SampleIndex] = InHeightField.GetPointScaled(SampleIndex);
		}
	}, VerticesProcessingLoopFlags);

	EParallelForFlags PolygonsRowsProcessingLoopFlags = PolygonsGridNumRows > MaxRowsNumToProcessInSingleThread ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None;

	FThreadSafeBool bHasHoles = false;

	ParallelFor(PolygonsGridNumRows, [this, VertexGridNumColumns, PolygonsGridNumColumns, &InHeightField, &bHasHoles](int32 RowIndex)
	{
		for (int32 ColumIndex = 0; ColumIndex < PolygonsGridNumColumns; ++ColumIndex)
		{
			if (InHeightField.IsHole(ColumIndex, RowIndex))
			{
				bHasHoles = true;
				continue;
			}

			Chaos::TVec2<Chaos::FReal> CellCoordinates(ColumIndex,RowIndex);

			// Vertices of each corner of the current cell
			// Starting from the vertex index in the actual vertex grid instead of the polygon one
			const int32 Vertex0 = RowIndex * VertexGridNumColumns + ColumIndex;
			const int32 Vertex1 = Vertex0 + 1;
			const int32 Vertex2 = Vertex0 + VertexGridNumColumns;
			const int32 Vertex3 = Vertex2 + 1;

			// Define the two triangles that form the current cell and add it to the generator
			FIndex3i Triangle(Vertex0,Vertex3,Vertex1);
			FIndex3i Triangle2(Vertex0,Vertex2,Vertex3);

			// Calculate which index in the polygon grid we are creating we are supposed to be in if it was represented in one-dimension array
			// And then use that as Polygon id for both triangles we are creating to for the Quad polygon
			const int32 PolygonID = RowIndex * PolygonsGridNumColumns + ColumIndex;

			// The start triangle index will always be 2x the polygon index, because we have two triangles per polygon
			constexpr int32 TrianglesPerCell = 2;
			const int32 LocalTriangleStartIndex = PolygonID * TrianglesPerCell;

			AppendTriangle(CellCoordinates, Triangle, InHeightField, PolygonID, LocalTriangleStartIndex);
			AppendTriangle(CellCoordinates, Triangle2, InHeightField, PolygonID, LocalTriangleStartIndex + 1);
		}
	}, PolygonsRowsProcessingLoopFlags);

	// If the heightmap has holes, then some triangle slots will be invalid. This causes warnings further down in the CVD mesh pipeline, so compact the triangle set here to remove all invalid triangles from holes.
	if (bHasHoles)
	{
		int32 WriteIndex = 0;
		for (int32 ReadIndex = 0; ReadIndex < NumTris; ReadIndex++)
		{
			const UE::Geometry::FIndex3i& TriangleIndices = Triangles[ReadIndex];

			//Only case for invalid indices in this generator are holes.
			if (TriangleIndices.A < 0 || TriangleIndices.B < 0 || TriangleIndices.C < 0)
			{
				continue;
			}
			if (WriteIndex != ReadIndex)
			{
				Triangles[WriteIndex] = TriangleIndices;
				TrianglePolygonIDs[WriteIndex] = TrianglePolygonIDs[ReadIndex];
				TriangleUVs[WriteIndex] = TriangleUVs[ReadIndex];
				TriangleNormals[WriteIndex] = TriangleNormals[ReadIndex];
			}
			WriteIndex++;
		}

		Triangles.SetNum(WriteIndex, EAllowShrinking::No);
		TrianglePolygonIDs.SetNum(WriteIndex, EAllowShrinking::No);
		TriangleUVs.SetNum(WriteIndex, EAllowShrinking::No);
		TriangleNormals.SetNum(WriteIndex, EAllowShrinking::No);
	}

	bIsGenerated = true;
}

UE::Geometry::FMeshShapeGenerator& FChaosVDHeightFieldMeshGenerator::Generate()
{
	ensureAlwaysMsgf(bIsGenerated, TEXT("You need to call FChaosVDHeightFieldMeshGenerator::GenerateFromHeightField before calling Generate"));
	return *this;
}
