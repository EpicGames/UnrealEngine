// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/Core.h"
#include "Generators/MeshShapeGenerator.h"

namespace Chaos
{
	class FHeightField;
}

/**
 * Generates an dynamic mesh using the data stored in a FHeightField implicit object
 */
class FChaosVDHeightFieldMeshGenerator : public UE::Geometry::FMeshShapeGenerator
{
public:

	/** Prepares the Mesh Generator so the Generate method can generate the desired Dynamic Mesh
	* @param InHeightField Implicit object used to generate the dynamic mesh
	*/
	void GenerateFromHeightField(const Chaos::FHeightField& InHeightField);

	virtual FMeshShapeGenerator& Generate() override;

private:
	void AppendTriangle(const Chaos::TVec2<Chaos::FReal>& InCellCoordinates, const UE::Geometry::FIndex3i& InTriangle, const Chaos::FHeightField& InHeightField, int32 PolygonID, int32 TriangleIndex);

	bool bIsGenerated = false;
};
