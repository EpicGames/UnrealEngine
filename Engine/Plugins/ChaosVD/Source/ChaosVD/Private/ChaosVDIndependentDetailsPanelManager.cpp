// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDIndependentDetailsPanelManager.h"

#include "ChaosVDObjectDetailsTab.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/SChaosVDMainTab.h"

FChaosVDIndependentDetailsPanelManager::FChaosVDIndependentDetailsPanelManager(const TSharedRef<SChaosVDMainTab>& InMainTab) : MainTab(InMainTab)
{
	AvailableDetailsPanelIDs.Add(FChaosVDTabID::IndependentDetailsPanel1);
	AvailableDetailsPanelIDs.Add(FChaosVDTabID::IndependentDetailsPanel2);
	AvailableDetailsPanelIDs.Add(FChaosVDTabID::IndependentDetailsPanel3);
	AvailableDetailsPanelIDs.Add(FChaosVDTabID::IndependentDetailsPanel4);

	InMainTab->RegisterTabSpawner<FChaosVDStandAloneObjectDetailsTab>(FChaosVDTabID::IndependentDetailsPanel1);
	InMainTab->RegisterTabSpawner<FChaosVDStandAloneObjectDetailsTab>(FChaosVDTabID::IndependentDetailsPanel2);
	InMainTab->RegisterTabSpawner<FChaosVDStandAloneObjectDetailsTab>(FChaosVDTabID::IndependentDetailsPanel3);
	InMainTab->RegisterTabSpawner<FChaosVDStandAloneObjectDetailsTab>(FChaosVDTabID::IndependentDetailsPanel4);
}

TSharedPtr<FChaosVDStandAloneObjectDetailsTab> FChaosVDIndependentDetailsPanelManager::GetAvailableStandAloneDetailsPanelTab()
{
	if (AvailableDetailsPanelIDs.IsEmpty())
	{
		return nullptr;
	}

	TSharedPtr<SChaosVDMainTab> OwningTabPtr = MainTab.Pin();
	if (!OwningTabPtr.IsValid())
	{
		return nullptr;
	}

	if (const TSharedPtr<FTabManager> TabManager = OwningTabPtr->GetTabManager())
	{
		FName AvailableTabID = AvailableDetailsPanelIDs.Pop();
		if (const TSharedPtr<FChaosVDStandAloneObjectDetailsTab> DetailsTab = OwningTabPtr->GetTabSpawnerInstance<FChaosVDStandAloneObjectDetailsTab>(AvailableTabID).Pin())
		{
			DetailsTab->OnTabDestroyed().AddSP(this, &FChaosVDIndependentDetailsPanelManager::HandleTabDestroyed, AvailableTabID);
			TabManager->TryInvokeTab(AvailableTabID);

			return DetailsTab;
		}
	}

	return nullptr;
}

void FChaosVDIndependentDetailsPanelManager::HandleTabDestroyed(TSharedRef<SDockTab> Tab, FName TabID)
{
	AvailableDetailsPanelIDs.Add(TabID);
	
	if (TSharedPtr<SChaosVDMainTab> OwningTabPtr = MainTab.Pin())
	{
		if (const TSharedPtr<FChaosVDStandAloneObjectDetailsTab> DetailsTab = OwningTabPtr->GetTabSpawnerInstance<FChaosVDStandAloneObjectDetailsTab>(TabID).Pin())
		{
			DetailsTab->OnTabDestroyed().RemoveAll(this);
		}
	}
}
