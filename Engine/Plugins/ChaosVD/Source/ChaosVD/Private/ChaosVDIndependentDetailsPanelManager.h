// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDTabsIDs.h"
#include "Templates/SharedPointer.h"

class SDockTab;
class FChaosVDStandAloneObjectDetailsTab;
class SChaosVDMainTab;

/**
 * Manager class that handles any selection independent details panel tab.
 */
class FChaosVDIndependentDetailsPanelManager : public TSharedFromThis<FChaosVDIndependentDetailsPanelManager>
{
public:
	FChaosVDIndependentDetailsPanelManager(const TSharedRef<SChaosVDMainTab>& InMainTab);

	/** Returns a shared ptr to a new selection independent details panel tab or null if no panels are available */
	TSharedPtr<FChaosVDStandAloneObjectDetailsTab> GetAvailableStandAloneDetailsPanelTab();

private:

	void HandleTabDestroyed(TSharedRef<SDockTab> Tab, FName TabID);

	TArray<FName> AvailableDetailsPanelIDs;
	
	TWeakPtr<SChaosVDMainTab> MainTab;
};
