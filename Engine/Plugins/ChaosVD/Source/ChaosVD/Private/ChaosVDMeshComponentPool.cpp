// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDMeshComponentPool.h"

#include "ChaosVDGeometryDataComponent.h"
#include "Components/ChaosVDStaticMeshComponent.h"
#include "UObject/Package.h"

void FChaosVDMeshComponentPool::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(CachedISMCOpaqueMaterialBase);
	Collector.AddReferencedObject(CachedISMCTranslucentBase);
	Collector.AddReferencedObject(CachedStaticMeshComponentTranslucentMaterialBase);
	Collector.AddReferencedObject(CachedStaticMeshComponentOpaqueMaterialBase);
}

FChaosVDMeshComponentPool::FChaosVDMeshComponentPool()
{
	CachedISMCOpaqueMaterialBase = FChaosVDGeometryComponentUtils::GetBaseMaterialForType(EChaosVDMaterialType::ISMCOpaque);
	CachedISMCTranslucentBase = FChaosVDGeometryComponentUtils::GetBaseMaterialForType(EChaosVDMaterialType::ISMCTranslucent);
	CachedStaticMeshComponentTranslucentMaterialBase = FChaosVDGeometryComponentUtils::GetBaseMaterialForType(EChaosVDMaterialType::SMTranslucent);
	CachedStaticMeshComponentOpaqueMaterialBase = FChaosVDGeometryComponentUtils::GetBaseMaterialForType(EChaosVDMaterialType::SMOpaque);

	StaticMeshComponentPool.SetPoolName(TEXT("Static Mesh Components Pool"));
	InstancedStaticMeshComponentPool.SetPoolName(TEXT("Instanced Static Mesh Components Pool"));
}

void FChaosVDMeshComponentPool::DisposeMeshComponent(UMeshComponent* MeshComponent)
{
	if (UChaosVDStaticMeshComponent* AsStaticMeshComponent = Cast<UChaosVDStaticMeshComponent>(MeshComponent))
	{
		StaticMeshComponentPool.DisposeObject(AsStaticMeshComponent);
	}
	else if (UChaosVDInstancedStaticMeshComponent* AsInstancedStaticMeshComponent = Cast<UChaosVDInstancedStaticMeshComponent>(MeshComponent))
	{
		InstancedStaticMeshComponentPool.DisposeObject(AsInstancedStaticMeshComponent);
	}
}

UMaterialInterface* FChaosVDMeshComponentPool::GetMaterialForType(EChaosVDMaterialType Type)
{
	switch(Type)
	{
	case EChaosVDMaterialType::SMTranslucent:
		return CachedStaticMeshComponentTranslucentMaterialBase;
	case EChaosVDMaterialType::SMOpaque:
		return CachedStaticMeshComponentOpaqueMaterialBase;
	case EChaosVDMaterialType::ISMCOpaque:
		return CachedISMCOpaqueMaterialBase;
	case EChaosVDMaterialType::ISMCTranslucent:
		return CachedISMCTranslucentBase;
	default:
		{
			ensure(false);
			return nullptr;
		}
		
	}	
}

