// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDObjectPool.h"
#include "Components/ChaosVDInstancedStaticMeshComponent.h"
#include "Components/ChaosVDStaticMeshComponent.h"
#include "Containers/Array.h"

class FChaosVDMeshComponentPool : public FGCObject
{
public:
	
	FChaosVDMeshComponentPool();

	virtual FString GetReferencerName() const override
	{
		return TEXT("FChaosVDMeshComponentPool");
	}
	
	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	template<typename TMeshComponent>
	TMeshComponent* AcquireMeshComponent(UObject* Outer, FName Name);

	void DisposeMeshComponent(UMeshComponent* MeshComponent);

	UMaterialInterface* GetMaterialForType(EChaosVDMaterialType Type);

private:

	TChaosVDObjectPool<UChaosVDStaticMeshComponent> StaticMeshComponentPool;
	TChaosVDObjectPool<UChaosVDInstancedStaticMeshComponent> InstancedStaticMeshComponentPool;

	TObjectPtr<UMaterialInterface> CachedISMCOpaqueMaterialBase;
	TObjectPtr<UMaterialInterface> CachedISMCTranslucentBase;
	TObjectPtr<UMaterialInterface> CachedStaticMeshComponentTranslucentMaterialBase;
	TObjectPtr<UMaterialInterface> CachedStaticMeshComponentOpaqueMaterialBase;
};

template <typename TMeshComponent>
TMeshComponent* FChaosVDMeshComponentPool::AcquireMeshComponent(UObject* Outer, FName Name)
{
	if constexpr (std::is_base_of_v<UStaticMeshComponent, TMeshComponent> && !std::is_base_of_v<UInstancedStaticMeshComponent, TMeshComponent>)
	{
		return StaticMeshComponentPool.AcquireObject(Outer, Name);
	}
	else if constexpr (std::is_base_of_v<UInstancedStaticMeshComponent, TMeshComponent>)
	{
		return InstancedStaticMeshComponentPool.AcquireObject(Outer, Name);
	}
	else
	{
		return nullptr;
	}
}
