// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDMiscSceneObjectTypeNamesForTesting.h"

namespace Chaos::VD::Test::SceneObjectTypes
{
	FName SceneQuery = FName("SceneQuery");
	FName CollisionConstraint = FName("CollisionConstraint");
	FName JointConstraint = FName("JointConstraint");
	FName CharacterGroundConstraint = FName("CharacterGroundConstraint");
}
