// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDModule.h"

#include "Chaos/ChaosVDEngineEditorBridge.h"
#include "ChaosVDCommands.h"
#include "ChaosVDEngine.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDRuntimeModule.h"
#include "ChaosVDSceneCompositionReport.h"
#include "ChaosVDSceneParticleCustomization.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "DesktopPlatformModule.h"
#include "ExtensionsSystem/ChaosVDExtensionsManager.h"
#include "Framework/Application/SlateApplication.h"
#include "HAL/FileManager.h"
#include "IDesktopPlatform.h"
#include "Misc/App.h"
#include "Misc/Guid.h"
#include "Settings/ChaosVDGeneralSettings.h"
#include "Trace/ChaosVDTraceModule.h"
#include "TraceSessionsManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SChaosVDMainTab.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

DEFINE_LOG_CATEGORY(LogChaosVDEditor);

FAutoConsoleCommand ChaosVDTakePlaybackEngineSnapshot(
	TEXT("p.Chaos.VD.TakePlaybackEngineSnapshot"),
	TEXT("Take a snapshot of CVD's playback engine at the current frame to be used in functional tests"),
	FConsoleCommandWithArgsDelegate::CreateLambda([](const TArray<FString>& Args)
	{
		FChaosVDModule::Get().TakePlaybackEngineStateSnapshot();
	})
);

FString FChaosVDModule::ChaosVisualDebuggerProgramName = TEXT("ChaosVisualDebugger");

FChaosVDModule& FChaosVDModule::Get()
{
	return FModuleManager::Get().LoadModuleChecked<FChaosVDModule>(TEXT("ChaosVD"));
}

void FChaosVDModule::StartupModule()
{
	FChaosVDStyle::Initialize();

	FChaosVDCommands::Register();

	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(FChaosVDTabID::ChaosVisualDebuggerTab, FOnSpawnTab::CreateRaw(this, &FChaosVDModule::SpawnMainTab))
								.SetDisplayName(LOCTEXT("VisualDebuggerTabTitle", "Chaos Visual Debugger"))
								.SetTooltipText(LOCTEXT("VisualDebuggerTabDesc", "Opens the Chaos Visual Debugger window"))
								.SetIcon(FSlateIcon(FChaosVDStyle::GetStyleSetName(), "ChaosVisualDebugger"))
								.SetGroup(WorkspaceMenu::GetMenuStructure().GetDeveloperToolsDebugCategory());

#if WITH_CHAOS_VISUAL_DEBUGGER
	ChaosVDTraceManager = MakeShared<UE::TraceBasedDebuggers::FTraceSessionsManager>(
		FChaosVDRuntimeModule::Get()
		, FChaosVDEngineEditorBridge::Get()
		, MakeShared<FChaosVDTraceModule>()
		, /*DefaultSavePath*/TEXT("ChaosVisualDebugger"));

	// Set options using lambdas so the most recent values from the settings are used
	// (i.e., users can change settings during the session while the trace manager is created only once)
	ChaosVDTraceManager->SetSaveRecordingsToDisk(MakeAttributeLambda([]
		{
			if (const UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
			{
				return Settings->bSaveRecordingsToDisk;
			}
			return false;
		}));

	ChaosVDTraceManager->SetSavePathOverride(MakeAttributeLambda([]
		{
			if (const UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
			{
				return Settings->DefaultSavePathOverride;
			}
			return FFilePath{};
		}));

	UE::Trace::ToggleChannel(TEXT("ChaosVD"), /*bEnabled*/true);
#endif // WITH_CHAOS_VISUAL_DEBUGGER

	if (IsStandaloneChaosVisualDebugger())
	{
		// In the standalone app, once the engine is initialized we need to spawn the main tab otherwise there will be no UI
		// because we intentionally don't load the mainframe / rest of the editor UI
		FCoreDelegates::OnFEngineLoopInitComplete.AddRaw(this, &FChaosVDModule::SpawnCVDTab);
	}

	FCoreDelegates::OnEnginePreExit.AddRaw(this, &FChaosVDModule::CloseActiveInstances);
}

void FChaosVDModule::ShutdownModule()
{
	FChaosVDStyle::Shutdown();
	
	FChaosVDCommands::Unregister();

	FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(FChaosVDTabID::ChaosVisualDebuggerTab);

	if (IsStandaloneChaosVisualDebugger())
	{
		FCoreDelegates::OnFEngineLoopInitComplete.RemoveAll(this);
	}

	FCoreDelegates::OnEnginePreExit.RemoveAll(this);

	CloseActiveInstances();

	FChaosVDExtensionsManager::TearDown();
	
	FChaosVDSettingsManager::TearDown();
}

void FChaosVDModule::SpawnCVDTab()
{
	if (IsStandaloneChaosVisualDebugger())
	{
		// In the standalone app, we need to load the status bar module so the status bar subsystem is initialized
		FModuleManager::Get().LoadModule("StatusBar");
	}

	FGlobalTabmanager::Get()->TryInvokeTab(FChaosVDTabID::ChaosVisualDebuggerTab);
}

TSharedRef<SDockTab> FChaosVDModule::SpawnMainTab(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> MainTabInstance =
		SNew(SDockTab)
		.TabRole(ETabRole::MajorTab)
		.Label(LOCTEXT("MainTabLabel", "Chaos Visual Debugger"))
		.ToolTipText(LOCTEXT("MainTabToolTip", "Chaos Visual Debugger is an experimental tool and it can be unstable"));

	// Initialize the Chaos VD Engine instance this tab will represent
	// For now its lifetime will be controlled by this tab
	const TSharedPtr<FChaosVDEngine> ChaosVDEngineInstance = MakeShared<FChaosVDEngine>();
	ChaosVDEngineInstance->Initialize();

	MainTabInstance->SetContent
	(
		SNew(SChaosVDMainTab, ChaosVDEngineInstance)
			.OwnerTab(MainTabInstance.ToSharedPtr())
	);

	const FGuid InstanceGuid = ChaosVDEngineInstance->GetInstanceGuid();
	RegisterChaosVDEngineInstance(InstanceGuid, ChaosVDEngineInstance);

	const SDockTab::FOnTabClosedCallback ClosedCallback = SDockTab::FOnTabClosedCallback::CreateRaw(this, &FChaosVDModule::HandleTabClosed, InstanceGuid);
	MainTabInstance->SetOnTabClosed(ClosedCallback);

	RegisterChaosVDTabInstance(InstanceGuid, MainTabInstance.ToSharedPtr());

	return MainTabInstance;
}

void FChaosVDModule::HandleTabClosed(TSharedRef<SDockTab> ClosedTab, FGuid InstanceGUID)
{
	if (IsStandaloneChaosVisualDebugger())
	{
		// If this is the standalone CVD app, we can assume that tab closed indicates an exit request
		RequestEngineExit(TEXT("MainCVDTabClosed"));
	}

	// Workaround. Currently the ChaosVD Engine instance determines the lifetime of the Editor world and other objects
	// Some widgets, like UE Level viewport tries to iterate on these objects on destruction
	// For now we can avoid any crashes by just de-initializing ChaosVD Engine on the next frame but that is not the real fix.
	// Unless we are shutting down the engine
	
	//TODO: Ensure that systems that uses the Editor World we create know beforehand when it is about to be Destroyed and GC'd
	// Related Jira Task UE-191876
	if (bIsShuttingDown)
	{
		DeregisterChaosVDTabInstance(InstanceGUID);
		DeregisterChaosVDEngineInstance(InstanceGUID);
	}
	else
	{
		DeregisterChaosVDTabInstance(InstanceGUID);

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, InstanceGUID](float DeltaTime)->bool
		{
			DeregisterChaosVDEngineInstance(InstanceGUID);
			return false;
		}));
	}
}

void FChaosVDModule::RegisterChaosVDEngineInstance(const FGuid& InstanceGuid, TSharedPtr<FChaosVDEngine> Instance)
{
	ActiveChaosVDInstances.Add(InstanceGuid, Instance);
}

void FChaosVDModule::DeregisterChaosVDEngineInstance(const FGuid& InstanceGuid)
{
	if (TSharedPtr<FChaosVDEngine>* InstancePtrPtr = ActiveChaosVDInstances.Find(InstanceGuid))
	{
		if (TSharedPtr<FChaosVDEngine> InstancePtr = *InstancePtrPtr)
		{
			InstancePtr->DeInitialize();
		}
	
		ActiveChaosVDInstances.Remove(InstanceGuid);
	}	
}

void FChaosVDModule::RegisterChaosVDTabInstance(const FGuid& InstanceGuid, TSharedPtr<SDockTab> Instance)
{
	ActiveCVDTabs.Add(InstanceGuid, Instance);
}

void FChaosVDModule::DeregisterChaosVDTabInstance(const FGuid& InstanceGuid)
{
	ActiveCVDTabs.Remove(InstanceGuid);
}

void FChaosVDModule::CloseActiveInstances()
{
	bIsShuttingDown = true;
	for (const TPair<FGuid, TWeakPtr<SDockTab>>& CVDTabWithID : ActiveCVDTabs)
	{
		if (TSharedPtr<SDockTab> CVDTab = CVDTabWithID.Value.Pin())
		{
			CVDTab->RequestCloseTab();
		}
		else
		{
			// if the tab Instance no longer exist, make sure the CVD engine instance is shutdown
			DeregisterChaosVDEngineInstance(CVDTabWithID.Key);
		}
	}

	ActiveChaosVDInstances.Reset();
	ActiveCVDTabs.Reset();
}

bool FChaosVDModule::IsStandaloneChaosVisualDebugger()
{
	return FPlatformProperties::IsProgram() && FApp::GetProjectName() == ChaosVisualDebuggerProgramName;
}

void FChaosVDModule::ReloadInstanceUI(FGuid InstanceGUID)
{
	// CVD UI is (or it should be) fully de-coupled from the tool's non-ui state
	// Therefore to reload its UI we can just re-spawn the tab widget and initialize it with the existing CVD engine instance

	TWeakPtr<SDockTab>* ActiveTabPtrPtr = ActiveCVDTabs.Find(InstanceGUID);
	if (TSharedPtr<SDockTab> TabPtr = ActiveTabPtrPtr ? ActiveTabPtrPtr->Pin() : nullptr)
	{
		TSharedPtr<FChaosVDEngine>* ChaosVDEngineInstancePtrPtr = ActiveChaosVDInstances.Find(InstanceGUID);
		if (TSharedPtr<FChaosVDEngine> ChaosVDEngineInstancePtr = ChaosVDEngineInstancePtrPtr ? *ChaosVDEngineInstancePtrPtr : nullptr)
		{
			TabPtr->SetContent(SNew(SChaosVDMainTab, ChaosVDEngineInstancePtr).OwnerTab(TabPtr));
		}
		else
		{
			UE_LOGF(LogChaosVDEditor, Error, "[%s]: Failed to find CVD engine instance with ID [%ls]. The UI was not reloaded.", __func__, *InstanceGUID.ToString());
		}
	}
	else
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%s]: Failed to find CVD tab instance with ID [%ls]. The UI was not reloaded.", __func__, *InstanceGUID.ToString());
	}
}


void FChaosVDModule::TakePlaybackEngineStateSnapshot()
{
	if (ActiveChaosVDInstances.IsEmpty())
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%s]: There are no active CVD Instances.", __func__);
		return;
	}

	if (ActiveChaosVDInstances.Num() > 1)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%s]: Snapshots can only be taken when only one CVD instance is active.", __func__);
		return;
	}
	
	TSharedPtr<FChaosVDEngine> CVDEngineInstance;
	for (const TPair<FGuid, TSharedPtr<FChaosVDEngine>>& EngineInstanceWithID : ActiveChaosVDInstances)
	{
		if (EngineInstanceWithID.Value)
		{
			CVDEngineInstance = EngineInstanceWithID.Value;
			break;
		}
	}
	
	FChaosVDPlaybackEngineSnapshot StateData = CVDEngineInstance->GetPlaybackController()->GeneratePlaybackEngineSnapshot();

	TArray<FString> OutSelectedFilenames;
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		FString ExtensionStr;
		ExtensionStr += TEXT("CVD Playback Engine State|*.cvdplaystate");
		
		DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(nullptr),
			LOCTEXT("PlaybackSnapshotSaveDialogTitle", "Dump Playback Engine Sate").ToString(),
			*FPaths::ProfilingDir(),
			TEXT(""),
			*ExtensionStr,
			EFileDialogFlags::None,
			OutSelectedFilenames
		);
	}

	if (!OutSelectedFilenames.IsEmpty())
	{
		const FString& TargetFilePath = OutSelectedFilenames[0];

		IFileManager& FileManager = IFileManager::Get();
		FArchive* FileWriter = FileManager.CreateFileWriter(*TargetFilePath);

		TArray<uint8> RawData;
		FMemoryWriter MemWriter(RawData);
		FChaosVDPlaybackEngineSnapshot::StaticStruct()->SerializeTaggedProperties(MemWriter, reinterpret_cast<uint8*>(&StateData),FChaosVDPlaybackEngineSnapshot::StaticStruct(), nullptr);

		FileWriter->Serialize(RawData.GetData(), RawData.Num());
		FileWriter->Close();
		FPlatformProcess::ExploreFolder(*TargetFilePath);
	}
}

#undef LOCTEXT_NAMESPACE
	
IMPLEMENT_MODULE(FChaosVDModule, ChaosVD)
