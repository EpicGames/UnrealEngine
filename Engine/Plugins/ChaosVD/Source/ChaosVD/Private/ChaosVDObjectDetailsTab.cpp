// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDObjectDetailsTab.h"

#include "ChaosVDCollisionDataDetailsTab.h"
#include "ChaosVDScene.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "Editor.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "TEDS/ChaosVDSelectionInterface.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "Templates/SharedPointer.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/SChaosVDCollisionDataInspector.h"
#include "Widgets/SChaosVDDetailsView.h"
#include "Widgets/SChaosVDMainTab.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void FChaosVDStandAloneObjectDetailsTab::AddUnsupportedStruct(const UStruct* Struct)
{
	UnsupportedStructs.Add(Struct);
}

bool FChaosVDStandAloneObjectDetailsTab::IsSupportedStruct(const TWeakObjectPtr<const UStruct>& InWeakStructPtr)
{
	return !UnsupportedStructs.Contains(InWeakStructPtr);
}

TSharedRef<SDockTab> FChaosVDStandAloneObjectDetailsTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DetailsPanelTab =
	SNew(SDockTab)
	.TabRole(ETabRole::PanelTab)
	.Label(LOCTEXT("DetailsPanel", "Details"))
	.ToolTipText(LOCTEXT("DetailsPanelToolTip", "See the details of the selected object"));

	// The following types have their own data inspectors, we should not open them in the details pannel
	AddUnsupportedStruct(FChaosVDConstraintDataWrapperBase::StaticStruct());
	AddUnsupportedStruct(FChaosVDQueryDataWrapper::StaticStruct());
	AddUnsupportedStruct(FChaosVDParticlePairMidPhase::StaticStruct());

	float CurrentSlotSize = 0;

	if (const TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin())
	{
		DetailsPanelTab->SetContent
		(
			SNew(SVerticalBox)
			+ SVerticalBox::Slot()
			[
				SAssignNew(DetailsSplitter, SSplitter)
					.MinimumSlotHeight(60.0f)
					.Orientation(Orient_Vertical)
					.Style(FAppStyle::Get(), "SplitterDark")
					.PhysicalSplitterHandleSize(2.0f)
					+ SSplitter::Slot()
					[
						SAssignNew(DetailsPanelView, SChaosVDDetailsView, MainTabPtr.ToSharedRef())
					]
			]
		);

		DetailsSplitter->AddSlot(0)
			.Value(.2f)
			[
				SAssignNew(GeometryTreeWidget, SChaosVDGeometryTree, GetChaosVDScene())
			];
		CurrentSlotSize = DetailsSplitter->SlotAt(0).GetSizeValue();
	}
	else
	{
		DetailsPanelTab->SetContent(GenerateErrorWidget());
	}

	if (CurrentSlotSize != 0)
	{
		SavedSplitSize = CurrentSlotSize;
	}

	UpdateTreeDetailsSplit(nullptr);

	DetailsPanelTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("TabIconDetailsPanel"));

	SpawnedTab = DetailsPanelTab;

	HandleTabSpawned(DetailsPanelTab);

	return DetailsPanelTab;
}

void FChaosVDStandAloneObjectDetailsTab::UpdateTabLabel(const FText& NewLabel)
{
	if (TSharedPtr<SDockTab> Tab = SpawnedTab.Pin())
	{
		Tab->SetLabel(NewLabel);
		Tab->SetTabToolTipWidget(SNew(SToolTip).Text(NewLabel));
	}
}

void FChaosVDStandAloneObjectDetailsTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	FChaosVDTabSpawnerBase::HandleTabClosed(InTabClosed);

	SpawnedTab.Reset();
	GeometryTreeWidget.Reset();
	DetailsSplitter.Reset();
	DetailsPanelView.Reset();
}

void FChaosVDStandAloneObjectDetailsTab::UpdateTreeDetailsSplit(const FChaosVDSceneParticle* Particle)
{
	if (!GeometryTreeWidget || !DetailsSplitter || DetailsSplitter->GetChildren()->NumSlot() < 2)
	{
		return;
	}
	GeometryTreeWidget->SetDataToInspect(Particle);
	if (Particle == nullptr)
	{
		float CurrentSize = DetailsSplitter->SlotAt(0).GetSizeValue();
		if (CurrentSize != 0)
		{
			SavedSplitSize = CurrentSize;
		}
		DetailsSplitter->SlotAt(0).SetSizeValue(0);
		DetailsSplitter->SlotAt(0).SetResizable(false);
		DetailsSplitter->SlotAt(1).SetResizable(false);
		GeometryTreeWidget->SetVisibility(EVisibility::Hidden);
	}
	else
	{
		GeometryTreeWidget->SetVisibility(EVisibility::All);
		DetailsSplitter->SlotAt(0).SetResizable(true);
		DetailsSplitter->SlotAt(1).SetResizable(true);
		float CurrentSize = DetailsSplitter->SlotAt(0).GetSizeValue();
		if (CurrentSize != 0)
		{
			SavedSplitSize = CurrentSize;
		}
		if (SavedSplitSize != 0)
		{
			DetailsSplitter->SlotAt(0).SetSizeValue(SavedSplitSize);
		}
	}
}

void FChaosVDObjectDetailsTab::HandleActorsSelection(TArrayView<AActor*> SelectedActors)
{
	if (SelectedActors.Num() > 0)
	{
		// We don't support multi selection yet
		ensure(SelectedActors.Num() == 1);

		CurrentSelectedObject = SelectedActors[0];

		if (DetailsPanelView)
		{
			DetailsPanelView->SetSelectedObject(CurrentSelectedObject.Get());
		}
	}
	else
	{
		CurrentSelectedObject = nullptr;
	}
	UpdateTreeDetailsSplit(nullptr);
}

TSharedRef<SDockTab> FChaosVDObjectDetailsTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> NewTab = FChaosVDStandAloneObjectDetailsTab::HandleTabSpawnRequest(Args);
	
	if (const TSharedPtr<FChaosVDScene> ScenePtr = GetChaosVDScene().Pin())
	{
		RegisterSelectionSetObject(ScenePtr->GetElementSelectionSet());

		if (TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			SolverDataSelectionObject->GetDataSelectionChangedDelegate().AddSP(this, &FChaosVDObjectDetailsTab::HandleSolverDataSelectionChange);
		}
	}

	// If we closed the tab and opened it again with an object already selected, try to restore the selected object view
	if (DetailsPanelView.IsValid() && CurrentSelectedObject.IsValid())
	{
		DetailsPanelView->SetSelectedObject(CurrentSelectedObject.Get());
	}
	
	return NewTab;
}

void FChaosVDObjectDetailsTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = GetChaosVDScene().Pin())
	{
		if (TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			SolverDataSelectionObject->GetDataSelectionChangedDelegate().RemoveAll(this);
		}
	}

	FChaosVDStandAloneObjectDetailsTab::HandleTabClosed(InTabClosed);
}

void FChaosVDObjectDetailsTab::HandlePostSelectionChange(const UTypedElementSelectionSet* ChangedSelectionSet)
{
	TArray<AActor*> SelectedActors = ChangedSelectionSet->GetSelectedObjects<AActor>();
	if (SelectedActors.Num() > 0)
	{
		HandleActorsSelection(SelectedActors);
		return;
	}

	constexpr int32 MaxElements = 1;
	TArray<FTypedElementHandle, TInlineAllocator<MaxElements>> SelectedParticlesHandles;
	ChangedSelectionSet->GetSelectedElementHandles(SelectedParticlesHandles, UChaosVDSelectionInterface::StaticClass());

	if (SelectedParticlesHandles.Num() > 0)
	{
		// We don't support multi selection yet
		ensure(SelectedParticlesHandles.Num() == MaxElements);

		using namespace Chaos::VD::TypedElementDataUtil;
	
		DetailsPanelView->SetSelectedStruct(GetStructOnScopeDataFromTypedElementHandle(SelectedParticlesHandles[0]));
		
		const FStructTypedElementData* StructElement = SelectedParticlesHandles[0].GetData<FStructTypedElementData>(true);
		const FChaosVDSceneParticle* PartPtr = StructElement ? StructElement->GetData<FChaosVDSceneParticle>() : nullptr;
		UpdateTreeDetailsSplit(PartPtr);
		
		return;
	}

	UpdateTreeDetailsSplit(nullptr);
	DetailsPanelView->SetSelectedObject(nullptr);
	DetailsPanelView->SetSelectedStruct(nullptr);
}

void FChaosVDObjectDetailsTab::HandleSolverDataSelectionChange(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& SelectionHandle)
{
	TSharedPtr<FStructOnScope> StructOnScope = SelectionHandle ? SelectionHandle->GetDataAsStructScope() : nullptr;
	if (!StructOnScope || !IsSupportedStruct(StructOnScope->GetStructPtr()))
	{
		DetailsPanelView->SetSelectedStruct(nullptr);
		UpdateTreeDetailsSplit(nullptr);
		return;
	}

	HandleActorsSelection(TArrayView<AActor*>());

	DetailsPanelView->SetSelectedStruct(SelectionHandle->GetCustomDataReadOnlyStructViewForDetails());
}

#undef LOCTEXT_NAMESPACE
