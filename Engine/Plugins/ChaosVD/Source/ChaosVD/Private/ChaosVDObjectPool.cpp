// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDObjectPool.h"

bool FChaosVDObjectPoolCVars::bUseObjectPool = true;
FAutoConsoleVariableRef FChaosVDObjectPoolCVars::CVarUseObjectPool(TEXT("p.Chaos.VD.Tool.UseObjectPool"),FChaosVDObjectPoolCVars::bUseObjectPool, TEXT("Set to false to disable the use of a pool system for uobjects."));
