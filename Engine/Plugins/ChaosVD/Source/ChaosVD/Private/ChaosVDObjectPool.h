// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDModule.h"
#include "Async/ParallelFor.h"
#include "Containers/Array.h"
#include "Components/ActorComponent.h"
#include "Containers/Ticker.h"
#include "GameFramework/Actor.h"
#include "Interfaces/ChaosVDPooledObject.h"
#include "UObject/GCObject.h"
#include "UObject/Object.h"
#include "UObject/Package.h"

struct FChaosVDObjectPoolCVars
{
	static bool bUseObjectPool;
	static FAutoConsoleVariableRef CVarUseObjectPool;
};

/** Basic Pool system for UObjects */
template<typename ObjectType>
class TChaosVDObjectPool : public FGCObject, public FTSTickerObjectBase
{
public:
	TChaosVDObjectPool() = default;
	virtual ~TChaosVDObjectPool() override;

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;

	void SetPoolName(const FString& InName) { PoolName = InName; };
	
	virtual FString GetReferencerName() const override
	{
		return PoolName;
	}

	virtual bool Tick(float DeltaTime) override;

	ObjectType* AcquireObject(UObject* Outer, FName Name);

	void DisposeObject(UObject* Object);

	TFunction<ObjectType*(UObject*,FName)> ObjectFactoryOverride;

protected:
	void DestroyUObject(UObject* Object);

	FString PoolName = TEXT("ChaosVDObjectPool");
	int32 PoolHits = 0;
	int32 PoolRequests = 0;
	std::atomic<bool> bGrowingPoolInBackground = false;
	std::atomic<int32> PoolSize = 0;
	FRWLock PoolLock;
	TArray<TObjectPtr<ObjectType>> PooledObjects;
};

template <typename ObjectType>
TChaosVDObjectPool<ObjectType>::~TChaosVDObjectPool()
{
	float HitMissRatio = PoolRequests > 0 ? ((static_cast<float>(PoolHits) / static_cast<float>(PoolRequests)) * 100.0f) : 0;
	UE_LOGF(LogChaosVDEditor, Log, "Object pooling Statics for pool [%ls] | Hits [%d] | Total Acquire requests [%d] | [%f] percent hit/miss ratio", *PoolName, PoolHits,  PoolRequests, HitMissRatio);

	if (FChaosVDObjectPoolCVars::bUseObjectPool)
	{
		FWriteScopeLock WriteLock(PoolLock);
		for (TObjectPtr<ObjectType> ObjectToDestroy : PooledObjects)
		{
			DestroyUObject(ObjectToDestroy.Get());
		}

		PooledObjects.Empty();
	}
}

template <typename ObjectType>
void TChaosVDObjectPool<ObjectType>::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(PooledObjects);
}

template <typename ObjectType>
bool TChaosVDObjectPool<ObjectType>::Tick(float DeltaTime)
{
	constexpr int32 MinimumPoolSize = 250;
	if (FChaosVDObjectPoolCVars::bUseObjectPool && PoolSize < MinimumPoolSize && !bGrowingPoolInBackground)
	{
		const int32 ObjectsToCreate = MinimumPoolSize - PoolSize;
		UE::Tasks::Launch(TEXT("GrowingCVDObjectPool"), [this, ObjectsToCreate]()
		{
			bGrowingPoolInBackground = true;
			TArray<TObjectPtr<ObjectType>, TInlineAllocator<MinimumPoolSize>> CreatedObjects;

			UPackage* TransientPackage = GetTransientPackage();

			for (int32 Count = 0; Count < ObjectsToCreate; Count++)
			{
				const FName NewName = MakeUniqueObjectName(TransientPackage, ObjectType::StaticClass());

				ObjectType* CreatedObject = nullptr;
				if (ObjectFactoryOverride)
				{
					CreatedObject = ObjectFactoryOverride(TransientPackage, NewName);
				}
				else
				{
					CreatedObject = NewObject<ObjectType>(TransientPackage, NewName);
				}

				CreatedObjects.Emplace(CreatedObject);
			}

			{
				FWriteScopeLock WriteLock(PoolLock);

				PooledObjects.Append(CreatedObjects);
				PoolSize = PooledObjects.Num();
	
				// Clear the async flag. It seems to be added to objects created outside the GT and prevents the object from being GC'd
				// We intend to use these objects only in the GT
				ParallelFor(CreatedObjects.Num(), [&CreatedObjects](int32 ObjectIndex)
				{
					CreatedObjects[ObjectIndex]->ClearInternalFlags(EInternalObjectFlags::Async);
				});
			}

			bGrowingPoolInBackground = false;
		});
	}
	
	return true;
}

template <typename ObjectType>
ObjectType* TChaosVDObjectPool<ObjectType>::AcquireObject(UObject* Outer, FName Name)
{
	PoolRequests++;

	// If pooling is disabled, just fall through the code that will create the object
	if (FChaosVDObjectPoolCVars::bUseObjectPool && PoolSize > 0)
	{
		ObjectType* Object = nullptr;

		{
			FWriteScopeLock WriteLock(PoolLock);
			Object = PooledObjects.Pop();
			PoolSize = PooledObjects.Num();
		}

		if (Object)
		{
			const FName NewName = MakeUniqueObjectName(Outer, ObjectType::StaticClass(), Name);
			Object->Rename(*NewName.ToString() , Outer, REN_NonTransactional | REN_DoNotDirty | REN_SkipGeneratedClasses | REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
		
			if (IChaosVDPooledObject* AsPooledObject = Cast<IChaosVDPooledObject>(Object))
			{
				AsPooledObject->OnAcquired();
			}

			PoolHits++;
			
			return Object;
		}
	}

	const FName NewName = MakeUniqueObjectName(Outer, ObjectType::StaticClass(), Name);

	ObjectType* CreatedObject = nullptr;
	if (ObjectFactoryOverride)
	{
		CreatedObject = ObjectFactoryOverride(Outer, NewName);
	}
	else
	{
		CreatedObject = NewObject<ObjectType>(Outer, NewName);
	}

	if (IChaosVDPooledObject* AsPooledObject = Cast<IChaosVDPooledObject>(CreatedObject))
	{
		AsPooledObject->OnAcquired();
	}
	
	return CreatedObject;
}

template <typename ObjectType>
void TChaosVDObjectPool<ObjectType>::DisposeObject(UObject* Object)
{
	// If pooling is disabled, just destroy the object
	//TODO: Should we provide a way to override how these are destroyed?
	if (!FChaosVDObjectPoolCVars::bUseObjectPool)
	{
		DestroyUObject(Object);

		return;
	}

	if (IChaosVDPooledObject* AsPooledObject = Cast<IChaosVDPooledObject>(Object))
	{
		AsPooledObject->OnDisposed();
	}

	UPackage* TransientPackage = GetTransientPackage();
	const FName NewName = MakeUniqueObjectName(TransientPackage, ObjectType::StaticClass());
	Object->Rename(*NewName.ToString(), TransientPackage, REN_NonTransactional | REN_DoNotDirty | REN_SkipGeneratedClasses | REN_DontCreateRedirectors);

	{
		FWriteScopeLock WriteLock(PoolLock);
		PooledObjects.Emplace(Cast<ObjectType>(Object));
	}

	PoolSize = PooledObjects.Num();
}

template <typename ObjectType>
void TChaosVDObjectPool<ObjectType>::DestroyUObject(UObject* Object)
{
	if (UActorComponent* AsActorComponent = Cast<UActorComponent>(Object))
	{
		AsActorComponent->DestroyComponent();
	}
	else if (AActor* AsActor = Cast<AActor>(Object))
	{
		AsActor->Destroy();
	}
	else
	{
		Object->ConditionalBeginDestroy();
	}
}
