// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDOutputLogTab.h"

#include "ChaosLog.h"
#include "ChaosVDModule.h"
#include "ChaosVDStyle.h"
#include "OutputLogCreationParams.h"
#include "OutputLogModule.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

TSharedRef<SDockTab> FChaosVDOutputLogTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	FOutputLogCreationParams Params;
	Params.bCreateDockInLayoutButton = true;
	Params.SettingsMenuCreationFlags = EOutputLogSettingsMenuFlags::SkipClearOnPie
		| EOutputLogSettingsMenuFlags::SkipOpenSourceButton
		| EOutputLogSettingsMenuFlags::SkipEnableWordWrapping; // Checkbox relies on saving an editor config file and does not work correctly

	Params.AllowAsInitialLogCategory = FAllowLogCategoryCallback::CreateLambda([](const FName LogCategory) {
		return LogCategory == LogChaosVDEditor.GetCategoryName() || LogCategory == LogChaos.GetCategoryName() || LogCategory == FName("Cmd");
		});

	Params.DefaultCategorySelection.Emplace(LogChaosVDEditor.GetCategoryName(), true);
	Params.DefaultCategorySelection.Emplace(LogChaos.GetCategoryName(), true);
	Params.DefaultCategorySelection.Emplace(FName("Cmd"), true);
	
	TSharedRef<SDockTab> OutputLogTab =
		SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("OutputLogTabLabel", "Output Log"));
	
	OutputLogTab->SetContent
	(
		FOutputLogModule::Get().MakeOutputLogWidget(Params)
	);

	OutputLogTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("TabIconOutputLog"));

	HandleTabSpawned(OutputLogTab);

	return OutputLogTab;
}

#undef LOCTEXT_NAMESPACE
