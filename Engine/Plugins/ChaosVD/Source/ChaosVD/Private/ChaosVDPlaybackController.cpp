// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDPlaybackController.h"

#include "Actors/ChaosVDSolverInfoActor.h"
#include "ChaosVDModule.h"
#include "ChaosVDRecording.h"
#include "ChaosVDRuntimeModule.h"
#include "ChaosVDScene.h"
#include "ChaosVDSceneCompositionReport.h"
#include "ChaosVDSettingsManager.h"
#include "ExtensionsSystem/ChaosVDExtensionsManager.h"
#include "Misc/MessageDialog.h"
#include "Settings/ChaosVDGeneralSettings.h"
#include "Trace/ChaosVDTraceProvider.h"
#include "TraceSessionsManager.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "Widgets/SChaosVDTimelineWidget.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDPlaybackController)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

class UChaosVDGeneralSettings;

namespace Chaos::VisualDebugger::Cvars
{
	static bool bPlayAllPreviousFrameSteps = true;
	static FAutoConsoleVariableRef CVarChaosVDbPlayAllPreviousFrameSteps(
		TEXT("p.Chaos.VD.Tool.PlayAllPreviousFrameSteps"),
		bPlayAllPreviousFrameSteps,
		TEXT("If true, each time we get playback a solver frame in a specific stage, we will play all the previous steps from that frame in sequence to ensure we have the correct visualization for what happened in that frame."));
}

bool FChaosVDTrackInfo::operator==(const FChaosVDTrackInfo& Other) const
{
	return Other.TrackID == TrackID && Other.TrackType == TrackType;
}

bool FChaosVDTrackInfo::AreSameTrack(const TSharedRef<const FChaosVDTrackInfo>& TrackA, const TSharedRef<const FChaosVDTrackInfo>& TrackB)
{
	return TrackA->TrackID == TrackB->TrackID && TrackA->TrackType == TrackB->TrackType; 
}

FChaosVDPlaybackController::FChaosVDPlaybackController(const TWeakPtr<FChaosVDScene>& InSceneToControl) : ActiveTrack(MakeShared<FChaosVDTrackInfo>())
{
	SceneToControl = InSceneToControl;
}

FChaosVDPlaybackController::~FChaosVDPlaybackController()
{
	UnloadCurrentRecording(EChaosVDUnloadRecordingFlags::Silent);
	DeInitialize();
}

void FChaosVDPlaybackController::Initialize()
{
	if (bInitialized)
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%s] Playback controller already initialized", __func__);
		return;
	}

	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		HandleGeneralSettingsUpdated(Settings);
		Settings->OnSettingsChanged().AddSP(this, &FChaosVDPlaybackController::HandleGeneralSettingsUpdated);
	}

	bInitialized = true;
}

void FChaosVDPlaybackController::DeInitialize()
{
	if (!bInitialized)
	{
		UE_LOGF(LogChaosVDEditor, Verbose, "[%s] Playback controller not initialized", __func__);
		return;
	}

	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		Settings->OnSettingsChanged().RemoveAll(this);
	}
	
	bInitialized = false;
}

bool FChaosVDPlaybackController::LoadChaosVDRecordingFromTraceSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor)
{
	if (!ensure(InSessionDescriptor.IsValid()))
	{
		return false;
	}

	if (LoadedRecording.IsValid())
	{
		UnloadCurrentRecording();
	}

	if (const TSharedPtr<const TraceServices::IAnalysisSession> TraceSession = FChaosVDModule::Get().GetTraceSessionsManager()->GetSession(InSessionDescriptor.SessionName))
	{
		if (const FChaosVDTraceProvider* ChaosVDProvider = TraceSession->ReadProvider<FChaosVDTraceProvider>(FChaosVDTraceProvider::ProviderName))
		{
			LoadedRecording = ChaosVDProvider->GetRecordingForSession();
		}
	}

	if (!ensure(LoadedRecording.IsValid()))
	{
		FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("LoadRecordingFailedMessage", "Failed to load the selected CVD recording. Please see the logs for more details... "));

		return false;
	}

	LoadedRecording->SetIsLive(InSessionDescriptor.bIsLiveSession);

	if (TSharedPtr<FChaosVDScene> ScenePtr = SceneToControl.Pin())
	{
		ScenePtr->SetLoadedRecording(LoadedRecording);
	}

	HandleCurrentRecordingUpdated();
	
	bHasPendingGTUpdateBroadcast = true;

	return true;
}

void FChaosVDPlaybackController::UnloadCurrentRecording(EChaosVDUnloadRecordingFlags UnloadOptions)
{
	ensure(IsInGameThread());

	RecordingLastSeenTimeUpdatedAsCycle = 0;

	TrackInfoUpdateGTQueue.Empty();

	if (const TSharedPtr<FChaosVDScene> SceneToControlSharedPtr = SceneToControl.Pin())
	{
		if (SceneToControlSharedPtr->IsInitialized())
		{
			SceneToControlSharedPtr->CleanUpScene(EChaosVDSceneCleanUpOptions::ReInitializeGeometryBuilder | EChaosVDSceneCleanUpOptions::CollectGarbage);
		}	
	}
	
	if (LoadedRecording.IsValid())
	{
		LoadedRecording.Reset();
	}

	// This will make sure the cached data used by the UI is up to date.
	// It already handles internally an unloaded recording, in which case the cached data will be properly reset
	TrackInfoPerType.Reset();
	HandleCurrentRecordingUpdated();

	// Once the cached track data is cleared out, we need to mimic what the normal update loop does so the updates are sent to the UI
	// Before we attempt to load a new recording
	const TWeakPtr<FChaosVDPlaybackController> ThisWeakPtr = DoesSharedInstanceExist() ? AsWeak() : nullptr;
	if (TSharedPtr<const FChaosVDTrackInfo> GameTrackInfo = GetTrackInfo(EChaosVDTrackType::Game, GameTrackID))
	{
		TSharedRef<const FChaosVDTrackInfo> GTTrackInfoAsSharedRef = GameTrackInfo.ToSharedRef();
		TrySetActiveTrack(GTTrackInfoAsSharedRef);
		EnqueueTrackInfoUpdate(GTTrackInfoAsSharedRef, PlaybackSelfInstigatorID);
		ProcessQueuedTrackInfoUpdates(ThisWeakPtr);
	}
	
	if (EnumHasAnyFlags(UnloadOptions, EChaosVDUnloadRecordingFlags::BroadcastChanges))
	{
		BroadcastGTDataUpdate(ThisWeakPtr);
		bHasPendingGTUpdateBroadcast = false;
	}

	bPlayedFirstFrame = false;
}

void FChaosVDPlaybackController::PlayFromClosestKeyFrame_AssumesLocked(const int32 InTrackID, const int32 FrameNumber, FChaosVDScene& InSceneToControl)
{
	if (!LoadedRecording.IsValid())
	{
		return;
	}

	const int32 KeyFrameNumber = LoadedRecording->FindFirstSolverKeyFrameNumberFromFrame_AssumesLocked(InTrackID, FrameNumber);
	if (KeyFrameNumber < 0)
	{
		// This can happen during live debugging as we miss some of the events at the beginning.
		// Loading a trace file that was recorded as part of a live session, will have the same issue.
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Failed to find a keyframe close to frame [%d] of track [%d]", ANSI_TO_TCHAR(__FUNCTION__), FrameNumber, InTrackID);
		return;
	}

	// All keyframes should be played from stage 0 as in some scenarios we will generate a keyframe by collapsing multiple delta frames. In these frames there will be only a single "Generated" stage.
	constexpr int32 SolverStage = 0;

	// If this frame number has keyframe data, just use it directly and save the cost of copying the data to a "collapsed keyframe"
	if (KeyFrameNumber == FrameNumber)
	{
		constexpr bool bRequestingKeyFrameOnly = true;
		if (const FChaosVDSolverFrameData* SolverFrameData = LoadedRecording->GetSolverFrameData_AssumesLocked(InTrackID, FrameNumber, bRequestingKeyFrameOnly))
		{
			PlaySolverStageData(InTrackID, InSceneToControl.AsShared(), *SolverFrameData, SolverStage);
		}
		else
		{
			UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Failed to find a keyframe data for frame [%d] of track [%d]. The visualization might be out of sync until a new keyframe is played.", ANSI_TO_TCHAR(__FUNCTION__), FrameNumber, InTrackID);
		}

		return;
	}

	// If the frame number we wanted to play is not a keyframe, instead of playing back each delta frame since the key frame, generate a new solver frame with all the deltas collapsed in one
	// This increases the tool performance while scrubbing or live debugging if there are few keyframes
	FChaosVDSolverFrameData CollapsedFrameData;
	LoadedRecording->CollapseSolverFramesRange_AssumesLocked(InTrackID, KeyFrameNumber, FrameNumber, CollapsedFrameData);

	PlaySolverStageData(InTrackID, InSceneToControl.AsShared(), CollapsedFrameData, SolverStage);
}

void FChaosVDPlaybackController::EnqueueTrackInfoUpdate(const TSharedRef<const FChaosVDTrackInfo>& InTrackInfo, FGuid InstigatorID)
{
	// This will be used in the Game Thread on the first tick after this was added, so we need to make a copy of the state right now
	FChaosVDQueuedTrackInfoUpdate InfoUpdate;
	InfoUpdate.TrackInfo = InTrackInfo;
	InfoUpdate.InstigatorID = InstigatorID;

	TrackInfoUpdateGTQueue.Enqueue(InfoUpdate);
}

void FChaosVDPlaybackController::PlaySolverStageData(int32 TrackID, const TSharedRef<FChaosVDScene>& InSceneToControlSharedPtr, const FChaosVDSolverFrameData& InSolverFrameData, int32 StageIndex)
{
	if (InSolverFrameData.SolverSteps.IsValidIndex(StageIndex))
	{
		InSceneToControlSharedPtr->UpdateFromRecordedSolverStageData(TrackID, InSolverFrameData.SolverSteps[StageIndex], InSolverFrameData);
	}
	else
	{
		// This is common if we stop PIE, change worlds, and PIE again without stopping the recording
		UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] Tried to scrub to an invalid step | Step Number [%d] ...", ANSI_TO_TCHAR(__FUNCTION__), StageIndex);
	}
}

TSharedRef<FChaosVDTrackInfo> FChaosVDPlaybackController::CreateTrackInfo(int32 SlotIndex)
{
	TSharedRef<FChaosVDTrackInfo> NewTrack = MakeShared<FChaosVDTrackInfo>();

	NewTrack->TrackSlot = SlotIndex == INDEX_NONE ?  ++LastAssignedTrackSlot : SlotIndex;

	return NewTrack;
}

void FChaosVDPlaybackController::HandleGeneralSettingsUpdated(UObject* SettingsObject)
{
	if (UChaosVDGeneralSettings* GeneralSettings = Cast<UChaosVDGeneralSettings>(SettingsObject))
	{
		bStartLiveSessionPlaybackAutomatically = GeneralSettings->bAutoStartLiveSessionsPlayback;
	}
}

void FChaosVDPlaybackController::GoToRecordedSolverStage_AssumesLocked(const int32 InTrackID, const int32 FrameNumber, const int32 StageNumber, FGuid InstigatorID)
{
	if (const TSharedPtr<FChaosVDScene> SceneToControlSharedPtr = SceneToControl.Pin())
	{
		if (ensure(LoadedRecording.IsValid()))
		{
			TSharedPtr<FChaosVDTrackInfo> CurrentTrackInfo = GetMutableTrackInfo(EChaosVDTrackType::Solver, InTrackID);

			if (!ensure(CurrentTrackInfo.IsValid()))
			{
				UE_LOGF(LogChaosVDEditor, Error, "[%ls] Track info for track ID [%d]. We can't continue...", ANSI_TO_TCHAR(__FUNCTION__), InTrackID);
				return;
			}

			if (FChaosVDSolverFrameData* SolverFrameData = LoadedRecording->GetSolverFrameData_AssumesLocked(InTrackID, FrameNumber))
			{
				int32 TargetStageIndex = StageNumber;
				// All solver frames need to be played with a valid specified solver stage. If we don't have just early out
				if (!SolverFrameData->SolverSteps.IsValidIndex(TargetStageIndex))
				{
					if (SolverFrameData->SolverSteps.IsEmpty() || TargetStageIndex != INDEX_NONE)
					{
						UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] Invalid solver stage index [%d] at frame [%d] for Solver ID [%d]. We can't continue...", ANSI_TO_TCHAR(__FUNCTION__), TargetStageIndex, FrameNumber, InTrackID);
						return;
					}

					// If we got INDEX_NONE as stage number, we should play the last available stage
					TargetStageIndex = SolverFrameData->SolverSteps.Num() -1;
				}

				const int32 FrameDiff = FrameNumber - CurrentTrackInfo->CurrentFrame;
				constexpr int32 FrameDriftTolerance = 1;

				// If we go back, even for one single step and the particles that changed are not in the prev step, we have no data to restore their changed values.
				// So for now if we are going backwards, always play from closest keyframe.
				// TODO: Implement a less expensive way of handle these cases.
				// We should keep the previous state of each loaded particle so if when going back they are not in the new delta we are evaluating, (and were not destroyed)
				// we can just re-apply that last known state.
				const bool bNeedsToPlayFromKeyframe = !bPlayedFirstFrame || FrameDiff < 0 || FMath::Abs(FrameDiff) > FrameDriftTolerance;

				if (bNeedsToPlayFromKeyframe || CurrentTrackInfo->CurrentFrame == 0)
				{
					// As Frames are recorded as delta, we need to make sure of playing back all the deltas since the closest keyframe
					PlayFromClosestKeyFrame_AssumesLocked(InTrackID, FrameNumber, *SceneToControlSharedPtr.Get());
				}

				
				const int32 StageNumberDiff = TargetStageIndex - CurrentTrackInfo->CurrentStage;
				const bool bIsPlayingNewSolverFrame = CurrentTrackInfo->CurrentFrame != FrameNumber;

				const bool bNeedsPlayPreviousSteps = bIsPlayingNewSolverFrame || StageNumberDiff < 0 || FMath::Abs(StageNumberDiff) > FrameDriftTolerance;

				if (Chaos::VisualDebugger::Cvars::bPlayAllPreviousFrameSteps && bNeedsPlayPreviousSteps)
				{
					for (int32 StageIndex = 0; StageIndex <= TargetStageIndex; StageIndex++)
					{
						PlaySolverStageData(InTrackID, SceneToControlSharedPtr.ToSharedRef(), *SolverFrameData, StageIndex);
					}
				}
				else
				{
					PlaySolverStageData(InTrackID, SceneToControlSharedPtr.ToSharedRef(), *SolverFrameData, TargetStageIndex);
				}

				if (CurrentTrackInfo->CurrentFrame != FrameNumber)
				{
					CurrentTrackInfo->CurrentFrame = FrameNumber;

					// For server tracks, we only need to have a recorded internal frame number
					CurrentTrackInfo->bHasNetworkSyncData = SolverFrameData->HasNetworkSyncData(CurrentTrackInfo->bIsServer ? EChaosVDNetworkSyncDataRequirements::InternalFrameNumber : EChaosVDNetworkSyncDataRequirements::All);
			
					SceneToControlSharedPtr->HandleEnterNewSolverFrame(FrameNumber, *SolverFrameData);
				}

				CurrentTrackInfo->CurrentStage = TargetStageIndex;
				CurrentTrackInfo->bIsReSimulated = SolverFrameData->bIsResimulated;

				CurrentTrackInfo->CurrentStageNames.Reset();
				Algo::Transform(SolverFrameData->SolverSteps, CurrentTrackInfo->CurrentStageNames, &FChaosVDFrameStageData::StepName);
			
				EnqueueTrackInfoUpdate(CurrentTrackInfo.ToSharedRef(), InstigatorID);
			}
		}
	}
	else
	{
		ensureMsgf(false, TEXT("GoToRecordedStep Called without a valid scene to control"));	
	}
}

void FChaosVDPlaybackController::GoToRecordedGameFrame_AssumesLocked(const int32 FrameNumber, FGuid InstigatorID)
{
	if (const TSharedPtr<FChaosVDScene> SceneToControlSharedPtr = SceneToControl.Pin())
	{
		if (ensure(LoadedRecording.IsValid()))
		{
			const TSharedPtr<FChaosVDTrackInfo> TrackInfoSharedPtr = GetMutableTrackInfo(EChaosVDTrackType::Game, GameTrackID);
			if (!ensure(TrackInfoSharedPtr))
			{
				return;
			}

			if (const FChaosVDGameFrameData* FoundGameFrameData = LoadedRecording->GetGameFrameData_AssumesLocked(FrameNumber))
			{
				TArray<int32, TInlineAllocator<FChaosVDRecording::CommonTrackCount>> AvailableSolversID;
				TArray<int32, TInlineAllocator<FChaosVDRecording::CommonTrackCount>> RemovedSolversID;
				
				AvailableSolversID.Reset();
				RemovedSolversID.Reset();
				LoadedRecording->GetAvailableSolverIDsAtGameFrameNumber_AssumesLocked(FrameNumber, AvailableSolversID);

				SceneToControlSharedPtr->HandleEnterNewGameFrame(FrameNumber, AvailableSolversID, *FoundGameFrameData, RemovedSolversID);

				// We need to reset the current frame track info for any solver that is removed as so when (or if) it is loaded again, starts on a clean state
				for (const int32 SolverID : RemovedSolversID)
				{
					if (TSharedPtr<FChaosVDTrackInfo> RemovedSolverTrackPtr = GetMutableTrackInfo(EChaosVDTrackType::Solver, SolverID))
					{
						RemovedSolverTrackPtr->CurrentFrame = 0;
					}
				}

				TrackInfoSharedPtr->CurrentFrame = FrameNumber;
				EnqueueTrackInfoUpdate(TrackInfoSharedPtr.ToSharedRef(), InstigatorID);
			}
		}
	}
}

void FChaosVDPlaybackController::GoToTrackFrame(FGuid InstigatorID, EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber, int32 StageNumber)
{
	if (!LoadedRecording.IsValid())
	{
		return;
	}

	FReadScopeLock ReadLock(LoadedRecording->GetRecordingDataLock());
	GoToTrackFrame_AssumesLocked(InstigatorID, TrackType, InTrackID, FrameNumber, StageNumber);
}

void FChaosVDPlaybackController::GoToTrackFrame_AssumesLocked(FGuid InstigatorID, EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber, int32 StageNumber)
{
	switch (TrackType)
	{
	case EChaosVDTrackType::Game:
		GoToRecordedGameFrame_AssumesLocked(FrameNumber, InstigatorID);
		break;
	case EChaosVDTrackType::Solver:
		GoToRecordedSolverStage_AssumesLocked(InTrackID, FrameNumber, StageNumber, InstigatorID);
		break;
	default:
		ensure(false);
		break;
	}
}

void FChaosVDPlaybackController::GoToTrackFrameAndSync(FGuid InstigatorID, EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber, int32 StageNumber)
{
	if (!ensure(LoadedRecording.IsValid()))
	{
		return;
	}

	FReadScopeLock ReadLock(LoadedRecording->GetRecordingDataLock());	
	GoToTrackFrame_AssumesLockedAndSync(InstigatorID, TrackType, InTrackID, FrameNumber, StageNumber);
}

void FChaosVDPlaybackController::GoToTrackFrame_AssumesLockedAndSync(FGuid InstigatorID, EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber, int32 StageNumber)
{
	GoToTrackFrame_AssumesLocked(InstigatorID, TrackType, InTrackID, FrameNumber, StageNumber);

	if (CurrentSyncMode != EChaosVDSyncTimelinesMode::Manual)
	{
		if (TSharedPtr<const FChaosVDTrackInfo> TrackToSyncWith = GetTrackInfo(TrackType, InTrackID))
		{
			SyncTracks_AssumesLocked(TrackToSyncWith.ToSharedRef(), InstigatorID, CurrentSyncMode);
		}
	}
}

int32 FChaosVDPlaybackController::GetTrackStagesNumberAtFrame_AssumesLocked(EChaosVDTrackType TrackType, const int32 InTrackID, const int32 FrameNumber) const
{
	if (!LoadedRecording.IsValid())
	{
		return INDEX_NONE;
	}
	
	switch (TrackType)
	{
		case EChaosVDTrackType::Game:
			// Game Tracks do not have steps
			return 0;
		case EChaosVDTrackType::Solver:
			{
				if (const FChaosVDSolverFrameData* FrameData = LoadedRecording->GetSolverFrameData_AssumesLocked(InTrackID, FrameNumber))
				{
					return FrameData->SolverSteps.Num() > 0 ? FrameData->SolverSteps.Num() : INDEX_NONE;
				}
				else
				{
					return INDEX_NONE;
				}
			}
	default:
		return INDEX_NONE;
	}
}

const FChaosVDFrameStagesContainer* FChaosVDPlaybackController::GetTrackStagesDataAtFrame_AssumesLocked(EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber) const
{
	if (!LoadedRecording.IsValid())
	{
		return nullptr;
	}
	
	switch (TrackType)
	{
	case EChaosVDTrackType::Game:
		// Game Tracks do not have steps
		return nullptr;
	case EChaosVDTrackType::Solver:
		{
			if (const FChaosVDSolverFrameData* FrameData = LoadedRecording->GetSolverFrameData_AssumesLocked(InTrackID, FrameNumber))
			{
				return &FrameData->SolverSteps;
			}
			else
			{
				return nullptr;
			}
		}
	default:
		return nullptr;
	}
}

int32 FChaosVDPlaybackController::GetTrackFramesNumber_AssumesLocked(EChaosVDTrackType TrackType, const int32 InTrackID) const
{
	if (!LoadedRecording.IsValid())
	{
		return INDEX_NONE;
	}
	
	switch (TrackType)
	{
		case EChaosVDTrackType::Game:
		{
			// There is only one game track so no ID is needed
			int32 GameFrames = LoadedRecording->GetAvailableGameFramesNumber_AssumesLocked();
			return GameFrames > 0 ? GameFrames : INDEX_NONE;
		}

		case EChaosVDTrackType::Solver:
		{
			int32 SolverFrames = LoadedRecording->GetAvailableSolverFramesNumber_AssumesLocked(InTrackID);
			return  SolverFrames > 0 ? SolverFrames : INDEX_NONE;
		}
		default:
			return INDEX_NONE;
	}
}

int32 FChaosVDPlaybackController::GetTrackFramesNumber(EChaosVDTrackType TrackType, int32 InTrackID) const
{
	if (!LoadedRecording.IsValid())
	{
		return INDEX_NONE;
	}
	
	FReadScopeLock ReadLock(LoadedRecording->GetRecordingDataLock());	
	return GetTrackFramesNumber_AssumesLocked(TrackType, InTrackID);
}

int32 FChaosVDPlaybackController::ConvertCurrentFrameToOtherTrackFrame_AssumesLocked(const TSharedRef<const FChaosVDTrackInfo>& InFromTrack, const TSharedRef<const FChaosVDTrackInfo>& InToTrack, EChaosVDSyncTimelinesMode TrackSyncMode)
{	
	if (!ensure(LoadedRecording.IsValid()))
	{
		return INDEX_NONE;
	}

	// Each track is on a different "space time", because it's source data ticked at different rates when was recorded, and some start/end at different points on time
	// But all the recorded frame data on all of them use Platform Cycles as timestamps
	// This method wraps specialized methods in the recording object to convert between these spaces. For example Game frame 1500 could be frame 5 on a specific solver
	// And Frame 5 of that solver could be frame 30 on another solver.

	if (FChaosVDTrackInfo::AreSameTrack(InFromTrack, InToTrack))
	{
		return InFromTrack->CurrentFrame;
	}

	switch (InFromTrack->TrackType)
	{
		case EChaosVDTrackType::Game:
			{
				ensureMsgf(TrackSyncMode != EChaosVDSyncTimelinesMode::NetworkTick, TEXT("Game tracks cannot be converted to solver tracks using network sync mode. Falling back to timestamp mode."));
				// Convert from Game Frame to Solver Frame
				return LoadedRecording->GetLowestSolverFrameNumberGameFrame_AssumesLocked(InFromTrack->TrackID, InFromTrack->CurrentFrame);
			}	
		case EChaosVDTrackType::Solver:
			{
				if (InToTrack->TrackType == EChaosVDTrackType::Solver)
				{
					switch (TrackSyncMode)
					{
						case EChaosVDSyncTimelinesMode::RecordedTimestamp:
							{
								const FChaosVDSolverFrameData* FromSolverFrameData = LoadedRecording->GetSolverFrameData_AssumesLocked(InFromTrack->TrackID, InFromTrack->CurrentFrame);
								return ensure(FromSolverFrameData) ? LoadedRecording->GetLowestSolverFrameNumberAtCycle_AssumesLocked(InToTrack->TrackID, FromSolverFrameData->FrameCycle) : INDEX_NONE;
							}
						case EChaosVDSyncTimelinesMode::NetworkTick:
							{
								int32 ToFrame = INDEX_NONE;
								const FChaosVDSolverFrameData* FromSolverFrameData = LoadedRecording->GetSolverFrameData_AssumesLocked(InFromTrack->TrackID, InFromTrack->CurrentFrame);
								const FChaosVDSolverFrameData* ToSolverFrameData = LoadedRecording->GetSolverFrameData_AssumesLocked(InToTrack->TrackID, InToTrack->CurrentFrame);
								if (FromSolverFrameData && ToSolverFrameData)
								{
									const bool bTargetTrackIsActualSolverTrack = !EnumHasAnyFlags(ToSolverFrameData->GetAttributes(), EChaosVDSolverFrameAttributes::HasGTDataToReRoute);
									if (bTargetTrackIsActualSolverTrack)
									{
										if (InFromTrack->bIsServer) // Server --> Client track
										{
											if (ToSolverFrameData->NetworkTickOffset != INDEX_NONE)
											{
												ToFrame = FromSolverFrameData->InternalFrameNumber - ToSolverFrameData->NetworkTickOffset;
												
											}
											else
											{
												ToFrame = INDEX_NONE;
											}
										}
										else // Client --> Client Track or Client --> Server Track
										{

											// Server tracks will not have a frame offset set, so we need to clamp it to 0 first
											const int32 ToTrackNetworkTick = InToTrack->bIsServer && ToSolverFrameData->NetworkTickOffset == INDEX_NONE ? 0 : ToSolverFrameData->NetworkTickOffset;
											const int32 FromTrackNetworkTick = InFromTrack->bIsServer && FromSolverFrameData->NetworkTickOffset == INDEX_NONE ? 0 : FromSolverFrameData->NetworkTickOffset;

											// This works for Client --> Server conversion because in that case we want to add the frame offset.
											// As the tick offset in server tracks is 0,  the following calculation will return a negative offset, which it will result in the intended addition in the last caluclation
											if (ToTrackNetworkTick != INDEX_NONE || FromTrackNetworkTick != INDEX_NONE)
											{
												const int32 FrameOffset = ToTrackNetworkTick - FromTrackNetworkTick;
												ToFrame = FromSolverFrameData->InternalFrameNumber - FrameOffset;
											}
											else
											{
												ToFrame = INDEX_NONE;
											}
										}
									}
									else
									{
										// This fallback is required because at the moment of implementing this, multi-file / multi-session mode, process additional GT data as
										// a fake solver track, because CVD does not support multiple gt data tracks.
										// This also means it needs to fallback to time-stamp based sync mode for now.
										return LoadedRecording->GetLowestSolverFrameNumberAtCycle_AssumesLocked(InToTrack->TrackID, FromSolverFrameData->FrameCycle);
									}
								}

								if (ToFrame != INDEX_NONE)
								{
									return LoadedRecording->GetLowestSolverFrameNumberAtNetworkFrameNumber_AssumesLocked(InToTrack->TrackID, ToFrame);
								}

								// If we failed to re-map the track based on the network tick, just fallback to the timestamp sync logic.
								// The UI shows a warning on all the conditions this can happen, which is when any of the tracks does not have net tick offset data
								if (FromSolverFrameData)
								{
									UE_LOGF(LogChaosVDEditor, Verbose, "[%s] Failed to map frame number from recorded network tick offsets. Falling back to timestamp mode | From Track [%ls] To Track [%ls]", __func__, *InFromTrack->TrackName.ToString(), *InToTrack->TrackName.ToString());
									return LoadedRecording->GetLowestSolverFrameNumberAtCycle_AssumesLocked(InToTrack->TrackID, FromSolverFrameData->FrameCycle);
								}
							}
						default:
							break;
					}
				}
				
				// From Solver to Game track, we can only convert a frame based on timestamps
				// TODO: Technically we are ignoring the requested sync mode, but the current implementation of CVD relies in this fallback as we always want to sync from solver to game tracks using the original timestamp mode
				// We should investigate if it is worth coming up with a better API so this default fallback does not catch anyone using it off guard.
				return LoadedRecording->GetLowestGameFrameAtSolverFrameNumber_AssumesLocked(InFromTrack->TrackID, InFromTrack->CurrentFrame);
			}
		default:
			ensure(false);
			return INDEX_NONE;
	}
}

int32 FChaosVDPlaybackController::GetTrackCurrentFrame(EChaosVDTrackType TrackType, const int32 InTrackID) const
{
	if (const TrackInfoByIDMap* TrackInfoByID = TrackInfoPerType.Find(TrackType))
	{
		const TSharedPtr<FChaosVDTrackInfo>* TrackInfoPtrPtr = TrackInfoByID->Find(InTrackID);
		if (const TSharedPtr<FChaosVDTrackInfo> TrackInfoSharedPtr = TrackInfoPtrPtr ? *TrackInfoPtrPtr : nullptr)
		{
			return TrackInfoSharedPtr->CurrentFrame;
		}
	}

	return INDEX_NONE;
}

int32 FChaosVDPlaybackController::GetTrackCurrentStage(EChaosVDTrackType TrackType, const int32 InTrackID) const
{
	if (const TrackInfoByIDMap* TrackInfoByID = TrackInfoPerType.Find(TrackType))
	{
		const TSharedPtr<FChaosVDTrackInfo>* TrackInfoPtrPtr = TrackInfoByID->Find(InTrackID);
		if (const TSharedPtr<FChaosVDTrackInfo> TrackInfoSharedPtr = TrackInfoPtrPtr ? *TrackInfoPtrPtr : nullptr)
		{
			return TrackInfoSharedPtr->CurrentStage;
		}
	}

	return INDEX_NONE;
}

int32 FChaosVDPlaybackController::GetTrackLastStageAtFrame(EChaosVDTrackType TrackType, int32 InTrackID, int32 InFrameNumber) const
{
	if (!LoadedRecording.IsValid())
	{
		return INDEX_NONE;
	}

	FReadScopeLock ReadLock(LoadedRecording->GetRecordingDataLock());
	return GetTrackLastStageAtFrame_AssumesLocked(TrackType, InTrackID, InFrameNumber);
}

int32 FChaosVDPlaybackController::GetTrackLastStageAtFrame_AssumesLocked(EChaosVDTrackType TrackType, int32 InTrackID, int32 InFrameNumber) const
{
	switch (TrackType)
	{
		case EChaosVDTrackType::Solver:
		{
			const int32 AvailableSteps = GetTrackStagesNumberAtFrame_AssumesLocked(EChaosVDTrackType::Solver, InTrackID, InFrameNumber);
			return AvailableSteps == INDEX_NONE ? INDEX_NONE: AvailableSteps -1;
		}
		case EChaosVDTrackType::Game:
		default:
			return INDEX_NONE;
	}
}

TSharedPtr<const FChaosVDTrackInfo> FChaosVDPlaybackController::GetTrackInfo(EChaosVDTrackType TrackType, int32 TrackID)
{
	return GetMutableTrackInfo(TrackType, TrackID);
}

TSharedPtr<FChaosVDTrackInfo> FChaosVDPlaybackController::GetMutableTrackInfo(EChaosVDTrackType TrackType, int32 TrackID)
{
	if (const TrackInfoByIDMap* TrackInfoByID = TrackInfoPerType.Find(TrackType))
	{
		const TSharedPtr<FChaosVDTrackInfo>* TrackInfoPtrPtr = TrackInfoByID->Find(TrackID);
		if (const TSharedPtr<FChaosVDTrackInfo> TrackInfoSharedPtr = TrackInfoPtrPtr ? *TrackInfoPtrPtr : nullptr)
		{
			return TrackInfoSharedPtr;
		}
	}

	return nullptr;
}

void FChaosVDPlaybackController::LockTrackInCurrentStage(EChaosVDTrackType TrackType, int32 TrackID)
{
	if (TSharedPtr<FChaosVDTrackInfo> TrackInfo = GetMutableTrackInfo(TrackType, TrackID))
	{
		TrackInfo->LockedOnStage = TrackInfo->CurrentStage;
	}
}

void FChaosVDPlaybackController::UnlockTrackStage(EChaosVDTrackType TrackType, int32 TrackID)
{
	if (TSharedPtr<FChaosVDTrackInfo> TrackInfo = GetMutableTrackInfo(TrackType, TrackID))
	{
		TrackInfo->LockedOnStage = INDEX_NONE;
	}
}

void FChaosVDPlaybackController::GetAvailableTracks(EChaosVDTrackType TrackType, TArray<TSharedPtr<const FChaosVDTrackInfo>>& OutTrackInfo)
{
	OutTrackInfo.Reset();
	
	TMap<int32, TSharedPtr<FChaosVDTrackInfo>>& AvailableTracksByType = TrackInfoPerType.FindOrAdd(TrackType);
	
	for (const TPair<int32, TSharedPtr<FChaosVDTrackInfo>>& Track : AvailableTracksByType)
	{
		OutTrackInfo.Emplace(Track.Value);
	}
}

void FChaosVDPlaybackController::GetMutableAvailableTracks(EChaosVDTrackType TrackType, TArray<TSharedPtr<FChaosVDTrackInfo>>& OutTrackInfo)
{
	OutTrackInfo.Reset();
	TrackInfoPerType.FindOrAdd(TrackType).GenerateValueArray(OutTrackInfo);
}

void FChaosVDPlaybackController::GetAvailableTrackInfosAtTrackFrame(EChaosVDTrackType TrackTypeToFind, const TSharedRef<const FChaosVDTrackInfo>& InFromTrack, TArray<TSharedPtr<const FChaosVDTrackInfo>>& OutTrackInfo)
{
	if (!LoadedRecording.IsValid())
	{
		return;
	}

	FReadScopeLock ReadLock(LoadedRecording->GetRecordingDataLock());
	GetAvailableTrackInfosAtTrackFrame_AssumesLocked(TrackTypeToFind, InFromTrack, OutTrackInfo);
}

void FChaosVDPlaybackController::GetAvailableTrackInfosAtTrackFrame_AssumesLocked(EChaosVDTrackType TrackTypeToFind, const TSharedRef<const FChaosVDTrackInfo>& InFromTrack, TArray<TSharedPtr<const FChaosVDTrackInfo>>& OutTrackInfo)
{
	OutTrackInfo.Reset();

	if (!LoadedRecording.IsValid())
	{
		return;
	}

	int32 CorrectedFrameNumber = INDEX_NONE;
	switch (InFromTrack->TrackType)
	{
	case EChaosVDTrackType::Game:
		{
			CorrectedFrameNumber = InFromTrack->CurrentFrame;
		}
		break;
	case EChaosVDTrackType::Solver:
		{
			CorrectedFrameNumber = LoadedRecording->GetLowestGameFrameAtSolverFrameNumber_AssumesLocked(InFromTrack->TrackID, InFromTrack->CurrentFrame);
			break;
		}
	default:
		ensure(false);
		break;
	}

	switch (TrackTypeToFind)
	{
	case EChaosVDTrackType::Game:
		{
			OutTrackInfo.Add(GetTrackInfo(EChaosVDTrackType::Game, GameTrackID));
			break;
		}
	case EChaosVDTrackType::Solver:
		{
			TArray<int32, TInlineAllocator<FChaosVDRecording::CommonTrackCount>> AvailableSolversID;
			LoadedRecording->GetAvailableSolverIDsAtGameFrameNumber_AssumesLocked(CorrectedFrameNumber, AvailableSolversID);
	
			TrackInfoByIDMap& TrackInfoMap = TrackInfoPerType.FindOrAdd(TrackTypeToFind);
			for (const int32 SolverID : AvailableSolversID)
			{
				// The recording might have the solver data available added because it was added the trace analysis thread, 
				// but the playback controller didn't process it in the game thread yet
				if (const TSharedPtr<FChaosVDTrackInfo>* SolverTrackInfo = TrackInfoMap.Find(SolverID))
				{
					OutTrackInfo.Add(*SolverTrackInfo);
				}	
			}
			break;
		}
	default:
		ensure(false);
		break;
	}
}

void FChaosVDPlaybackController::BroadcastGTDataUpdate(const TWeakPtr<FChaosVDPlaybackController>& InControllerWeakPtr)
{
	ensure(IsInGameThread());

	FChaosVDExtensionsManager::Get().EnumerateExtensions([InControllerWeakPtr](const TSharedRef<FChaosVDExtension>& Extension)
		{
			Extension->HandlePlaybackControllerDataUpdated(InControllerWeakPtr);
			return true;
		});

	ControllerUpdatedDelegate.Broadcast(InControllerWeakPtr);
}

void FChaosVDPlaybackController::ProcessQueuedTrackInfoUpdates(const TWeakPtr<FChaosVDPlaybackController>& InControllerWeakPtr)
{
	ensure(IsInGameThread());

	FChaosVDQueuedTrackInfoUpdate TrackInfoUpdate;
	while (TrackInfoUpdateGTQueue.Dequeue(TrackInfoUpdate))
	{
		OnTrackFrameUpdated().Broadcast(InControllerWeakPtr, TrackInfoUpdate.TrackInfo, TrackInfoUpdate.InstigatorID);

		FChaosVDExtensionsManager::Get().EnumerateExtensions([InControllerWeakPtr, &TrackInfoUpdate](const TSharedRef<FChaosVDExtension>& Extension)
			{
				Extension->HandleControllerTrackFrameUpdated(InControllerWeakPtr, TrackInfoUpdate.TrackInfo, TrackInfoUpdate.InstigatorID);
				return true;
			});
	}
}

bool FChaosVDPlaybackController::Tick(float DeltaTime)
{
	TickPlayback(DeltaTime);

	const TWeakPtr<FChaosVDPlaybackController> ThisWeakPtr = DoesSharedInstanceExist() ? AsWeak() : nullptr;
	if (!ThisWeakPtr.IsValid())
	{
		return true;
	}

	const bool bIsRecordingLoaded = LoadedRecording.IsValid();

	if (bIsRecordingLoaded)
	{
		uint64 CurrentLastUpdatedTime = LoadedRecording->GetLastUpdatedTimeAsCycle();
		if (CurrentLastUpdatedTime != RecordingLastSeenTimeUpdatedAsCycle)
		{
			RecordingLastSeenTimeUpdatedAsCycle = CurrentLastUpdatedTime;

			HandleCurrentRecordingUpdated();
		}
	}

	if (bHasPendingGTUpdateBroadcast)
	{
		BroadcastGTDataUpdate(ThisWeakPtr);
		bHasPendingGTUpdateBroadcast = false;
	}

	if (!TrackInfoUpdateGTQueue.IsEmpty())
	{
		ProcessQueuedTrackInfoUpdates(ThisWeakPtr);
	}

	if (bIsRecordingLoaded)
	{
		// Load at least the first frame
		if (!bPlayedFirstFrame)
		{
			// Make sure we have valid data for the first frame before trying to play it
			const TSharedPtr<const FChaosVDTrackInfo> GameTrackInfo = GetTrackInfo(EChaosVDTrackType::Game, GameTrackID);
			const bool bHasValidGameThreadTrackData = GameTrackInfo && GameTrackInfo->MaxFrames > 0;
			const TrackInfoByIDMap* TracksByType = bHasValidGameThreadTrackData ? TrackInfoPerType.Find(EChaosVDTrackType::Solver) : nullptr;

			if (const bool bHasSolverTrackData = TracksByType && !TracksByType->IsEmpty())
			{
				// If this a live session and automatic playback is enabled, start playing
				if (IsPlayingLiveSession() && bStartLiveSessionPlaybackAutomatically)
				{
					constexpr int32 GameFrameToLoad = 0;
					constexpr int32 SolverStageToLoad = 0;
					//Ensure Data Container Actors are created in time before the first frame callback.
					GoToTrackFrameAndSync(PlaybackSelfInstigatorID, EChaosVDTrackType::Game, GameTrackID, GameFrameToLoad, SolverStageToLoad);

					HandleFramePlaybackControlInput(EChaosVDPlaybackButtonsID::Play, GameTrackInfo.ToSharedRef(), PlaybackSelfInstigatorID);
					bPlayedFirstFrame = true;
				}
				else if (!LoadedRecording->IsEmpty())
				{
					constexpr int32 GameFrameToLoad = 0;
					// Game frames do not have stages
					constexpr int32 SolverStageToLoad = 0;
					GoToTrackFrameAndSync(PlaybackSelfInstigatorID, EChaosVDTrackType::Game, GameTrackID, GameFrameToLoad, SolverStageToLoad);
					bPlayedFirstFrame = true;
				}

				if (bPlayedFirstFrame)
				{
					const TWeakPtr<FChaosVDPlaybackController> InControllerWeakPtr = DoesSharedInstanceExist() ? AsWeak() : nullptr;

					FChaosVDExtensionsManager::Get().EnumerateExtensions([InControllerWeakPtr](const TSharedRef<FChaosVDExtension>& Extension)
					{
						Extension->HandleRecordingFirstFrameLoaded(InControllerWeakPtr);
						return true;
					});
				}
			}
		}

		// If we are live and playing, make sure we don't lag too much behind
		if (!bPauseRequested && IsPlayingLiveSession() && ActiveTrack->bIsPlaying)
		{
			if (const TSharedPtr<const FChaosVDTrackInfo> GameTrackInfo = GetTrackInfo(EChaosVDTrackType::Game, GameTrackID))
			{
				const int32 CurrentFrameDeltaFromLast = FMath::Abs(GameTrackInfo->MaxFrames - GameTrackInfo->CurrentFrame);
				if (CurrentFrameDeltaFromLast > MaxFramesLaggingBehindDuringLiveSession)
				{
					// Playing the middle point between last and the threshold. We don't want to play the last available frame as it could be incomplete,
					// and we don't want to go to close to the threshold.
					const int32 GameFrameToLoad = LoadedRecording->GetAvailableGameFramesNumber() - MinFramesLaggingBehindDuringLiveSession;
					constexpr int32 Step = 0;
					GoToTrackFrameAndSync(PlaybackSelfInstigatorID, EChaosVDTrackType::Game, GameTrackID, GameFrameToLoad, Step);
				}
			}
		}
	}

	return true;
}

bool FChaosVDPlaybackController::IsPlayingLiveSession() const
{
	return LoadedRecording.IsValid() ? LoadedRecording->IsLive() : false;
}

void FChaosVDPlaybackController::HandleDisconnectedFromSession()
{
	if (LoadedRecording.IsValid())
	{
		LoadedRecording->SetIsLive(false);
	}

	// Queue a general update in the Game Thread
	bHasPendingGTUpdateBroadcast = true;
}

void FChaosVDPlaybackController::StopPlayback(const FGuid& InstigatorGUID)
{
	if (TSharedPtr<const FChaosVDTrackInfo> GameFramesTrack = GetTrackInfo(EChaosVDTrackType::Game, GameTrackID))
	{
		constexpr int32 FrameNumber = 0;
		constexpr int32 StepNumber = 0;
	
		GoToTrackFrameAndSync(InstigatorGUID, GameFramesTrack->TrackType, GameFramesTrack->TrackID, FrameNumber, StepNumber);
	}

	VisitAvailableTracks([this](const TSharedPtr<FChaosVDTrackInfo>& TrackInfo)
	{
		if (TrackInfo)
		{
			TrackInfo->bIsPlaying = false;
		}

		return true;
	});
}

float FChaosVDPlaybackController::GetFrameTimeOverride() const
{
	constexpr int32 MinimumFrameRateOverride = 1;
	return (CurrentFrameRateOverride >= MinimumFrameRateOverride) ? 1.0f / static_cast<float>(CurrentFrameRateOverride) : InvalidFrameRateOverride;
}

int32 FChaosVDPlaybackController::GetFrameRateOverride() const
{
	constexpr int32 MinimumFrameRateOverride = 1;
	return (CurrentFrameRateOverride >= MinimumFrameRateOverride) ? CurrentFrameRateOverride : InvalidFrameRateOverride;
}

void FChaosVDPlaybackController::SetFrameRateOverride(float NewFrameRateOverride)
{
	constexpr int32 MinimumFrameRateOverride = 1;
	CurrentFrameRateOverride = (NewFrameRateOverride >= MinimumFrameRateOverride) ? static_cast<int32>(NewFrameRateOverride) : InvalidFrameRateOverride;
}

float FChaosVDPlaybackController::GetFrameTimeForTrack_AssumesLocked(EChaosVDTrackType TrackType, int32 TrackID,const TSharedRef<const FChaosVDTrackInfo>& InTrackInfo) const
{
	const float TargetFrameTimeOverride = GetFrameTimeOverride();
	const bool bHasValidFrameRateOverride = bUseFrameRateOverride && !FMath::IsNearlyEqual(TargetFrameTimeOverride, InvalidFrameRateOverride);
	if (bHasValidFrameRateOverride)
	{
		return TargetFrameTimeOverride;
	}

	float CurrentTargetFrameTime = FallbackFrameTime;
	if (LoadedRecording)
	{
		switch(TrackType)
		{
			case EChaosVDTrackType::Solver:
			{
				if (const FChaosVDSolverFrameData* FrameData = LoadedRecording->GetSolverFrameData_AssumesLocked(TrackID, InTrackInfo->CurrentFrame))
				{
					CurrentTargetFrameTime = static_cast<float>(FrameData->GetFrameTime());
				}
				break;
			}
			case EChaosVDTrackType::Game:
			{
				if (const FChaosVDGameFrameData* FrameData = LoadedRecording->GetGameFrameData_AssumesLocked(InTrackInfo->CurrentFrame))
				{
					CurrentTargetFrameTime = static_cast<float>(FrameData->GetFrameTime());
				}

				break;
			}
		default:
			break;
		}
	}

	return CurrentTargetFrameTime;
}

float FChaosVDPlaybackController::GetFrameTimeForTrack(EChaosVDTrackType TrackType, int32 TrackID, const TSharedRef<const FChaosVDTrackInfo>& InTrackInfo) const
{
	if (!LoadedRecording)
	{
		return FallbackFrameTime;
	}

	FReadScopeLock ReadLock(LoadedRecording->GetRecordingDataLock());
	return GetFrameTimeForTrack_AssumesLocked(TrackType, TrackID, InTrackInfo);
}

void FChaosVDPlaybackController::UpdateTrackVisibility(EChaosVDTrackType Type, int32 TrackID, bool bNewVisibility)
{
	switch (Type)
	{
		case EChaosVDTrackType::Solver:
			{
				if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneToControl.Pin())
				{
					if (AChaosVDSolverInfoActor* SolverActorInfo = ScenePtr->GetSolverInfoActor(TrackID))
					{
						SolverActorInfo->SetIsTemporarilyHiddenInEditor(!bNewVisibility);
					}
				}
				break;
			}
		case EChaosVDTrackType::Game:
		default:
			ensure(false);
			break;
	}
}

bool FChaosVDPlaybackController::IsTrackVisible(EChaosVDTrackType Type, int32 TrackID)
{
	switch (Type)
    {
    	case EChaosVDTrackType::Solver:
    		{
    			if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneToControl.Pin())
    			{
    				if (AChaosVDSolverInfoActor* SolverActorInfo = ScenePtr->GetSolverInfoActor(TrackID))
    				{
    					return SolverActorInfo->IsVisible();
    				}
    			}
    		return false;
    		}
    	case EChaosVDTrackType::Game:
    	default:
    			return ensure(false);	
    }
}

void FChaosVDPlaybackController::HandleFramePlaybackControlInput(EChaosVDPlaybackButtonsID ButtonID, const TSharedRef<const FChaosVDTrackInfo>& InTrackInfoRef, FGuid Instigator)
{
	if (!IsRecordingLoaded())
	{
		return;
	}

	if (InTrackInfoRef->TrackID != GameTrackID && !IsCompatibleWithSyncMode(InTrackInfoRef, CurrentSyncMode))
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Attempted to perform a playback action with an incompatible sync mode | Ignoring... ", ANSI_TO_TCHAR(__FUNCTION__));
		return;
	}

	if (InTrackInfoRef != ActiveTrack)
	{
		if (ButtonID == EChaosVDPlaybackButtonsID::Play && ActiveTrack->bIsPlaying)
		{
			UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Attempted to play when there was another track playing. This should not happen, this is probably a widget with controls not properly disabled.", ANSI_TO_TCHAR(__FUNCTION__));
			ActiveTrack->bIsPlaying = false;
		}
	}

	// We receive a const ref from the UI as the track info referenced there is read only, but once we are in the controller again we can and want to modify it.
	ActiveTrack = ConstCastSharedRef<FChaosVDTrackInfo>(InTrackInfoRef);

	switch (ButtonID)
	{
	case EChaosVDPlaybackButtonsID::Play:
		{
			bPauseRequested = false;

			ActiveTrack->bIsPlaying = true;
			break;
		}
	case EChaosVDPlaybackButtonsID::Pause:
		{
			if (ensure(ActiveTrack->bIsPlaying))
			{
				ActiveTrack->bIsPlaying = false;
			}
			else
			{
				UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Attempted to pause but there was not a track currently playing. This should not happen.", ANSI_TO_TCHAR(__FUNCTION__));
			}

			bPauseRequested = true;

			break;
		}
	case EChaosVDPlaybackButtonsID::Stop:
		{
			StopPlayback(Instigator);
			break;
		}
	case EChaosVDPlaybackButtonsID::Next:
		{
			const int32 NextFrame = InTrackInfoRef->CurrentFrame +1;
			const int32 LastSolverStage = GetTrackLastStageAtFrame(InTrackInfoRef->TrackType, InTrackInfoRef->TrackID, NextFrame);
			GoToTrackFrameAndSync(Instigator, InTrackInfoRef->TrackType, InTrackInfoRef->TrackID, NextFrame, LastSolverStage);
		}
		break;
	case EChaosVDPlaybackButtonsID::Prev:
		{
			const int32 PrevFrame = InTrackInfoRef->CurrentFrame -1;
			const int32 LastSolverStage = GetTrackLastStageAtFrame(InTrackInfoRef->TrackType, InTrackInfoRef->TrackID, PrevFrame);
			GoToTrackFrameAndSync(Instigator, InTrackInfoRef->TrackType, InTrackInfoRef->TrackID, PrevFrame, LastSolverStage);
		}
		break;
	default:
		break;
	}
}

void FChaosVDPlaybackController::HandleFrameStagePlaybackControlInput(EChaosVDPlaybackButtonsID ButtonID, const TSharedRef<const FChaosVDTrackInfo>& InTrackInfoRef, FGuid Instigator)
{
	if (!IsRecordingLoaded())
	{
		return;
	}

	if (ActiveTrack->bIsPlaying)
	{
		return;
	}

	// We receive a const ref from the UI as the track info referenced there is read only, but once we are in the controller again we can and want to modify it.
	ActiveTrack = ConstCastSharedRef<FChaosVDTrackInfo>(InTrackInfoRef);

	switch (ButtonID)
	{
	case EChaosVDPlaybackButtonsID::Next:
		{
			int32 NextSolverStage = InTrackInfoRef->CurrentStage +1;
			GoToTrackFrame(Instigator, EChaosVDTrackType::Solver, InTrackInfoRef->TrackID, InTrackInfoRef->CurrentFrame, NextSolverStage);
			break;
		}
	case EChaosVDPlaybackButtonsID::Prev:
		{
			int32 PrevSolverStage = InTrackInfoRef->CurrentStage -1;
			GoToTrackFrame(Instigator, EChaosVDTrackType::Solver, InTrackInfoRef->TrackID, InTrackInfoRef->CurrentFrame, PrevSolverStage);
		}
		break;
	case EChaosVDPlaybackButtonsID::Play:
	case EChaosVDPlaybackButtonsID::Pause:
	case EChaosVDPlaybackButtonsID::Stop:
	default:
		{
			ensure(false);
			UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Unsupported Input type | [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *(UEnum::GetDisplayValueAsText(ButtonID).ToString()));
			break;
		}
	}
}

void FChaosVDPlaybackController::TickPlayback(float DeltaTime)
{
	if (!ActiveTrack->IsValidTrack() || !ActiveTrack->bIsPlaying)
	{
		return;
	}

	if (!IsPlayingLiveSession() && ActiveTrack->CurrentFrame == ActiveTrack->MaxFrames - 1)
	{
		StopPlayback(PlaybackSelfInstigatorID);
		return;
	}

	CurrentPlaybackTime += DeltaTime;

	const float TargetFrameTime = GetFrameTimeForTrack(ActiveTrack->TrackType, ActiveTrack->TrackID, ActiveTrack);

	const bool bIsGameTrack = ActiveTrack->TrackType == EChaosVDTrackType::Game;

	const int32 FramesToAdvance = static_cast<int32>(FMath::Floor(CurrentPlaybackTime / TargetFrameTime));

	CurrentPlaybackTime -= TargetFrameTime * static_cast<float>(FramesToAdvance);

	const int32 NextFrame = FMath::Clamp(ActiveTrack->CurrentFrame + FramesToAdvance, 0, ActiveTrack->MaxFrames);
	const int32 NextStageNumber =  bIsGameTrack ? 0 : GetTrackLastStageAtFrame(ActiveTrack->TrackType, ActiveTrack->TrackID, NextFrame);
	GoToTrackFrameAndSync(PlaybackSelfInstigatorID, ActiveTrack->TrackType, ActiveTrack->TrackID, NextFrame, NextStageNumber);
}

void FChaosVDPlaybackController::GetTracksByType(EChaosVDTrackType Type, TArray<TSharedPtr<FChaosVDTrackInfo>>& OutTracks)
{
	OutTracks.Reset();
	if (TrackInfoByIDMap* TracksById = TrackInfoPerType.Find(Type))
	{
		TracksById->GenerateValueArray(OutTracks);
	}
}

void FChaosVDPlaybackController::SyncTracks(const TSharedRef<const FChaosVDTrackInfo>& FromTrack, FGuid InstigatorID, EChaosVDSyncTimelinesMode TrackSyncMode)
{
	if (!LoadedRecording.IsValid())
	{
		return;
	}

	FReadScopeLock ReadLock(LoadedRecording->GetRecordingDataLock());	
	SyncTracks_AssumesLocked(FromTrack, InstigatorID, TrackSyncMode);
}

void FChaosVDPlaybackController::SyncTracks_AssumesLocked(const TSharedRef<const FChaosVDTrackInfo>& FromTrack, FGuid InstigatorID, EChaosVDSyncTimelinesMode TrackSyncMode)
{
	if (!FromTrack->bTrackSyncEnabled)
	{
		return;
	}

	TArray<TSharedPtr<const FChaosVDTrackInfo>> AvailableSolverTracks;
	GetAvailableTrackInfosAtTrackFrame_AssumesLocked(EChaosVDTrackType::Solver, FromTrack, AvailableSolverTracks);

	switch (FromTrack->TrackType)
	{
	case EChaosVDTrackType::Game:
		{
			// Game track types are old and will be deprecated in the future (UE-277464)
			// These only support the default time-stamp sync mode.
			// So if we are in an incompatible sync mode, instead of fully disabling its use, we first try to find a solver track that is compatible, advance that using the timestamp base mode
			// and then sync the rest using the desired sync mode.
			// This is ok because the actual data we visualize is on solver tracks, except scene queries (hence why we plan to deprecate game tracks in the future)
			// This a common case in PIE, when the recording is started before starting PIE
			if (!IsCompatibleWithSyncMode(FromTrack, CurrentSyncMode))
			{
				TSharedPtr<const FChaosVDTrackInfo> TrackToReSyncFrom = GetTrackToReSyncFromOnModeChange_AssumesLocked(CurrentSyncMode);

				const bool bCanUseNewTrackForResync = TrackToReSyncFrom && TrackToReSyncFrom->TrackID != GameTrackID;
				if (bCanUseNewTrackForResync)
				{
					const int32 SolverFrameNumber = LoadedRecording->GetLowestSolverFrameNumberGameFrame_AssumesLocked(TrackToReSyncFrom->TrackID, FromTrack->CurrentFrame);
					const int32 StageNumber = GetTrackLastStageAtFrame_AssumesLocked(EChaosVDTrackType::Solver, TrackToReSyncFrom->TrackID, SolverFrameNumber);

					GoToTrackFrame_AssumesLocked(InstigatorID, EChaosVDTrackType::Solver, TrackToReSyncFrom->TrackID, SolverFrameNumber, StageNumber);
					
					return SyncTracks_AssumesLocked(TrackToReSyncFrom.ToSharedRef(), InstigatorID, TrackSyncMode);
				}
			}
	
			for (const TSharedPtr<const FChaosVDTrackInfo>& SolverTrack : AvailableSolverTracks)
			{
				if (!SolverTrack->bTrackSyncEnabled)
				{
					continue;
				}

				const int32 SolverFrameNumber = LoadedRecording->GetLowestSolverFrameNumberGameFrame_AssumesLocked(SolverTrack->TrackID, FromTrack->CurrentFrame);
				const int32 StageNumber = GetTrackLastStageAtFrame_AssumesLocked(EChaosVDTrackType::Solver, SolverTrack->TrackID, SolverFrameNumber);

				GoToTrackFrame_AssumesLocked(InstigatorID, EChaosVDTrackType::Solver, SolverTrack->TrackID, SolverFrameNumber, StageNumber);
			}
			break;
		}	
	case EChaosVDTrackType::Solver:
		{
			if (const TSharedPtr<const FChaosVDTrackInfo>& GameFramesTrackInfo = GetTrackInfo(EChaosVDTrackType::Game, GameTrackID))
			{
				const int32 TargetGameFrameNumber = ConvertCurrentFrameToOtherTrackFrame_AssumesLocked(FromTrack, GameFramesTrackInfo.ToSharedRef(), TrackSyncMode);
				constexpr int32 StageNumber = 0;
				GoToTrackFrame_AssumesLocked(InstigatorID, GameFramesTrackInfo->TrackType, GameFramesTrackInfo->TrackID, TargetGameFrameNumber, StageNumber);
			}

			for (const TSharedPtr<const FChaosVDTrackInfo>& SolverTrack : AvailableSolverTracks)
			{
				if (!SolverTrack->bTrackSyncEnabled)
				{
					continue;
				}

				if (FChaosVDTrackInfo::AreSameTrack(FromTrack, SolverTrack.ToSharedRef()))
				{
					continue;
				}

				const int32 SolverFrameNumber = ConvertCurrentFrameToOtherTrackFrame_AssumesLocked(FromTrack, SolverTrack.ToSharedRef(), TrackSyncMode);
				const int32 StageNumber = GetTrackLastStageAtFrame_AssumesLocked(EChaosVDTrackType::Solver, SolverTrack->TrackID, SolverFrameNumber);

				GoToTrackFrame_AssumesLocked(InstigatorID, EChaosVDTrackType::Solver, SolverTrack->TrackID, SolverFrameNumber, StageNumber);
			}
			break;
		}
	default:
		ensure(false);
		break;
	}
}

void FChaosVDPlaybackController::ToggleTrackSyncEnabled(const TSharedRef<const FChaosVDTrackInfo>& InTrackInfoRef)
{
	TSharedRef<FChaosVDTrackInfo> MutableTrackInfoRef = ConstCastSharedRef<FChaosVDTrackInfo>(InTrackInfoRef);
	MutableTrackInfoRef->bTrackSyncEnabled = !MutableTrackInfoRef->bTrackSyncEnabled;
}

bool FChaosVDPlaybackController::IsPlaying() const
{
	return ActiveTrack->bIsPlaying;
}

void FChaosVDPlaybackController::SetScrubbingTimeline(bool bNewIsScrubbingTimeline)
{
	bIsScrubbingTimeline = bNewIsScrubbingTimeline;
}

TSharedPtr<const FChaosVDTrackInfo> FChaosVDPlaybackController::GetTrackToReSyncFromOnModeChange(EChaosVDSyncTimelinesMode SyncMode)
{
	if (!LoadedRecording)
	{
		return nullptr;
	}

	FReadScopeLock ReadLock(LoadedRecording->GetRecordingDataLock());
	return GetTrackToReSyncFromOnModeChange_AssumesLocked(SyncMode);
}

TSharedPtr<const FChaosVDTrackInfo> FChaosVDPlaybackController::GetTrackToReSyncFromOnModeChange_AssumesLocked(EChaosVDSyncTimelinesMode SyncMode)
{
	switch (SyncMode)
	{
	case EChaosVDSyncTimelinesMode::RecordedTimestamp:
		{
			return ActiveTrack->IsValidTrack() ? ActiveTrack : GetTrackInfo(EChaosVDTrackType::Game, GameTrackID);
		}
	case EChaosVDSyncTimelinesMode::NetworkTick:
		{
			// Network ticks can only sync from other solver tracks
			if (ActiveTrack->IsValidTrack() && ActiveTrack->TrackType != EChaosVDTrackType::Game)
			{
				return ActiveTrack;
			}

			TSharedPtr<const FChaosVDTrackInfo> GameThreadTrack = GetTrackInfo(EChaosVDTrackType::Game, GameTrackID);
			if (GameThreadTrack && GameThreadTrack->IsValidTrack())
			{
				TArray<TSharedPtr<const FChaosVDTrackInfo>> AvailableSolverTracks;
				GetAvailableTrackInfosAtTrackFrame_AssumesLocked(EChaosVDTrackType::Solver, GameThreadTrack.ToSharedRef(), AvailableSolverTracks);

				for (const TSharedPtr<const FChaosVDTrackInfo>& Track : AvailableSolverTracks)
				{
					if (Track->bHasNetworkSyncData)
					{
						return Track;
					}
				}

				return nullptr;
			}

			return nullptr;
		}
	default:
		return nullptr;
	}
}

bool FChaosVDPlaybackController::IsCompatibleWithSyncMode(const TSharedRef<const FChaosVDTrackInfo>& InTrackRef, EChaosVDSyncTimelinesMode SyncMode)
{
	switch (SyncMode)
	{
		case EChaosVDSyncTimelinesMode::RecordedTimestamp:
			return true;
		case EChaosVDSyncTimelinesMode::NetworkTick:
		{
			if (InTrackRef->TrackID == GameTrackID)
			{
				return false;
			}
			return InTrackRef->bHasNetworkSyncData;
		}
		case EChaosVDSyncTimelinesMode::Manual:
			return true;
		case EChaosVDSyncTimelinesMode::None:
		default:
			return false;
	}
}

void FChaosVDPlaybackController::SetTimelineSyncMode(EChaosVDSyncTimelinesMode SyncMode)
{
	if (CurrentSyncMode != SyncMode)
	{
		CurrentSyncMode = SyncMode;

		const TSharedPtr<const FChaosVDTrackInfo> TrackToReSyncFrom = GetTrackToReSyncFromOnModeChange(SyncMode);

		if (TrackToReSyncFrom && TrackToReSyncFrom->IsValidTrack())
		{
			bool bWasIsPlaying = IsPlaying();
			const TSharedRef<const FChaosVDTrackInfo> TrackToReSyncFromRef = TrackToReSyncFrom.ToSharedRef();

			TrySetActiveTrack(TrackToReSyncFromRef);

			if (CurrentSyncMode != EChaosVDSyncTimelinesMode::Manual)
			{
				SyncTracks(TrackToReSyncFromRef, PlaybackSelfInstigatorID, CurrentSyncMode);
			}

			if (bWasIsPlaying)
			{
				HandleFramePlaybackControlInput(EChaosVDPlaybackButtonsID::Play, TrackToReSyncFromRef, PlaybackSelfInstigatorID);
			}

			OnDataUpdated().Broadcast(AsWeak());
		}
		else
		{
			UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Failed to re-sync tracks after sync mode was changed.", ANSI_TO_TCHAR(__FUNCTION__));
		}
	}
}

void FChaosVDPlaybackController::TrySetActiveTrack(int32 SlotIndex)
{
	VisitAvailableTracks([this, SlotIndex](const TSharedPtr<FChaosVDTrackInfo>& TrackInfo)
	{
		if (TrackInfo && TrackInfo->TrackSlot == SlotIndex)
		{
			if (ActiveTrack->IsValidTrack())
			{
				ActiveTrack->bIsPlaying = false;
			}

			ActiveTrack = TrackInfo.ToSharedRef();
			return false;
		}

		return true;
	});
}

void FChaosVDPlaybackController::TrySetActiveTrack(const TSharedRef<const FChaosVDTrackInfo>& NewActiveTrack)
{
	VisitAvailableTracks([this, NewActiveTrack](const TSharedPtr<FChaosVDTrackInfo>& TrackInfo)
	{
		if (TrackInfo && FChaosVDTrackInfo::AreSameTrack(TrackInfo.ToSharedRef(), NewActiveTrack))
		{
			ActiveTrack->bIsPlaying = false;
			ActiveTrack = TrackInfo.ToSharedRef();
			return false;
		}

		return true;
	});
}

FChaosVDPlaybackEngineSnapshot FChaosVDPlaybackController::GeneratePlaybackEngineSnapshot()
{
	FChaosVDPlaybackEngineSnapshot PlaybackEngineState;

	PlaybackEngineState.TimelineSyncMode = GetTimelineSyncMode();

	if (const TSharedPtr<const FChaosVDTrackInfo> GameTrackInfo = GetTrackInfo(EChaosVDTrackType::Game, GameTrackID))
	{
		PlaybackEngineState.FrameIndexDataByTrackID.Add(GameTrackID, { GameTrackInfo->CurrentFrame, GameTrackInfo->CurrentStage });
		TArray<TSharedPtr<const FChaosVDTrackInfo>> TrackInfoArray;
		GetAvailableTrackInfosAtTrackFrame(EChaosVDTrackType::Solver, GameTrackInfo.ToSharedRef(), TrackInfoArray);

		for (const TSharedPtr<const FChaosVDTrackInfo>& SolversTrackInfo : TrackInfoArray)
		{
			if (SolversTrackInfo)
			{
				PlaybackEngineState.FrameIndexDataByTrackID.Add(SolversTrackInfo->TrackID, { SolversTrackInfo->CurrentFrame, SolversTrackInfo->CurrentStage });
			}
		}
	}

	FChaosVDExtensionsManager::Get().EnumerateExtensions([&PlaybackEngineState](const TSharedRef<FChaosVDExtension>& Extension)
	{
		PlaybackEngineState.InstalledExtensionsNames.Add(Extension->GetExtensionType());
		return true;
	});

	if (TSharedPtr<FChaosVDScene> CVDScenePtr = SceneToControl.Pin())
	{
		CVDScenePtr->AppendSceneCompositionTestData(PlaybackEngineState);
	}

	return PlaybackEngineState;
}

void FChaosVDPlaybackController::UpdateSolverTracksData()
{
	if (!LoadedRecording.IsValid())
	{
		// If the recording is no longer valid, clear any existing solver track info data so the UI can be updated accordingly
		if (TrackInfoByIDMap* SolverTracks = TrackInfoPerType.Find(EChaosVDTrackType::Solver))
		{
			SolverTracks->Empty();
		}

		return;
	}

	FReadScopeLock RecordingReadLock(LoadedRecording->GetRecordingDataLock());

	const TMap<int32, TArray<FChaosVDSolverFrameData>>& SolversByID = LoadedRecording->GetAvailableSolvers_AssumesLocked();
	for (const TPair<int32, TArray<FChaosVDSolverFrameData>>& SolverIDPair : SolversByID)
	{
		TSharedPtr<FChaosVDTrackInfo>& SolverTrackInfo = TrackInfoPerType[EChaosVDTrackType::Solver].FindOrAdd(SolverIDPair.Key);

		bool bIsNewTrack = !SolverTrackInfo.IsValid();
		if (bIsNewTrack)
		{
			SolverTrackInfo = CreateTrackInfo();
			// Solver frame tracks only holding GT Data, these should not be shown in the timeline controls UI
			// TODO: The following is being commented out because although the comment above is correct, we can end up with two or more source files whose game thread
			// ticked at different rates. The main game frames timeline controls are populated from the first loaded file, therefore this means that if that recording had its GT ticking slower
			// , when you scrub forward, the other GT tracks might skip multiple frames (correctly) but you will miss data without having anything in the UI telling you what happened.
			// This will be addressed when we fully remove the game thread data tracks as unique type in the next release (UE-277464), but for now it means we need to keep these track timeline controls enabled
			//SolverTrackInfo->bCanShowTrackControls = !SolverIDPair.Value.IsEmpty() ? !EnumHasAnyFlags(SolverIDPair.Value[0].GetAttributes(), EChaosVDSolverFrameAttributes::HasGTDataToReRoute) : true;
			SolverTrackInfo->bCanShowTrackControls = true;
			SolverTrackInfo->CurrentFrame = 0;
			SolverTrackInfo->CurrentStage = 0;
		}
		
		SolverTrackInfo->TrackID = SolverIDPair.Key;
		SolverTrackInfo->MaxFrames = GetTrackFramesNumber_AssumesLocked(EChaosVDTrackType::Solver, SolverIDPair.Key);
		SolverTrackInfo->TrackName = LoadedRecording->GetSolverFName_AssumedLocked(SolverIDPair.Key);
		SolverTrackInfo->TrackType = EChaosVDTrackType::Solver;
		SolverTrackInfo->bIsServer = LoadedRecording->IsServerSolver_AssumesLocked(SolverIDPair.Key);

		if (bIsNewTrack)
		{
			SyncTracks_AssumesLocked(ActiveTrack, PlaybackSelfInstigatorID, CurrentSyncMode);
		}
	}
}

void FChaosVDPlaybackController::HandleCurrentRecordingUpdated()
{
	// These two tracks should always exist
	TrackInfoPerType.FindOrAdd(EChaosVDTrackType::Game);
	TrackInfoPerType.FindOrAdd(EChaosVDTrackType::Solver);

	// Same for the Game Track, needs to always exist
	TSharedPtr<FChaosVDTrackInfo>& GameTrackInfo = TrackInfoPerType[EChaosVDTrackType::Game].FindOrAdd(GameTrackID);
	if (!GameTrackInfo.IsValid())
	{
		GameTrackInfo = CreateTrackInfo(GameTrackSlot);
		GameTrackInfo->TrackID = GameTrackID;
		GameTrackInfo->CurrentFrame = 0;
		GameTrackInfo->CurrentStage = 0;

		// If no track is active, let's default to the game tracks as it is the only one guaranteed to always exists
		if (!ActiveTrack->IsValidTrack())
		{
			ActiveTrack = GameTrackInfo.ToSharedRef();
		}
	}

	GameTrackInfo->MaxFrames = LoadedRecording.IsValid() ? LoadedRecording->GetAvailableGameFramesNumber() : INDEX_NONE;
	GameTrackInfo->TrackType = EChaosVDTrackType::Game;

	// Each time the recording is updated, populate or update the existing solver tracks data
	UpdateSolverTracksData();

	bHasPendingGTUpdateBroadcast = true;
}

#undef LOCTEXT_NAMESPACE
