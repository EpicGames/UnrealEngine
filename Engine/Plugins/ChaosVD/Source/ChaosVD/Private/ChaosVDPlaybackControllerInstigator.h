// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Misc/Guid.h"

/** Common interface for any object that alters a ChaosVDPlaybackController */
class IChaosVDPlaybackControllerInstigator
{
public:
	IChaosVDPlaybackControllerInstigator() : ID(FGuid::NewGuid())
	{
	}

	static FGuid InvalidGuid;

	const FGuid& GetInstigatorID() const { return ID;};
private:
	FGuid ID;
};
