// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDPlaybackControllerObserver.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDScene.h"


FChaosVDPlaybackControllerObserver::~FChaosVDPlaybackControllerObserver()
{
	if (const TSharedPtr<FChaosVDPlaybackController> CurrentPlaybackControllerPtr = PlaybackController.Pin())
	{
		CurrentPlaybackControllerPtr->OnDataUpdated().RemoveAll(this);
		CurrentPlaybackControllerPtr->OnTrackFrameUpdated().RemoveAll(this);
	}
}

TWeakPtr<FChaosVDPlaybackController> FChaosVDPlaybackControllerObserver::GetObservedController()
{
	return PlaybackController;
}

void FChaosVDPlaybackControllerObserver::RegisterNewController(TWeakPtr<FChaosVDPlaybackController> NewController)
{
	if (PlaybackController != NewController)
	{
		if (const TSharedPtr<FChaosVDPlaybackController> CurrentPlaybackControllerPtr = PlaybackController.Pin())
		{
			CurrentPlaybackControllerPtr->OnDataUpdated().RemoveAll(this);
			CurrentPlaybackControllerPtr->OnTrackFrameUpdated().RemoveAll(this);
		}

		PlaybackController = NewController;

		if (const TSharedPtr<FChaosVDPlaybackController> NewPlaybackControllerPtr = PlaybackController.Pin())
		{
			if (TSharedPtr<FChaosVDScene> ScenePtr = NewPlaybackControllerPtr->GetControllerScene().Pin())
			{
				RegisterSelectionSetObject(ScenePtr->GetElementSelectionSet());
			}
	
			NewPlaybackControllerPtr->OnDataUpdated().AddRaw(this, &FChaosVDPlaybackControllerObserver::HandlePlaybackControllerDataUpdated);
			NewPlaybackControllerPtr->OnTrackFrameUpdated().AddRaw(this, &FChaosVDPlaybackControllerObserver::HandleControllerTrackFrameUpdated);
			HandlePlaybackControllerDataUpdated(PlaybackController);
		}
	}
}
