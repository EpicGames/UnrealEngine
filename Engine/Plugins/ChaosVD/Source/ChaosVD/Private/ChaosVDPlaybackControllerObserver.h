// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDSceneSelectionObserver.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"

struct FChaosVDTrackInfo;
class FChaosVDPlaybackController;


/**
 * Class to be used as base to any object that needs to Process Player Controller changes,
 * and scene Selection Changes
 */
class FChaosVDPlaybackControllerObserver : public FChaosVDSceneSelectionObserver
{
public:

	virtual ~FChaosVDPlaybackControllerObserver() override;

	/** Returns a Weak Ptr to the Playback Controller we are currently observing */
	TWeakPtr<FChaosVDPlaybackController> GetObservedController();

protected:
	virtual void RegisterNewController(TWeakPtr<FChaosVDPlaybackController> NewController);
	virtual void HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InController) {};
	virtual void HandleControllerTrackFrameUpdated(TWeakPtr<FChaosVDPlaybackController> InController, TWeakPtr<const FChaosVDTrackInfo> UpdatedTrackInfo, FGuid InstigatorGuid){};
	
	virtual void HandlePostSelectionChange(const UTypedElementSelectionSet* ChangesSelectionSet) override{};
	
	TWeakPtr<FChaosVDPlaybackController> PlaybackController;
};
