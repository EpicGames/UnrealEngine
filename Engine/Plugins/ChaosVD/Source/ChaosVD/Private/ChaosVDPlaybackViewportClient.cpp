// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVD/Public/ChaosVDPlaybackViewportClient.h"

#include "CanvasTypes.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "ChaosVDScene.h"
#include "ChaosVDSkySphereInterface.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "ComponentVisualizer.h"
#include "EditorModeManager.h"
#include "Engine/DirectionalLight.h"
#include "EngineUtils.h"
#include "SceneView.h"
#include "SEditorViewport.h"
#include "Selection.h"
#include "UnrealWidget.h"
#include "TEDS/ChaosVDSelectionInterface.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"
#include "Widgets/SChaosVDMainTab.h"

FChaosVDPlaybackViewportClient::FChaosVDPlaybackViewportClient(const TSharedPtr<FEditorModeTools>& InModeTools, const TSharedPtr<SEditorViewport>& InEditorViewportWidget) : FEditorViewportClient(InModeTools.Get(), nullptr, InEditorViewportWidget), CVDWorld(nullptr)
{
	Widget->SetUsesEditorModeTools(InModeTools.Get());

	if (GEngine)
	{
		GEngine->OnActorMoving().AddRaw(this, &FChaosVDPlaybackViewportClient::HandleActorMoving);
	}

	// Set the default to infinity (the engine treats 0 as inifinity), because CVD's streaming system already handles what should be rendered
	constexpr float DefaultFarClipPlaneOverride = 0.0f;
	OverrideFarClipPlane(DefaultFarClipPlaneOverride);
	OverrideNearClipPlane(UE_KINDA_SMALL_NUMBER);
}

FChaosVDPlaybackViewportClient::~FChaosVDPlaybackViewportClient()
{
	if (FocusRequestDelegateHandle.IsValid())
	{
		if (TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin())
		{
			ScenePtr->OnFocusRequest().Remove(FocusRequestDelegateHandle);
		}
	}

	if (GEngine)
	{
		GEngine->OnActorMoving().RemoveAll(this);
	}
}

void FChaosVDPlaybackViewportClient::ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY)
{
	FEditorViewportClient::ProcessClick(View, HitProxy, Key, Event, HitX, HitY);

	if (HitProxy == nullptr)
	{
		return;
	}
	
	const TSharedPtr<SChaosVDMainTab> MainTabToolkitHost = ModeTools.IsValid() ? StaticCastSharedPtr<SChaosVDMainTab>(ModeTools->GetToolkitHost()) : nullptr;
	if (!MainTabToolkitHost.IsValid())
	{
		return;
	}
	
	const bool bIsShiftKeyDown = Viewport->KeyState(EKeys::LeftShift) || Viewport->KeyState(EKeys::RightShift);

	const FViewportClick Click(&View, this, Key, Event, HitX, HitY);

	if (const TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin())
	{
		bool bClickHandled = false;

		HComponentVisProxy* ComponentVisProxy = HitProxyCast<HComponentVisProxy>(HitProxy);
		const TConstArrayView<TSharedPtr<FComponentVisualizer>> AllVisualizers = MainTabToolkitHost->GetAllComponentVisualizers();
		for (const TSharedPtr<FComponentVisualizer>& Visualizer : AllVisualizers)
		{
			// Not sure if this is compliant with the normal use of the component visualizers,
			// but passing a null hitproxy when the hit proxy was not a component
			// It allow us to handle things like clear selection on the Collision Data Visualizer
			if (Visualizer->VisProxyHandleClick(this, ComponentVisProxy, Click))
			{
				bClickHandled = true;
				break;
			}
		}

		if (bClickHandled)
		{
			return;
		}

		const IChaosVDGeometryComponent* AsCVDGeometryComponent = nullptr;
		int32 MeshInstanceIndex = INDEX_NONE;

		if (const HInstancedStaticMeshInstance* InstancedStaticMeshProxy = HitProxyCast<HInstancedStaticMeshInstance>(HitProxy))
		{
			AsCVDGeometryComponent = Cast<IChaosVDGeometryComponent>(InstancedStaticMeshProxy->Component);
			MeshInstanceIndex = InstancedStaticMeshProxy->InstanceIndex;
		}
		else if (const HActor* ActorHitProxy = HitProxyCast<HActor>(HitProxy))
		{
			AsCVDGeometryComponent = Cast<IChaosVDGeometryComponent>(ActorHitProxy->PrimComponent.Get());
			MeshInstanceIndex = 0;
		}

		if (AsCVDGeometryComponent && MeshInstanceIndex != INDEX_NONE)
		{
			if (const TSharedPtr<FChaosVDInstancedMeshData> MeshDataHandle = AsCVDGeometryComponent->GetMeshDataInstanceHandle(MeshInstanceIndex))
			{
				if (TSharedPtr<FChaosVDSceneParticle> ClickedActor = ScenePtr->GetParticleInstance(MeshDataHandle->GetOwningSolverID(), MeshDataHandle->GetOwningParticleID()))
				{
					Chaos::VisualDebugger::SelectParticleWithGeometryInstance(ScenePtr.ToSharedRef(), ClickedActor.Get(), bIsShiftKeyDown ? MeshDataHandle : nullptr);
					bClickHandled = true;
				}
			}
		}

		if (bClickHandled)
		{
			return;
		}

		if (HitProxy->IsA(HActor::StaticGetType()))
		{
			const HActor* ActorHitProxy = static_cast<HActor*>(HitProxy);
			if (AActor* ClickedActor = ActorHitProxy->Actor)
			{
				ScenePtr->SetSelectedObject(ClickedActor);
			}
		}
	}
}

void FChaosVDPlaybackViewportClient::SetScene(TWeakPtr<FChaosVDScene> InScene)
{
	if (TSharedPtr<FChaosVDScene> ScenePtr = InScene.Pin())
	{
		CVDWorld = ScenePtr->GetUnderlyingWorld();
		CVDScene = InScene;

		FocusRequestDelegateHandle = ScenePtr->OnFocusRequest().AddRaw(this, &FChaosVDPlaybackViewportClient::HandleFocusRequest);
	}
}

void FChaosVDPlaybackViewportClient::SetCanSelectTranslucentGeometry(bool bCanSelect)
{
	bAllowTranslucentHitProxies = bCanSelect;

	Invalidate();
}

void FChaosVDPlaybackViewportClient::ToggleCanSelectTranslucentGeometry()
{
	SetCanSelectTranslucentGeometry(!bAllowTranslucentHitProxies);
}

bool FChaosVDPlaybackViewportClient::Internal_InputKey(const FInputKeyEventArgs& EventArgs)
{
	if (bAutoTrackSelectedObject && !IsFlightCameraActive())
	{
		constexpr float TrackingViewZoomStepSize = 50.0f;
		const float CameraSpeed = GetCameraSpeed();
		const float TargetDistanceChangeStep = TrackingViewZoomStepSize * CameraSpeed;
		if (EventArgs.Key == EKeys::MouseScrollUp)
		{
			TrackingViewDistance -= TargetDistanceChangeStep;
		}
		else if (EventArgs.Key == EKeys::MouseScrollDown)
		{
			TrackingViewDistance += TargetDistanceChangeStep;
		}

		TrackingViewDistance = TrackingViewDistance < 0 ? 0 : TrackingViewDistance;
	}

	return FEditorViewportClient::Internal_InputKey(EventArgs);
}

void FChaosVDPlaybackViewportClient::ExecuteVisualizationCallback(const TSharedRef<SChaosVDMainTab>& InMainTabToolkitHostRef,
	const TSharedRef<FChaosVDScene>& SceneRef, const TFunction<void(const UActorComponent*, const TSharedRef<FComponentVisualizer>&)>& VisitorCallback)
{
	//TODO: Currently we can safely assume that any component in these actors is meant to have a visualizer, but we might need a proper interface for these components in the future
	TInlineComponentArray<const UActorComponent*> ComponentsToVisualize;

	TConstArrayView<TObjectPtr<AChaosVDDataContainerBaseActor>> DataContainerActors = SceneRef->GetDataContainerActorsView();
	for (const TObjectPtr<AChaosVDDataContainerBaseActor>& DataContainerActor : DataContainerActors)
	{
		if (DataContainerActor)
		{
			constexpr bool bIncludeFromChildActors = false;
			DataContainerActor->ForEachComponent(bIncludeFromChildActors, [&ComponentsToVisualize](UActorComponent* Component)
			{
				ComponentsToVisualize.Emplace(Component);
			});
		}
	}

	for (const UActorComponent* Component : ComponentsToVisualize)
	{
		if (!FChaosVDDebugDrawUtils::CanDebugDraw())
		{
			break;
		}

		if (const TSharedPtr<FComponentVisualizer> Visualizer = InMainTabToolkitHostRef->FindComponentVisualizer(Component->GetClass()))
		{
			VisitorCallback(Component, Visualizer.ToSharedRef());
		}
	}
}

void FChaosVDPlaybackViewportClient::HandleFocusRequest(FBox BoxToFocusOn)
{
	FocusViewportOnBox(BoxToFocusOn);
}

void FChaosVDPlaybackViewportClient::HandleActorMoving(AActor* MovedActor) const
{
	if (Cast<ADirectionalLight>(MovedActor))
	{
		if (const TSharedPtr<FChaosVDScene> SceneSharedPtr = CVDScene.Pin())
		{
			if (SceneSharedPtr->GetSkySphereActor()->Implements<UChaosVDSkySphereInterface>())
			{
				FEditorScriptExecutionGuard AllowEditorScriptGuard;
				IChaosVDSkySphereInterface::Execute_Refresh(SceneSharedPtr->GetSkySphereActor());
			}
		}
	}
}

void FChaosVDPlaybackViewportClient::UpdateObjectTracking()
{
	if (!bAutoTrackSelectedObject)
	{
		ToggleOrbitCamera(false);
		return;
	}

	const FBox SelectionBounds = GetSelectionBounds();
	ToggleOrbitCamera(static_cast<bool>(SelectionBounds.IsValid));

	if (SelectionBounds.IsValid)
	{
		const float TargetViewDistance = TrackingViewDistance + static_cast<float>(SelectionBounds.GetExtent().Size());
		SetViewLocationForOrbiting(SelectionBounds.GetCenter(), TargetViewDistance);
	}
}

void FChaosVDPlaybackViewportClient::FocusOnSelectedObject()
{
	FBox SelectionBounds = GetSelectionBounds();
	if (SelectionBounds.IsValid)
	{
		FocusViewportOnBox(SelectionBounds.ExpandBy(TrackingViewDistance), true);
	}
}

FBox FChaosVDPlaybackViewportClient::GetSelectionBounds() const
{
	if (const TSharedPtr<FChaosVDScene> CVDSceneSharedPtr = CVDScene.Pin())
	{
		UTypedElementSelectionSet* SelectionSet = CVDSceneSharedPtr->GetElementSelectionSet();

		//TODO: Update this if we add multi selection support
		constexpr int32 MaxElements = 1;
		TArray<FTypedElementHandle, TInlineAllocator<MaxElements>> SelectedParticlesHandles;
		SelectionSet->GetSelectedElementHandles(SelectedParticlesHandles, UChaosVDSelectionInterface::StaticClass());

		if (SelectedParticlesHandles.Num() > 0)
		{
			using namespace Chaos::VD::TypedElementDataUtil;
			if (FChaosVDSceneParticle* Particle = GetStructDataFromTypedElementHandle<FChaosVDSceneParticle>(SelectedParticlesHandles[0]))
			{
				return Particle->GetBoundingBox();
			}
		}
	}

	return FBox();
}

void FChaosVDPlaybackViewportClient::UpdateMouseDelta()
{
	// Make sure we get the camera in the correct position before a mouse drag is handled
	UpdateObjectTracking();

	FEditorViewportClient::UpdateMouseDelta();
}

void FChaosVDPlaybackViewportClient::HandleCVDSceneUpdated()
{
	UpdateObjectTracking();
	Invalidate();
}

void FChaosVDPlaybackViewportClient::ToggleObjectTrackingIfSelected()
{
	bAutoTrackSelectedObject = !bAutoTrackSelectedObject;
}

void FChaosVDPlaybackViewportClient::SetAutoTrackingViewDistance(float NewDistance)
{
	TrackingViewDistance = NewDistance;
}

void FChaosVDPlaybackViewportClient::GoToLocation(const FVector& InLocation)
{
	FViewportCameraTransform& ViewTransform = GetViewTransform();
	ViewTransform.SetLocation(InLocation);

	Invalidate();
}

void FChaosVDPlaybackViewportClient::Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	if (View)
	{
		// Hack to allow CVD control the selection of translucent objects (for CVD is all geometry set as Query Only)
		// The current setting to allow this behaviour is project wide or on custom hitproxies implementations which we can't use
		// A proper fix would be to have a way to override this per viewport, which could be done by adding a new method to FViewElementDrawer
		const_cast<FSceneView*>(View)->bAllowTranslucentPrimitivesInHitProxy = bAllowTranslucentHitProxies;
	}

	const TSharedPtr<SChaosVDMainTab> MainTabToolkitHost = ModeTools.IsValid() ? StaticCastSharedPtr<SChaosVDMainTab>(ModeTools->GetToolkitHost()) : nullptr;
	if (!MainTabToolkitHost.IsValid())
	{
		return;
	}

	if (const TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin())
	{
		ScenePtr->UpdateWorldStreamingLocation(GetViewLocation());

		auto DrawComponentVisualization = [&View, &PDI](const UActorComponent* Component, const TSharedRef<FComponentVisualizer>& InVisualizer)
		{
			InVisualizer->DrawVisualization(Component, View, PDI);
		};

		ExecuteVisualizationCallback(MainTabToolkitHost.ToSharedRef(), ScenePtr.ToSharedRef(), DrawComponentVisualization);
	}

	FEditorViewportClient::Draw(View, PDI);
	
	FChaosVDDebugDrawUtils::DebugDrawFrameEnd();
}

void FChaosVDPlaybackViewportClient::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	FEditorViewportClient::DrawCanvas(InViewport, View, Canvas);

	FChaosVDDebugDrawUtils::DrawCanvas(InViewport, View, Canvas);

	{
		
		const TSharedPtr<SChaosVDMainTab> MainTabToolkitHost = ModeTools.IsValid() ? StaticCastSharedPtr<SChaosVDMainTab>(ModeTools->GetToolkitHost()) : nullptr;
		const TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin();

		if (MainTabToolkitHost && ScenePtr)
		{
			auto DrawComponentHUDVisualization = [&View, &Canvas, &InViewport](const UActorComponent* Component, const TSharedRef<FComponentVisualizer>& InVisualizer)
			{
				InVisualizer->DrawVisualizationHUD(Component, &InViewport, &View, &Canvas);
			};

			ExecuteVisualizationCallback(MainTabToolkitHost.ToSharedRef(), ScenePtr.ToSharedRef(), DrawComponentHUDVisualization);
		}
	}
}
