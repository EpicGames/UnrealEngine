// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDPlaybackViewportTab.h"

#include "ChaosVDEngine.h"
#include "ChaosVDStyle.h"
#include "Widgets/SChaosVDPlaybackViewport.h"
#include "ChaosVDTabsIDs.h"
#include "EditorModeManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SChaosVDMainTab.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

TSharedRef<SDockTab> FChaosVDPlaybackViewportTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> ViewportTab =
		SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("ViewportTabLabel", "Playback Viewport"))
		.ToolTipText(LOCTEXT("ViewportTabToolTip", "Contains the viewport where the recorded physics scene will be rendered"));
	
	if (TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin())
	{
		ViewportTab->SetContent
		(
			SAssignNew(PlaybackViewportWidget, SChaosVDPlaybackViewport, GetChaosVDScene(), MainTabPtr->GetChaosVDEngineInstance()->GetPlaybackController(), MainTabPtr->GetEditorModeManager().AsShared())
		);
	}
	else
	{
		ViewportTab->SetContent(GenerateErrorWidget());
	}
	
	ViewportTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("ChaosVisualDebugger"));
	
	HandleTabSpawned(ViewportTab);

	return ViewportTab;
}

void FChaosVDPlaybackViewportTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	FChaosVDTabSpawnerBase::HandleTabClosed(InTabClosed);

	PlaybackViewportWidget.Reset();
}

#undef LOCTEXT_NAMESPACE
