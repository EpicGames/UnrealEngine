// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "ChaosVDTabSpawnerBase.h"

class FChaosVDScene;
class SChaosVDPlaybackViewport;

/** Spawns and handles and instance for the visual debugger Playback viewport tab
 * It contains the Playback controls and the 3D viewport
 */
class FChaosVDPlaybackViewportTab : public FChaosVDTabSpawnerBase, public TSharedFromThis<FChaosVDPlaybackViewportTab>
{
public:
	FChaosVDPlaybackViewportTab(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget) : FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{	
	}
	
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;

	virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

	TWeakPtr<SChaosVDPlaybackViewport> GetPlaybackViewportWidget() { return PlaybackViewportWidget; }

protected:
	TSharedPtr<SChaosVDPlaybackViewport> PlaybackViewportWidget;
};
