// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDRecordedLogTab.h"

#include "ChaosLog.h"
#include "ChaosVDModule.h"
#include "ChaosVDStyle.h"
#include "Framework/Docking/TabManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Widgets/SChaosVDRecordedLogInstances.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

TSharedRef<SDockTab> FChaosVDRecordedLogTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> RecordedLogTab =
		SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("RecordedOutputLogTabLabel", "Recorded Output Log"));
	
	if (const TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin())
	{
		RecordedLogTab->SetContent
		(
			SNew(SChaosVDRecordedLogInstances, RecordedLogTab, MainTabPtr->GetChaosVDEngineInstance())
		);
	}
	else
	{
		RecordedLogTab->SetContent
		(
			GenerateErrorWidget()
		);
	}

	RecordedLogTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("TabIconOutputLog"));

	HandleTabSpawned(RecordedLogTab);

	return RecordedLogTab;
}

#undef LOCTEXT_NAMESPACE

