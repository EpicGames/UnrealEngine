// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "ChaosVDTabSpawnerBase.h"

class SOutputLog;
class FOutputLogHistory;
class FName;

/** Spawns and handles and instance for the visual debugger Recorded Output Panel */
class FChaosVDRecordedLogTab : public FChaosVDTabSpawnerBase, public TSharedFromThis<FChaosVDRecordedLogTab>
{
public:

	FChaosVDRecordedLogTab(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget) : FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{
	}

	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
};

