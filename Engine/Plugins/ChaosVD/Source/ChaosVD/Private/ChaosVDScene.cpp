// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDScene.h"

#include "Actors/ChaosVDSolverInfoActor.h"
#include "ChaosVDGeometryBuilder.h"
#include "ChaosVDModule.h"
#include "ChaosVDSceneParticle.h"
#include "Chaos/ImplicitObject.h"
#include "ChaosVDRecording.h"
#include "ChaosVDSceneCompositionReport.h"
#include "ChaosVDSelectionCustomization.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDSkySphereInterface.h"
#include "EngineUtils.h"
#include "Components/ChaosVDSceneQueryDataComponent.h"
#include "Components/ChaosVDSolverCollisionDataComponent.h"
#include "Engine/DirectionalLight.h"
#include "Engine/Engine.h"
#include "Engine/Level.h"
#include "Engine/StreamableManager.h"
#include "Engine/World.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Materials/Material.h"
#include "Misc/ScopedSlowTask.h"
#include "Selection.h"
#include "Actors/ChaosVDGameFrameInfoActor.h"
#include "Settings/ChaosVDCoreSettings.h"
#include "UObject/Package.h"
#include "Actors/ChaosVDGeometryContainer.h"
#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Engine/PostProcessVolume.h"
#include "Engine/TextureCube.h"
#include "TEDS/ChaosVDParticleEditorDataFactory.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "TEDS/ChaosVDSelectionInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDScene)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDScene::FChaosVDScene() = default;

FChaosVDScene::~FChaosVDScene() = default;

namespace Chaos::VisualDebugger::Cvars
{
	static bool bReInitializeGeometryBuilderOnCleanup = true;
	static FAutoConsoleVariableRef CVarChaosVDReInitializeGeometryBuilderOnCleanup(
		TEXT("p.Chaos.VD.Tool.ReInitializeGeometryBuilderOnCleanup"),
		bReInitializeGeometryBuilderOnCleanup,
		TEXT("If true, any static mesh component and static mesh component created will be destroyed when a new CVD recording is loaded"));
}

void FChaosVDScene::Initialize()
{
	if (!ensure(!bIsInitialized))
	{
		return;
	}

	InitializeSelectionSets();
	
	StreamableManager = MakeShared<FStreamableManager>();

	if (UChaosVDCoreSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCoreSettings>())
	{
		// TODO: Do an async load instead, and prepare a loading screen or notification popup
		// Jira for tracking UE-191639
		StreamableManager->RequestSyncLoad(Settings->QueryOnlyMeshesMaterial.ToSoftObjectPath());
		StreamableManager->RequestSyncLoad(Settings->SimOnlyMeshesMaterial.ToSoftObjectPath());
		StreamableManager->RequestSyncLoad(Settings->InstancedMeshesMaterial.ToSoftObjectPath());
		StreamableManager->RequestSyncLoad(Settings->InstancedMeshesQueryOnlyMaterial.ToSoftObjectPath());
		StreamableManager->RequestSyncLoad(Settings->AmbientCubeMapTexture.ToSoftObjectPath());
		StreamableManager->RequestSyncLoad(Settings->BoxMesh.ToSoftObjectPath());
		StreamableManager->RequestSyncLoad(Settings->SphereMesh.ToSoftObjectPath());
	}
	
	PhysicsVDWorld = CreatePhysicsVDWorld();

	GeometryGenerator = MakeShared<FChaosVDGeometryBuilder>();

	GeometryGenerator->Initialize(AsWeak());

	bIsInitialized = true;
}

void FChaosVDScene::PerformGarbageCollection()
{
	FScopedSlowTask CollectingGarbageSlowTask(1, LOCTEXT("CollectingGarbageDataMessage", "Collecting Garbage ..."));
	CollectingGarbageSlowTask.MakeDialog();

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);

	CollectingGarbageSlowTask.EnterProgressFrame();
}

void FChaosVDScene::DeInitialize()
{
	constexpr float AmountOfWork = 1.0f;
	FScopedSlowTask ClosingSceneSlowTask(AmountOfWork, LOCTEXT("ClosingSceneMessage", "Closing Scene ..."));
	ClosingSceneSlowTask.MakeDialog();

	if (!ensure(bIsInitialized))
	{
		return;
	}

	CleanUpScene();

	DeInitializeSelectionSets();

	GeometryGenerator.Reset();

	if (PhysicsVDWorld)
	{
		PhysicsVDWorld->RemoveOnActorDestroyedHandler(ActorDestroyedHandle);

		PhysicsVDWorld->DestroyWorld(true);
		GEngine->DestroyWorldContext(PhysicsVDWorld);

		PhysicsVDWorld->MarkAsGarbage();
		PhysicsVDWorld = nullptr;
	}

	PerformGarbageCollection();

	bIsInitialized = false;
}

void FChaosVDScene::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(PhysicsVDWorld);
	Collector.AddReferencedObject(SelectionSet);
	Collector.AddReferencedObject(ObjectSelection);
	Collector.AddReferencedObject(ActorSelection);
	Collector.AddReferencedObject(ComponentSelection);
	Collector.AddStableReferenceArray(&AvailableDataContainerActors);
}

void FChaosVDScene::UpdateFromRecordedSolverStageData(const int32 SolverID, const FChaosVDFrameStageData& InRecordedStepData, const FChaosVDSolverFrameData& InFrameData)
{
	AChaosVDSolverInfoActor* SolverSceneData = nullptr;
	if (AChaosVDSolverInfoActor** SolverSceneDataPtrPtr = SolverDataContainerBySolverID.Find(SolverID))
	{
		SolverSceneData = *SolverSceneDataPtrPtr;
	}
	else
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Attempted to playback a solver frame from an invalid solver container", ANSI_TO_TCHAR(__FUNCTION__));	
	}

	if (!SolverSceneData)
	{
		return;
	}

	SolverSceneData->SetSimulationTransform(InFrameData.SimulationTransform);

	SolverSceneData->UpdateFromNewSolverStageData(InFrameData, InRecordedStepData);

	bPendingUpdateRequest = true;
}

void FChaosVDScene::AddFromCVDWorldTagToActor(AActor* Actor)
{
	using namespace UE::Editor::DataStorage;
	// Add a selection column in TEDS
	if (const ICompatibilityProvider* Compatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
	{
		RowHandle Row = Compatibility->FindRowWithCompatibleObject(Actor);
		if (Row != InvalidRowHandle)
		{
			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				DataStorage->AddColumn<FTypedElementFromCVDWorldTag>(Row);
				DataStorage->AddColumn<FChaosVDActiveObjectTag>(Row);
				DataStorage->AddColumn<FTypedElementSyncFromWorldTag>(Row);
			}
		}
	}
}

void FChaosVDScene::SetLoadedRecording(const TSharedPtr<FChaosVDRecording>& NewRecordingInstance)
{
	if (LoadedRecording != NewRecordingInstance)
	{
		if (LoadedRecording)
		{
			FWriteScopeLock RecordingWriteLock(LoadedRecording->GetRecordingDataLock());
			LoadedRecording->OnGeometryDataLoaded().RemoveAll(GeometryGenerator.Get());
		}

		LoadedRecording = NewRecordingInstance;

		if (GeometryGenerator)
		{
			FWriteScopeLock RecordingWriteLock(LoadedRecording->GetRecordingDataLock());
			LoadedRecording->OnGeometryDataLoaded().AddSP(GeometryGenerator.ToSharedRef(), &FChaosVDGeometryBuilder::HandleNewGeometryData);
		}
	}
}

AActor* FChaosVDScene::GetMeshComponentsContainerActor() const
{
	return MeshComponentContainerActor;
}

AChaosVDSolverInfoActor* FChaosVDScene::GetOrCreateSolverInfoActor(int32 SolverID)
{
	if (AChaosVDSolverInfoActor** SolverInfoActorPtrPtr = SolverDataContainerBySolverID.Find(SolverID))
	{
		return *SolverInfoActorPtrPtr;
	}
	
	AChaosVDSolverInfoActor* SolverDataInfo = PhysicsVDWorld->SpawnActor<AChaosVDSolverInfoActor>();
	check(SolverDataInfo);

	FName SolverName = LoadedRecording->GetSolverFName_AssumedLocked(SolverID);
	FString NameAsString = SolverName.ToString();
	const bool bIsServer = NameAsString.Contains(TEXT("Server"));

	const FStringFormatOrderedArguments Args {NameAsString, FString::FromInt(SolverID)};
	const FName FolderPath = *FString::Format(TEXT("Solver {0} | ID {1}"), Args);

	SolverDataInfo->SetFolderPath(FolderPath);
	SolverDataInfo->SetSolverName(SolverName);
	SolverDataInfo->SetIsServer(bIsServer);
	SolverDataInfo->SetSolverID(SolverID);
	SolverDataInfo->SetScene(AsWeak());

	SolverDataContainerBySolverID.Add(SolverID, SolverDataInfo);
	AvailableDataContainerActors.Add(SolverDataInfo);

	AddFromCVDWorldTagToActor(SolverDataInfo);

	SolverInfoActorCreatedDelegate.Broadcast(SolverDataInfo);

	return SolverDataInfo;
}

AChaosVDGameFrameInfoActor* FChaosVDScene::GetOrCreateGameFrameInfoActor()
{
	if (!GameFrameDataInfoActor)
	{
		const FName FolderPath("ChaosVisualDebugger/GameFrameData");

		GameFrameDataInfoActor = PhysicsVDWorld->SpawnActor<AChaosVDGameFrameInfoActor>();
		GameFrameDataInfoActor->SetFolderPath(FolderPath);
		GameFrameDataInfoActor->SetScene(AsWeak());
		AvailableDataContainerActors.Add(GameFrameDataInfoActor);
	}

	return GameFrameDataInfoActor;
}

void FChaosVDScene::HandleEnterNewGameFrame(int32 FrameNumber, const TArray<int32, TInlineAllocator<16>>& AvailableSolversIds, const FChaosVDGameFrameData& InNewGameFrameData, TArray<int32, TInlineAllocator<16>>& OutRemovedSolversIds)
{
	// Currently the particle actors from all the solvers are in the same level, and we manage them by keeping track
	// of to which solvers they belong using maps.
	// Using Level instead or a Sub ChaosVDScene could be a better solution
	// I'm intentionally not making that change right now until the "level streaming" solution for the tool is defined
	// As that would impose restriction on how levels could be used. For now the map approach is simpler and will be easier to refactor later on.

	TSet<int32> AvailableSolversSet;
	AvailableSolversSet.Reserve(AvailableSolversIds.Num());

	for (int32 SolverID : AvailableSolversIds)
	{
		AvailableSolversSet.Add(SolverID);

		AChaosVDSolverInfoActor* SolverInfoActor = GetOrCreateSolverInfoActor(SolverID);
		if (!ensure(SolverInfoActor))
		{
			UE_LOGF(LogChaosVDEditor, Error, "[%s] Failed to create solver data actor for id [%d]", __func__, SolverID);
		}
	}

	int32 AmountRemoved = 0;

	for (TMap<int32, AChaosVDSolverInfoActor*>::TIterator RemoveIterator = SolverDataContainerBySolverID.CreateIterator(); RemoveIterator; ++RemoveIterator)
	{
		if (!AvailableSolversSet.Contains(RemoveIterator.Key()))
		{
			UE_LOGF(LogChaosVDEditor, Log, "[%ls] Removing Solver [%d] as it is no longer present in the recording", ANSI_TO_TCHAR(__FUNCTION__), RemoveIterator.Key());

			if (AChaosVDSolverInfoActor* SolverInfoActor = RemoveIterator.Value())
			{
				AvailableDataContainerActors.Remove(SolverInfoActor);
				PhysicsVDWorld->DestroyActor(SolverInfoActor);
			}

			OutRemovedSolversIds.Add(RemoveIterator.Key());

			RemoveIterator.RemoveCurrent();
			AmountRemoved++;
		}
	}

	if (AmountRemoved > 0)
	{
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS);
	}

	if (AChaosVDGameFrameInfoActor* GameFrameDataContainer = GetOrCreateGameFrameInfoActor())
	{
		GameFrameDataContainer->UpdateFromNewGameFrameData(InNewGameFrameData);
	}

	bPendingUpdateRequest = true;
}

void FChaosVDScene::HandleEnterNewSolverFrame(int32 FrameNumber, const FChaosVDSolverFrameData& InFrameData)
{
	if (AChaosVDSolverInfoActor** SolverDataInfoContainerPtrPtr = SolverDataContainerBySolverID.Find(InFrameData.SolverID))
	{
		AChaosVDSolverInfoActor* SolverDataInfoContainerPtr = *SolverDataInfoContainerPtrPtr;
		if (SolverDataInfoContainerPtrPtr)
		{
			SolverDataInfoContainerPtr->UpdateFromNewSolverFrameData(InFrameData);
		}
	}

	bPendingUpdateRequest = true;
}

void FChaosVDScene::CleanUpScene(EChaosVDSceneCleanUpOptions Options)
{
	// AvailableDataContainerActors should always be at least the number of solver actors created
	ensure(AvailableDataContainerActors.Num() >= SolverDataContainerBySolverID.Num());

	if (AvailableDataContainerActors.Num() > 0)
	{
		constexpr float AmountOfWork = 1.0f;
		const float PercentagePerElement = 1.0f / static_cast<float>(AvailableDataContainerActors.Num());

		FScopedSlowTask CleaningSceneSlowTask(AmountOfWork, LOCTEXT("CleaningupSceneSolverMessage", "Clearing Solver Data ..."));
		CleaningSceneSlowTask.MakeDialog();

		ClearSelectionAndNotify();

		if (PhysicsVDWorld)
		{
			for (TObjectPtr<AChaosVDDataContainerBaseActor>& DataContainerActor : AvailableDataContainerActors)
			{
				if (DataContainerActor)
				{
					PhysicsVDWorld->DestroyActor(DataContainerActor.Get());
				}

				CleaningSceneSlowTask.EnterProgressFrame(PercentagePerElement);
			}
		}

		AvailableDataContainerActors.Reset();
		SolverDataContainerBySolverID.Reset();
		GameFrameDataInfoActor = nullptr;
	}

	if (Chaos::VisualDebugger::Cvars:: bReInitializeGeometryBuilderOnCleanup && EnumHasAnyFlags(Options, EChaosVDSceneCleanUpOptions::ReInitializeGeometryBuilder))
	{
		if (AChaosVDGeometryContainer* AsGeometryContainer = Cast<AChaosVDGeometryContainer>(MeshComponentContainerActor))
		{
			AsGeometryContainer->CleanUp();
		}

		GeometryGenerator->DeInitialize();
		GeometryGenerator.Reset();
	
		GeometryGenerator = MakeShared<FChaosVDGeometryBuilder>();
		GeometryGenerator->Initialize(AsWeak());
	}

	if (EnumHasAnyFlags(Options, EChaosVDSceneCleanUpOptions::CollectGarbage))
	{
		PerformGarbageCollection();
	}

	Chaos::VD::TypedElementDataUtil::CleanUpTypedElementStore();
}

Chaos::FConstImplicitObjectPtr FChaosVDScene::GetUpdatedGeometry(int32 GeometryID) const
{
	if (ensure(LoadedRecording.IsValid()))
	{
		if (const Chaos::FConstImplicitObjectPtr* Geometry = LoadedRecording->GetGeometryMap().Find(GeometryID))
		{
			return *Geometry;
		}
		else
		{
			UE_LOGF(LogChaosVDEditor, Warning, "Geometry for key [%d] is not loaded in the recording yet", GeometryID);
		}
	}

	return nullptr;
}

const Chaos::FImplicitObject* FChaosVDScene::UnpackGeometry(const Chaos::FImplicitObject* InImplicitObject, Chaos::FRigidTransform3& InOutTransform) const
{
	return GeometryGenerator->UnpackImplicitObject(InImplicitObject, InOutTransform);
}

bool FChaosVDScene::DoesGeometryNeedUnpacking(const Chaos::FImplicitObject* InImplicitObject) const
{
	return GeometryGenerator->ImplicitObjectNeedsUnpacking(InImplicitObject);
}

TSharedPtr<FChaosVDSceneParticle> FChaosVDScene::GetParticleInstance(int32 SolverID, int32 ParticleID)
{
	if (AChaosVDSolverInfoActor** SolverDataInfo = SolverDataContainerBySolverID.Find(SolverID))
	{
		return (*SolverDataInfo)->GetParticleInstance(ParticleID);
	}

	return nullptr;
}

AChaosVDSolverInfoActor* FChaosVDScene::GetSolverInfoActor(int32 SolverID)
{
	if (AChaosVDSolverInfoActor** SolverDataInfo = SolverDataContainerBySolverID.Find(SolverID))
	{
		return *SolverDataInfo;
	}

	return nullptr;
}

void FChaosVDScene::CreateBaseLights(UWorld* TargetWorld)
{
	if (!TargetWorld)
	{
		return;
	}

	const FName LightingFolderPath("ChaosVisualDebugger/Lighting");

	const FVector SpawnPosition(0.0, 0.0, 2000.0);
	
	if (const UChaosVDCoreSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCoreSettings>())
	{
		if (ADirectionalLight* DirectionalLightActor = TargetWorld->SpawnActor<ADirectionalLight>())
		{
			DirectionalLightActor->SetCastShadows(false);
			DirectionalLightActor->SetMobility(EComponentMobility::Movable);
			DirectionalLightActor->SetActorLocation(SpawnPosition);
			
			DirectionalLightActor->SetBrightness(4.0f);

			DirectionalLightActor->SetFolderPath(LightingFolderPath);

			TSubclassOf<AActor> SkySphereClass = Settings->SkySphereActorClass.TryLoadClass<AActor>();
			SkySphere = TargetWorld->SpawnActor(SkySphereClass.Get());
			if (SkySphere)
			{
				SkySphere->SetActorLocation(SpawnPosition);
				SkySphere->SetFolderPath(LightingFolderPath);
				
				if (SkySphere->Implements<UChaosVDSkySphereInterface>())
				{
					FEditorScriptExecutionGuard AllowEditorScriptGuard;
					IChaosVDSkySphereInterface::Execute_SetDirectionalLightSource(SkySphere, DirectionalLightActor);
				}

				// Keep it dark to reduce visual noise.
				// TODO: We should hide these components altogether when we switch to a unlit wireframe mode 
				const TSet<UActorComponent*>& Components = SkySphere->GetComponents();
				for (UActorComponent* Component : Components)
				{
					if (UStaticMeshComponent* AsStaticMeshComponent = Cast<UStaticMeshComponent>(Component))
					{
						AsStaticMeshComponent->bOverrideWireframeColor = true;
						AsStaticMeshComponent->WireframeColorOverride = FColor::Black;
					}
				}

				AddFromCVDWorldTagToActor(SkySphere);
				AddFromCVDWorldTagToActor(DirectionalLightActor);
			}
		}
	}
}

void FChaosVDScene::CreatePostProcessingVolumes(UWorld* TargetWorld)
{
	const FName LightingFolderPath("ChaosVisualDebugger/Lighting");

	if (const UChaosVDCoreSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCoreSettings>())
	{
		APostProcessVolume* PostProcessingVolume = TargetWorld->SpawnActor<APostProcessVolume>();
		if (ensure(PostProcessingVolume))
		{
			PostProcessingVolume->SetFolderPath(LightingFolderPath);
			PostProcessingVolume->Settings.bOverride_AmbientCubemapIntensity = true;
			PostProcessingVolume->Settings.AmbientCubemapIntensity = 0.3f;
			PostProcessingVolume->bUnbound = true;
			PostProcessingVolume->bEnabled = true;

			UTextureCube* AmbientCubemap = Settings->AmbientCubeMapTexture.Get();
			if (ensure(AmbientCubemap))
			{
				PostProcessingVolume->Settings.AmbientCubemap = AmbientCubemap;
			}
			
			PostProcessingVolume->MarkComponentsRenderStateDirty();

			AddFromCVDWorldTagToActor(PostProcessingVolume);
		}
	}
}

AActor* FChaosVDScene::CreateMeshComponentsContainer(UWorld* TargetWorld)
{
	const FName GeometryFolderPath("ChaosVisualDebugger/GeneratedMeshComponents");

	MeshComponentContainerActor = TargetWorld->SpawnActor<AChaosVDGeometryContainer>();
	MeshComponentContainerActor->SetFolderPath(GeometryFolderPath);
	MeshComponentContainerActor->SetScene(AsWeak());

	return MeshComponentContainerActor;
}

UWorld* FChaosVDScene::CreatePhysicsVDWorld()
{
	const FName UniqueWorldName = FName(FGuid::NewGuid().ToString());
	UWorld* NewWorld = NewObject<UWorld>( GetTransientPackage(), UniqueWorldName );
	
	NewWorld->WorldType = EWorldType::EditorPreview;

	FWorldContext& WorldContext = GEngine->CreateNewWorldContext( NewWorld->WorldType );
	WorldContext.SetCurrentWorld(NewWorld);

	NewWorld->InitializeNewWorld( UWorld::InitializationValues()
										  .AllowAudioPlayback( false )
										  .CreatePhysicsScene( false )
										  .RequiresHitProxies( true )
										  .CreateNavigation( false )
										  .CreateAISystem( false )
										  .ShouldSimulatePhysics( false )
										  .SetTransactional( false )
	);

	if (ULevel* Level = NewWorld->GetCurrentLevel())
	{
		Level->SetUseActorFolders(true);
	}

	CreateBaseLights(NewWorld);
	CreateMeshComponentsContainer(NewWorld);
	CreatePostProcessingVolumes(NewWorld);

	ActorDestroyedHandle = NewWorld->AddOnActorDestroyedHandler(FOnActorDestroyed::FDelegate::CreateRaw(this, &FChaosVDScene::HandleActorDestroyed));
	
	return NewWorld;
}

TArray<FTypedElementHandle> FChaosVDScene::GetSelectedElementHandles()
{
	return GetElementSelectionSet()->GetSelectedElementHandles(UChaosVDSelectionInterface::StaticClass());
}

FTypedElementHandle FChaosVDScene::GetSelectionHandleForObject(const UObject* Object) const
{
	FTypedElementHandle Handle;
	if (const AActor* Actor = Cast<AActor>(Object))
	{
		Handle = UEngineElementsLibrary::AcquireEditorActorElementHandle(Actor);
	}
	else if (const UActorComponent* Component = Cast<UActorComponent>(Object))
	{
		Handle = UEngineElementsLibrary::AcquireEditorComponentElementHandle(Component);
	}
	else
	{
		Handle = UEngineElementsLibrary::AcquireEditorObjectElementHandle(Object);
	}

	return Handle;
}

void FChaosVDScene::UpdateSelectionProxiesForActors(TArrayView<AActor*> SelectedActors)
{
	for (AActor* SelectedActor : SelectedActors)
	{
		if (SelectedActor)
		{
			SelectedActor->PushSelectionToProxies();
		}
	}
}

FName FChaosVDScene::GetTEDSSelectionSetName()
{
	return TEDSSelectionSetName;
}

void FChaosVDScene::HandleDeSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	using namespace Chaos::VD::TypedElementDataUtil;
	if (FChaosVDSceneParticle* Particle = GetStructDataFromTypedElementHandle<FChaosVDSceneParticle>(InElementSelectionHandle))
	{
		if (Particle)
		{
			Particle->HandleDeSelected();
		}
	}
}

void FChaosVDScene::HandleSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	using namespace Chaos::VD::TypedElementDataUtil;
	if (FChaosVDSceneParticle* Particle = GetStructDataFromTypedElementHandle<FChaosVDSceneParticle>(InElementSelectionHandle))
	{
		if (Particle)
		{
			Particle->HandleSelected();
		}
	}
}

void FChaosVDScene::ClearSelectionAndNotify()
{
	if (!SelectionSet)
	{
		return;
	}

	SelectionSet->ClearSelection(FTypedElementSelectionOptions());
	SelectionSet->NotifyPendingChanges();
}

void FChaosVDScene::RequestUpdate()
{
	bPendingUpdateRequest = true;
}

bool FChaosVDScene::Tick(float DeltaTime)
{
	if (bPendingUpdateRequest)
	{
		OnSceneUpdated().Broadcast();
        bPendingUpdateRequest = false;
	}

	return true;
}

void FChaosVDScene::UpdateWorldStreamingLocation(const FVector& InLocation)
{
	WorldStreamingLocation = InLocation;
	for (TObjectPtr<AChaosVDDataContainerBaseActor>& DataContainerActor : AvailableDataContainerActors)
	{
		if (DataContainerActor)
		{
			DataContainerActor->HandleWorldStreamingLocationUpdated(WorldStreamingLocation);
		}
	}
}

void FChaosVDScene::AppendSceneCompositionTestData(FChaosVDPlaybackEngineSnapshot& OutStateTestData)
{
	for (TObjectPtr<AChaosVDDataContainerBaseActor>& DataContainerActor : AvailableDataContainerActors)
	{
		if (DataContainerActor)
		{
			DataContainerActor->AppendSceneCompositionTestData(OutStateTestData.SceneComposition);
		}
	}
}

void FChaosVDScene::InitializeSelectionSets()
{
	SelectionSet = NewObject<UTypedElementSelectionSet>(GetTransientPackage(), NAME_None, RF_Transactional);

	TEDSSelectionSetName = FName(FString::Printf(TEXT("CVDSelectionSet%p"), SelectionSet.Get()));
	SelectionSet->SetNameForTedsIntegration(TEDSSelectionSetName);

	SelectionSet->AddToRoot();

	using namespace Chaos::VD::TypedElementDataUtil;

	SelectionSet->RegisterInterfaceCustomizationByTypeName(NAME_Actor, MakeUnique<FChaosVDSelectionCustomization>(AsShared()));
	SelectionSet->RegisterInterfaceCustomizationByTypeName(NAME_CVD_StructDataElement, MakeUnique<FChaosVDSelectionCustomization>(AsShared()));
	SelectionSet->RegisterInterfaceCustomizationByTypeName(NAME_Components, MakeUnique<FChaosVDSelectionCustomization>(AsShared()));
	SelectionSet->RegisterInterfaceCustomizationByTypeName(NAME_Object, MakeUnique<FChaosVDSelectionCustomization>(AsShared()));

	FString ActorSelectionObjectName = FString::Printf(TEXT("CVDSelectedActors-%s"), *FGuid::NewGuid().ToString());
	ActorSelection = USelection::CreateActorSelection(GetTransientPackage(), *ActorSelectionObjectName, RF_Transactional);
	ActorSelection->SetElementSelectionSet(SelectionSet);

	FString ComponentSelectionObjectName = FString::Printf(TEXT("CVDSelectedComponents-%s"), *FGuid::NewGuid().ToString());
	ComponentSelection = USelection::CreateComponentSelection(GetTransientPackage(), *ComponentSelectionObjectName, RF_Transactional);
	ComponentSelection->SetElementSelectionSet(SelectionSet);

	FString ObjectSelectionObjectName = FString::Printf(TEXT("CVDSelectedObjects-%s"), *FGuid::NewGuid().ToString());
	ObjectSelection = USelection::CreateObjectSelection(GetTransientPackage(), *ObjectSelectionObjectName, RF_Transactional);
	ObjectSelection->SetElementSelectionSet(SelectionSet);

	SolverDataSelectionObject = MakeShared<FChaosVDSolverDataSelection>();
}

void FChaosVDScene::DeInitializeSelectionSets()
{
	ActorSelection->SetElementSelectionSet(nullptr);
	ComponentSelection->SetElementSelectionSet(nullptr);
	ObjectSelection->SetElementSelectionSet(nullptr);

	SelectionSet->OnPreChange().RemoveAll(this);
	SelectionSet->OnChanged().RemoveAll(this);
}

void FChaosVDScene::HandleActorDestroyed(AActor* ActorDestroyed)
{
	if (IsObjectSelected(ActorDestroyed))
	{
		ClearSelectionAndNotify();
	}
}

void FChaosVDScene::SetSelectedObject(UObject* SelectedObject)
{
	if (!SelectionSet)
	{
		return;
	}

	if (!::IsValid(SelectedObject))
	{
		ClearSelectionAndNotify();
		return;
	}

	if (IsObjectSelected(SelectedObject))
	{
		// Already selected, nothing to do here
		return;
	}

	SelectionSet->ClearSelection(FTypedElementSelectionOptions());

	TArray<FTypedElementHandle> NewEditorSelection = { GetSelectionHandleForObject(SelectedObject) };

	SelectionSet->SetSelection(NewEditorSelection, FTypedElementSelectionOptions());
	SelectionSet->NotifyPendingChanges();
}

void FChaosVDScene::SetSelected(const FTypedElementHandle& InElementHandle)
{
	if (!SelectionSet)
	{
		return;
	}

	if (SelectionSet->IsElementSelected(InElementHandle, FTypedElementIsSelectedOptions()))
	{
		// Already selected, nothing to do here
		return;
	}

	SelectionSet->ClearSelection(FTypedElementSelectionOptions());

	TArray<FTypedElementHandle> NewEditorSelection = { InElementHandle };

	SelectionSet->SetSelection(NewEditorSelection, FTypedElementSelectionOptions());
	SelectionSet->NotifyPendingChanges();
}

bool FChaosVDScene::IsObjectSelected(const UObject* Object)
{
	if (!SelectionSet)
	{
		return false;
	}

	if (!::IsValid(Object))
	{
		return false;
	}

	return SelectionSet->IsElementSelected(GetSelectionHandleForObject(Object), FTypedElementIsSelectedOptions());
}

bool FChaosVDScene::IsSelected(const FTypedElementHandle& InElementHandle) const
{
	return SelectionSet->IsElementSelected(InElementHandle, FTypedElementIsSelectedOptions());
}

#undef LOCTEXT_NAMESPACE
