// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSceneParticle.h"

#include "ChaosVDGeometryBuilder.h"
#include "ChaosVDModule.h"
#include "ChaosVDScene.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Components/ChaosVDInstancedStaticMeshComponent.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "Components/ChaosVDSolverCharacterGroundConstraintDataComponent.h"
#include "Components/ChaosVDSolverCollisionDataComponent.h"
#include "Components/ChaosVDStaticMeshComponent.h"
#include "Components/MeshComponent.h"
#include "Components/SceneComponent.h"
#include "Components/StaticMeshComponent.h"
#include "DataStorage/Features.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Engine/StaticMesh.h"
#include "TEDS/ChaosVDParticleEditorDataFactory.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "TEDS/ChaosVDTedsUtils.h"
#include "Visualizers/ChaosVDSolverCollisionDataComponentVisualizer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSceneParticle)

namespace Chaos::VisualDebugger::Cvars
{
	static bool bForceStaticMeshComponentUse = false;
	static FAutoConsoleVariableRef CVarChaosVDForceStaticMeshComponentUse(
		TEXT("p.Chaos.VD.Tool.ForceStaticMeshComponentUse"),
		bForceStaticMeshComponentUse,
		TEXT("If true, static mesh components will be used instead of Instanced Static mesh components when recreating the geometry for each particle"));

	static bool bUseInstancedStaticMeshForLandscape = true;
	static FAutoConsoleVariableRef CVarChaosVDbUseInstancedStaticMeshForLandscape(
		TEXT("p.Chaos.VD.Tool.UseInstancedStaticMeshForLandscape"),
		bForceStaticMeshComponentUse,
		TEXT("If true, instanced static mesh components will be used instead of static mesh components when recreating the geometry for particles from Landscapes"));
}

FChaosVDSceneParticle::~FChaosVDSceneParticle()
{
	VisitGeometryInstances([](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
	{
		if (IChaosVDGeometryComponent* AsGeometryComponent = Cast<IChaosVDGeometryComponent>(MeshDataHandle->GetMeshComponent()))
		{
			AsGeometryComponent->RemoveMeshInstance(MeshDataHandle);
		}
	});

	MeshDataHandles.Empty();

	SetIsActive(false);

	ParticleDestroyedDelegate.ExecuteIfBound();
}

FChaosVDSceneParticle::FChaosVDSceneParticle()
{
	static FString UnnamedParticle("UnnamedParticle");
	SetDisplayName(UnnamedParticle);

	static FName DefaultIconName("RigidBodyIcon");
	SetIconName(DefaultIconName);
}

void FChaosVDSceneParticle::RemoveAllGeometry()
{
	VisitGeometryInstances([](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
	{
		if (IChaosVDGeometryComponent* AsGeometryComponent = Cast<IChaosVDGeometryComponent>(MeshDataHandle->GetMeshComponent()))
		{
			AsGeometryComponent->RemoveMeshInstance(MeshDataHandle);
		}
	});

	MeshDataHandles.Reset();

	EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Geometry);
}

FBox FChaosVDSceneParticle::GetStreamingBounds() const
{
	if (!CachedBounds.IsValid)
	{
		// The only valid case to not have bounds, is if we don't have geometry
		ensure(!CurrentRootGeometry);
	}

	return CachedBounds;
}

void FChaosVDSceneParticle::SyncStreamingState()
{
	if (StreamingState == EStreamingState::Visible)
	{
		if (MeshDataHandles.IsEmpty())
		{
			EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::PreUpdatePass);
			EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Geometry);
			EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Coloring);
			EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility);
			EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Transform);
			EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::CollisionData);

			ProcessPendingParticleDataUpdates();
		}
	}
	else if (!MeshDataHandles.IsEmpty())
	{
		RemoveAllGeometry();
	}
}

int32 FChaosVDSceneParticle::GetStreamingID() const
{
	if (ensure(ParticleDataPtr))
	{
		return ParticleDataPtr->ParticleIndex;
	}

	return INDEX_NONE;
}

void FChaosVDSceneParticle::UpdateParent(const TSharedPtr<FChaosVDParticleDataWrapper>& InRecordedData)
{
	if (TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		TSharedPtr<FChaosVDSceneParticle> ParentClusterParticle = InRecordedData->ParticleCluster.HasValidData() ?
																	ScenePtr->GetParticleInstance(InRecordedData->SolverID, InRecordedData->ParticleCluster.ParentParticleID)
																	: nullptr;
		if (ParentClusterParticle)
		{
			SetParent(ParentClusterParticle);
		}
		else
		{
			AChaosVDSolverInfoActor* SolverData = ScenePtr->GetSolverInfoActor(InRecordedData->SolverID);
			if (UChaosVDParticleDataComponent* ParticleDataComponent = SolverData ? SolverData->GetParticleDataComponent() : nullptr)
			{
				SetParent(ParticleDataComponent->GetParticleContainerByType(InRecordedData->Type));
			}
			else
			{
				SetParent(nullptr);
			}
		}
	}

	EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::TEDS);

	EnumRemoveFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Parent);
}

void FChaosVDSceneParticle::PreUpdateFromRecordedParticleData(const TSharedPtr<FChaosVDParticleDataWrapper>& InRecordedData, const Chaos::FRigidTransform3& SimulationTransform)
{
	if (!ensure(InRecordedData.IsValid()))
	{
		return;	
	}

	if (!ParticleDataPtr
		|| ParticleDataPtr->ParticleCluster.HasValidData() != InRecordedData->ParticleCluster.HasValidData()
		|| ParticleDataPtr->ParticleCluster.ParentParticleID != InRecordedData->ParticleCluster.ParentParticleID
		|| ParticleDataPtr->Type != InRecordedData->Type)
	{
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Parent);
	}

	//TODO: Make the simulation transform be cached on the CVD Scene, so we can query from it when needed
	// Copying it to each particle actor is not efficient
	CachedSimulationTransform = SimulationTransform;

	if (InRecordedData->ParticlePositionRotation.HasValidData())
	{
		const FVector TargetLocation = SimulationTransform.TransformPosition(InRecordedData->ParticlePositionRotation.MX);
		const FVector CurrentLocation = SimulationTransform.TransformPosition(ParticleDataPtr && ParticleDataPtr->ParticlePositionRotation.HasValidData() ? ParticleDataPtr->ParticlePositionRotation.MX : FVector::ZeroVector);

		const FQuat TargetRotation = SimulationTransform.GetRotation() * InRecordedData->ParticlePositionRotation.MR;
		const FQuat CurrentRotation = SimulationTransform.GetRotation() * (ParticleDataPtr && ParticleDataPtr->ParticlePositionRotation.HasValidData() ? ParticleDataPtr->ParticlePositionRotation.MR : FQuat::Identity);

		PendingParticleTransform.SetLocation(TargetLocation);
		PendingParticleTransform.SetRotation(TargetRotation);
		PendingParticleTransform.SetScale3D(FVector(1.0f,1.0f,1.0f));

		if (CurrentRotation != TargetRotation || CurrentLocation != TargetLocation)
		{
			EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Transform);
		}
	}

	// This is iterating and comparing each element of the array,
	// We might need to find a faster way of determine if the data changed, but for now this is faster than assuming it changed
	const bool bShapeDataIsDirty = !ParticleDataPtr || (ParticleDataPtr->CollisionDataPerShape != InRecordedData->CollisionDataPerShape);
	const bool bDisabledStateChanged = ParticleDataPtr && (ParticleDataPtr->ParticleDynamicsMisc.bDisabled != InRecordedData->ParticleDynamicsMisc.bDisabled);
	const bool bHasNewGeometry = !ParticleDataPtr || (ParticleDataPtr->GeometryHash != InRecordedData->GeometryHash);
	const bool bStateChanged = !ParticleDataPtr || (ParticleDataPtr->ParticleDynamicsMisc.MObjectState != InRecordedData->ParticleDynamicsMisc.MObjectState);
	// Particle name shouldn't change, but CVD keeps particle instances alive even when a particle is destroyed so they can be reused if a user scrubs back to a frame where the particle existed
	// But it could also be the case that the particle ID was re-used, therefore the name could have changed as this is a new particle
	const bool bHasNewName = !ParticleDataPtr || (ParticleDataPtr->MetadataId != InRecordedData->MetadataId);

	static FString UnnamedParticle("UnnamedParticle");
	if (bHasNewName)
	{
		SetDisplayName(InRecordedData->GetDebugName().IsEmpty() ? UnnamedParticle : InRecordedData->GetDebugName());
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::TEDS);
	}

	ParticleDataPtr = InRecordedData;

	if (bHasNewGeometry)
	{
		if (const TSharedPtr<FChaosVDScene>& ScenePtr = SceneWeakPtr.Pin())
		{
			uint32 GeometryHash = ParticleDataPtr->GeometryHash;

			if (GeometryHash != 0)
			{
				CurrentRootGeometry = ScenePtr->GetUpdatedGeometry(GeometryHash);

				if (!ensure(CurrentRootGeometry))
				{
					// We intentionally let the code continue, as passing down a null geometry  will take care of removing any existing mesh representation for this particle
					UE_LOGF(LogChaosVDEditor, Warning, "Failed to find Geometry for Particle ID [%d] | Geometry Hash [%u] | Debug Name [%ls]", ParticleDataPtr->ParticleIndex, GeometryHash, *ParticleDataPtr->GetDebugName());
				}
			}
			else
			{
				CurrentRootGeometry = nullptr;
			}
		}

		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Geometry);
	}

	// Now that we have updated particle data, update the Shape data and visibility as needed
	if (bShapeDataIsDirty || bHasNewGeometry)
	{
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::CollisionData);
	}
	else if (bDisabledStateChanged)
	{
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility);
	}

	if (EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility) || EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Parent))
	{
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::TEDS);
	}

	if (bStateChanged || EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility)
		|| EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::CollisionData)
		|| EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Geometry))
	{
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Coloring);
	}

	if (EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Transform) || EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Geometry))
	{
		CalculateAndCacheBounds();
	}
	
	EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::PreUpdatePass);
}

void FChaosVDSceneParticle::ProcessPendingParticleDataUpdates()
{
	if (!ParticleDataPtr)
	{
		return;
	}

	if (!ensure(IsInGameThread()))
	{
		UE_LOGF(LogChaosVDEditor, Error, "Attempted to update for particle [%ls] outside of the game thread!. This is not supported!", *GetDisplayName())
		return;
	}

	if (!ensure(EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::PreUpdatePass)))
	{
		UE_LOGF(LogChaosVDEditor, Error, "Attempted to process update for particle [%ls] without doing a pre pass first!. Current particle data is out of date!", *GetDisplayName())
		return;
	}

	if (StreamingState == EStreamingState::Visible)
	{
		if (EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Geometry))
		{
			UpdateGeometry(CurrentRootGeometry, EChaosVDActorGeometryUpdateFlags::ForceUpdate);
		}

		if (EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Transform))
		{
			ApplyPendingTransformData();
		}

		if (EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::CollisionData))
		{
			UpdateShapeDataComponents();
		}

		if (EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility))
		{
			UpdateGeometryComponentsVisibility();
		}

		if (EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Coloring))
		{
			UpdateGeometryColors();
		}
	}

	if (EnumHasAnyFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Parent))
	{
		UpdateParent(ParticleDataPtr);
	}

	EnumRemoveFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::PreUpdatePass);
}

void FChaosVDSceneParticle::SetSelectedMeshInstance(const TWeakPtr<FChaosVDInstancedMeshData>& GeometryInstanceToSelect)
{
	if (!ParticleDataPtr)
	{
		return;
	}

	const TSharedPtr<FChaosVDInstancedMeshData> GeometryInstanceToSelectPtr = GeometryInstanceToSelect.Pin();
	if (!GeometryInstanceToSelectPtr)
	{
		return;
	}

	if (ensure(ParticleDataPtr->ParticleIndex == GeometryInstanceToSelectPtr->GetOwningParticleID()))
	{
		CurrentSelectedGeometryInstance = GeometryInstanceToSelect;
	}
}

void FChaosVDSceneParticle::HandleDeSelected()
{
	CurrentSelectedGeometryInstance = nullptr;
	UpdateMeshInstancesSelectionState();
}

void FChaosVDSceneParticle::HandleSelected()
{
	UpdateMeshInstancesSelectionState();
}

bool FChaosVDSceneParticle::IsSelected()
{
	// The implementation of this method in UObject, used a global edit callback,
	// but as we don't use the global editor selection system, we need to re-route it.
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		return ScenePtr->IsSelected(Chaos::VD::TypedElementDataUtil::AcquireTypedElementHandleForStruct(this, true));
	}

	return false;
}

void FChaosVDSceneParticle::ProcessUpdatedAndRemovedHandles(TArray<TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>>& OutExtractedGeometryDataHandles)
{
	for (TArray<TSharedRef<FChaosVDInstancedMeshData>>::TIterator MeshDataHandleRemoveIterator = MeshDataHandles.CreateIterator(); MeshDataHandleRemoveIterator; ++MeshDataHandleRemoveIterator)
	{
		TSharedRef<FChaosVDInstancedMeshData>& ExistingMeshDataHandle = *MeshDataHandleRemoveIterator;

		bool bExists = false;

		// TODO: This search is n2, but I didn't see this as bottleneck. We should check if it is worth adding this to a TSet, or implementing the < operator so we can sort the array and do a binary search
		// (avoiding the need to allocate a new container) 
		for (TArray<TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>>::TIterator HandleRemoveIterator = OutExtractedGeometryDataHandles.CreateIterator(); HandleRemoveIterator; ++HandleRemoveIterator)
		{
			const TSharedPtr<FChaosVDExtractedGeometryDataHandle> GeometryDataHandle = HandleRemoveIterator->Key;
			const TSharedRef<FChaosVDExtractedGeometryDataHandle> ExistingComponentGeometryDataHandle = ExistingMeshDataHandle->GetGeometryHandle();

			if (GeometryDataHandle && *GeometryDataHandle == *ExistingMeshDataHandle->GetGeometryHandle())
			{
				bExists = true;

				// Although the geometry is the same, we need to copy over all the new data on the updated handle
				// Otherwise the ptr to the root implicit object or the Shape Instance Index will be outdated 
				*ExistingMeshDataHandle->GetGeometryHandle() = *GeometryDataHandle;
				HandleRemoveIterator->Value->SetMeshData(ExistingMeshDataHandle);

				// If we have a CVD Geometry Component for this handle, just remove it from the list as it means we don't need to re-create it
				HandleRemoveIterator.RemoveCurrent();
				break;
			}
		}

		if (!bExists)
		{
			if (IChaosVDGeometryComponent* AsGeometryComponent = Cast<IChaosVDGeometryComponent>(ExistingMeshDataHandle->GetMeshComponent()))
			{
				AsGeometryComponent->RemoveMeshInstance(ExistingMeshDataHandle);
				ExistingMeshDataHandle->MarkPendingDestroy();
			}

			MeshDataHandleRemoveIterator.RemoveCurrent();
		}		
	}
}

void FChaosVDSceneParticle::UpdateGeometry(const Chaos::FConstImplicitObjectPtr& InImplicitObject, EChaosVDActorGeometryUpdateFlags OptionsFlags)
{
	if (EnumHasAnyFlags(OptionsFlags, EChaosVDActorGeometryUpdateFlags::ForceUpdate))
	{
		bIsGeometryDataGenerationStarted = false;
	}

	if (bIsGeometryDataGenerationStarted)
	{
		return;
	}

	if (!ParticleDataPtr)
	{
		return;
	}

	if (!InImplicitObject.IsValid())
	{
		RemoveAllGeometry();
		EnumRemoveFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Geometry);
		return;
	}

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr.IsValid())
	{
		return;
	}

	const TSharedPtr<FChaosVDGeometryBuilder> GeometryGenerator = ScenePtr->GetGeometryGenerator().Pin();
	if (!GeometryGenerator.IsValid())
	{
		return;
	}

	const int32 ObjectsToGenerateNum = InImplicitObject->CountLeafObjectsInHierarchyImpl();

	// If the new implicit object is empty, then we can just clear all the mesh components and early out
	if (ObjectsToGenerateNum == 0)
	{
		VisitGeometryInstances([this](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
		{
			if (IChaosVDGeometryComponent* AsGeometryComponent = Cast<IChaosVDGeometryComponent>(MeshDataHandle->GetMeshComponent()))
			{
				AsGeometryComponent->RemoveMeshInstance(MeshDataHandle);
			}
		});

		MeshDataHandles.Reset();
		return;
	}
	
	TArray<TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>> OutExtractedGeometryDataHandles;
	OutExtractedGeometryDataHandles.Reserve(ObjectsToGenerateNum);

	bool bHasToUseStaticMeshComponent = Chaos::VisualDebugger::Cvars::bForceStaticMeshComponentUse;

	if (!bHasToUseStaticMeshComponent && !Chaos::VisualDebugger::Cvars::bUseInstancedStaticMeshForLandscape)
	{
		// Heightfields need to be created as Static meshes and use normal Static Mesh components because we need LODs for them due to their high triangle count
		bHasToUseStaticMeshComponent = FChaosVDGeometryBuilder::DoesImplicitContainType(InImplicitObject, Chaos::ImplicitObjectType::HeightField);
	}

	constexpr int32 LODsToGenerateNum = 3;
	constexpr int32 LODsToGenerateNumForInstancedStaticMesh = 0;

	RootImplicitView = TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>();
	
	TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView> ImplicitRoot = MakeShared<Chaos::VD::FChaosVDImplicitObjectView>();
	ImplicitRoot->ImplicitObjectType = InImplicitObject->GetType();

	GeometryGenerator->CreateMeshesFromImplicitObject(InImplicitObject, OutExtractedGeometryDataHandles, ImplicitRoot.Get(), ParticleDataPtr->CollisionDataPerShape.Num(), bHasToUseStaticMeshComponent ? LODsToGenerateNum : LODsToGenerateNumForInstancedStaticMesh);

	// This should not happen in theory, but there might be some valid situations where it does. Adding an ensure to catch them and then evaluate if it is really an issue (if it is not I will remove the ensure later on). 
	if (!ensure(ObjectsToGenerateNum == OutExtractedGeometryDataHandles.Num()))
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Geometry objects being generated doesn't match the number of objects in the implicit object | Expected [%d] | Being generated [%d] | Particle Actor [%ls]", ANSI_TO_TCHAR(__FUNCTION__), ObjectsToGenerateNum, OutExtractedGeometryDataHandles.Num(), *GetDisplayName());
	}

	// Figure out what geometry was removed, and destroy their components as needed. Also, if a geometry is already generated an active, remove it from the geometry to generate list
	ProcessUpdatedAndRemovedHandles(OutExtractedGeometryDataHandles);

	if (OutExtractedGeometryDataHandles.Num() > 0)
	{
		for (const TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>& ExtractedPair : OutExtractedGeometryDataHandles)
		{
			TSharedPtr<FChaosVDExtractedGeometryDataHandle> ExtractedGeometryDataHandle = ExtractedPair.Key;
			//TODO: Time Slice component creation
			TSharedPtr<FChaosVDInstancedMeshData> MeshDataInstance;

			if (!ExtractedGeometryDataHandle)
			{
				UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed To Create mesh data instance for [%ls] | Invalid Source geometry", ANSI_TO_TCHAR(__FUNCTION__), *GetDisplayName());
				continue;
			}

			if (bHasToUseStaticMeshComponent)
			{
				MeshDataInstance = GeometryGenerator->CreateMeshDataInstance<UChaosVDStaticMeshComponent>(*ParticleDataPtr.Get(), ExtractedGeometryDataHandle.ToSharedRef());
			}
			else
			{
				MeshDataInstance = GeometryGenerator->CreateMeshDataInstance<UChaosVDInstancedStaticMeshComponent>(*ParticleDataPtr.Get(), ExtractedGeometryDataHandle.ToSharedRef());
			}

			if (!MeshDataInstance)
			{
				UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed To Create mesh data instance for [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetDisplayName());
				continue;
			}

			IChaosVDGeometryComponent* CreatedMeshComponent = Cast<IChaosVDGeometryComponent>(MeshDataInstance->GetMeshComponent());
			if (!CreatedMeshComponent)
			{
				UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed To Create mesh component for [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetDisplayName());
				continue;
			}

			// If we have a valid transform data, we need to update our instance with it as the mesh component is not part of this actor (and event if it is, we don't use the actor transform anymore)
			if (ParticleDataPtr && ParticleDataPtr->ParticlePositionRotation.HasValidData())
			{
				const FVector TargetLocation = CachedSimulationTransform.TransformPosition(ParticleDataPtr->ParticlePositionRotation.MX);
				const FQuat TargetRotation = CachedSimulationTransform.GetRotation() * ParticleDataPtr->ParticlePositionRotation.MR;

				FTransform ParticleTransform;
				ParticleTransform.SetLocation(TargetLocation);
				ParticleTransform.SetRotation(TargetRotation);

				MeshDataInstance->SetWorldTransform(ParticleTransform);
			}

			ExtractedPair.Value->SetMeshData(MeshDataInstance.ToSharedRef());

			MeshDataHandles.Add(MeshDataInstance.ToSharedRef());
		}

		// Ensure that visibility and colorization is up to date after updating this Particle's Geometry
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility);
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Coloring);
		EnumRemoveFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Geometry);
		
		bIsGeometryDataGenerationStarted = true;
	}

	RootImplicitView = ImplicitRoot;
}

void FChaosVDSceneParticle::CalculateAndCacheBounds() const
{
	if (CurrentRootGeometry == nullptr)
	{
		return;
	}

	if (CurrentRootGeometry->HasBoundingBox())
	{
		Chaos::FAABB3 ChaosBox = CurrentRootGeometry->CalculateTransformedBounds(PendingParticleTransform);
		CachedBounds = FBox(ChaosBox.Min(),ChaosBox.Max());
	}
}

FBox FChaosVDSceneParticle::GetBoundingBox() const
{
	if (!CachedBounds.IsValid)
	{
		CalculateAndCacheBounds();
	}

	return CachedBounds;
}

FBox FChaosVDSceneParticle::GetInflatedBoundingBox() const
{
	if (ParticleDataPtr && ParticleDataPtr->ParticleInflatedBounds.HasValidData())
	{
		return FBox(ParticleDataPtr->ParticleInflatedBounds.MMin, ParticleDataPtr->ParticleInflatedBounds.MMax);
	}
	return GetBoundingBox();
}

Chaos::TAABB<double, 3> FChaosVDSceneParticle::GetChaosBoundingBox() const
{
	FBox Bounds = GetBoundingBox();

	return Chaos::TAABB<double, 3>(Bounds.Min, Bounds.Max);
}

TConstArrayView<TSharedPtr<FChaosVDParticlePairMidPhase>> FChaosVDSceneParticle::GetCollisionData()
{
	if (const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>* MidPhases = GetCollisionMidPhasesArray())
	{
		return *MidPhases;
	}
	return TConstArrayView<TSharedPtr<FChaosVDParticlePairMidPhase>>();
}

bool FChaosVDSceneParticle::HasCollisionData()
{
	if (const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>* MidPhases = GetCollisionMidPhasesArray())
	{
		return MidPhases->Num() > 0;
	}

	return false;
}

void FChaosVDSceneParticle::UpdateMeshInstancesSelectionState()
{
	TSharedPtr<FChaosVDInstancedMeshData> CurrentSelectedGeometry = CurrentSelectedGeometryInstance.Pin();
	const bool bIsOwningParticleSelectedInEditor = IsSelected();
	VisitGeometryInstances([this, bIsOwningParticleSelectedInEditor, CurrentSelectedGeometry](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
	{
		const bool bShouldSelectInstance = bIsOwningParticleSelectedInEditor ? (CurrentSelectedGeometry ? CurrentSelectedGeometryInstance == MeshDataHandle : true) : false;

		MeshDataHandle->SetIsSelected(bShouldSelectInstance);
	});
}

void FChaosVDSceneParticle::GetCharacterGroundConstraintData(TArray<TSharedPtr<FChaosVDCharacterGroundConstraint>>& OutConstraintsFound)
{
	if (const TArray<TSharedPtr<FChaosVDConstraintDataWrapperBase>>* Constraints = GetCharacterGroundConstraintArray())
	{
		OutConstraintsFound.Reserve(Constraints->Num());

		for (const TSharedPtr<FChaosVDConstraintDataWrapperBase>& Constraint : *Constraints)
		{
			OutConstraintsFound.Add(StaticCastSharedPtr<FChaosVDCharacterGroundConstraint>(Constraint));
		}
	}
}

bool FChaosVDSceneParticle::HasCharacterGroundConstraintData()
{
	if (const TArray<TSharedPtr<FChaosVDConstraintDataWrapperBase>>* Constraints = GetCharacterGroundConstraintArray())
	{
		return Constraints->Num() > 0;
	}

	return false;
}

const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>* FChaosVDSceneParticle::GetCollisionMidPhasesArray() const
{
	if (!ParticleDataPtr.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr.IsValid())
	{
		return nullptr;
	}

	if (AChaosVDSolverInfoActor* SolverInfoActor = ScenePtr->GetSolverInfoActor(ParticleDataPtr->SolverID))
	{
		if (const UChaosVDSolverCollisionDataComponent* CollisionDataComponent = SolverInfoActor->GetCollisionDataComponent())
		{
			return CollisionDataComponent->GetMidPhasesForParticle(ParticleDataPtr->ParticleIndex, EChaosVDParticlePairSlot::Any);
		}
	}

	return nullptr;
}

const TArray<TSharedPtr<FChaosVDConstraintDataWrapperBase>>* FChaosVDSceneParticle::GetCharacterGroundConstraintArray() const
{
	if (!ParticleDataPtr.IsValid())
	{
		return nullptr;
	}

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr.IsValid())
	{
		return nullptr;
	}

	if (AChaosVDSolverInfoActor* SolverInfoActor = ScenePtr->GetSolverInfoActor(ParticleDataPtr->SolverID))
	{
		if (const UChaosVDSolverCharacterGroundConstraintDataComponent* ConstraintDataComponent = SolverInfoActor->GetCharacterGroundConstraintDataComponent())
		{
			return ConstraintDataComponent->GetConstraintsForParticle(ParticleDataPtr->ParticleIndex, EChaosVDParticlePairSlot::Primary);
		}
	}

	return nullptr;
}

void FChaosVDSceneParticle::UpdateShapeDataComponents()
{
	VisitGeometryInstances([this](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
	{
		if (ParticleDataPtr)
		{
			FChaosVDGeometryComponentUtils::UpdateCollisionDataFromShapeArray(ParticleDataPtr->CollisionDataPerShape, MeshDataHandle);
		}
	});


	EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility);
	EnumRemoveFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::CollisionData);
}

void FChaosVDSceneParticle::ApplyPendingTransformData()
{
	VisitGeometryInstances([this](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
	{
		MeshDataHandle->SetWorldTransform(PendingParticleTransform);
	});

	EnumRemoveFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Transform);
}

void FChaosVDSceneParticle::SetParent(const TSharedPtr<FChaosVDBaseSceneObject>& NewParent)
{
	FChaosVDBaseSceneObject::SetParent(NewParent);

	EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::TEDS);
}

void FChaosVDSceneParticle::UpdateGeometryComponentsVisibility(EChaosVDParticleVisibilityUpdateFlags Flags)
{
	VisitGeometryInstances([this](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
	{
		if (ParticleDataPtr)
		{
			FChaosVDGeometryComponentUtils::UpdateMeshVisibility(MeshDataHandle, *ParticleDataPtr.Get(), IsVisible());
		}
	});
	
	EnumRemoveFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility);

	if (EnumHasAnyFlags(Flags, EChaosVDParticleVisibilityUpdateFlags::DirtyScene))
	{
		if (TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
		{
			ScenePtr->RequestUpdate();
		}
	}
}

void FChaosVDSceneParticle::UpdateGeometryColors()
{
	VisitGeometryInstances([this](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
	{
		if (ParticleDataPtr)
		{
			FChaosVDGeometryComponentUtils::UpdateMeshColor(MeshDataHandle, *ParticleDataPtr.Get(), GetIsServerParticle());
		}
	});
	
	EnumRemoveFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Coloring);
}

void FChaosVDSceneParticle::SetIsActive(bool bNewActive)
{
	if (bIsActive != bNewActive)
	{
		bIsActive = bNewActive;

		if (bNewActive)
		{
			RemoveHiddenFlag(EChaosVDHideParticleFlags::HiddenByActiveState);
		}
		else
		{
			AddHiddenFlag(EChaosVDHideParticleFlags::HiddenByActiveState);
		}

		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::TEDS);
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Active);
	}
}

void FChaosVDSceneParticle::AddHiddenFlag(EChaosVDHideParticleFlags Flag)
{
	bool bOldIsVisible = IsVisible();

	EnumAddFlags(HideParticleFlags, Flag);

	if (bOldIsVisible != IsVisible())
	{
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility);
	}
}

void FChaosVDSceneParticle::RemoveHiddenFlag(EChaosVDHideParticleFlags Flag)
{
	bool bOldIsVisible = IsVisible();

	EnumRemoveFlags(HideParticleFlags, Flag);

	if (bOldIsVisible != IsVisible())
	{
		EnumAddFlags(DirtyFlags, EChaosVDSceneParticleDirtyFlags::Visibility);
	}
}

void FChaosVDSceneParticle::HideImmediate(EChaosVDHideParticleFlags Flag)
{
	AddHiddenFlag(Flag);
	UpdateGeometryComponentsVisibility(EChaosVDParticleVisibilityUpdateFlags::DirtyScene);
	Chaos::VD::TedsUtils::AddColumnToObject<FTypedElementSyncFromWorldTag>(this);
}

void FChaosVDSceneParticle::ShowImmediate()
{
	HideParticleFlags = EChaosVDHideParticleFlags::None;
	UpdateGeometryComponentsVisibility(EChaosVDParticleVisibilityUpdateFlags::DirtyScene);
	Chaos::VD::TedsUtils::AddColumnToObject<FTypedElementSyncFromWorldTag>(this);
}
