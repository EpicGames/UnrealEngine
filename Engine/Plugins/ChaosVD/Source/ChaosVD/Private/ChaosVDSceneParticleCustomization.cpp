// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSceneParticleCustomization.h"

#include "Actors/ChaosVDSolverInfoActor.h"
#include "ChaosVDCollisionDataDetailsTab.h"
#include "ChaosVDEngine.h"
#include "ChaosVDIndependentDetailsPanelManager.h"
#include "ChaosVDModule.h"
#include "ChaosVDObjectDetailsTab.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDScene.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailsView.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "DetailsCustomizations/ChaosVDDetailsCustomizationUtils.h"
#include "Misc/MessageDialog.h"
#include "Widgets/SChaosVDCollisionDataInspector.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDSceneParticleCustomization::FChaosVDSceneParticleCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab)
{
	AllowedCategories.Add(FChaosVDSceneParticleCustomization::ParticleDataCategoryName);
	AllowedCategories.Add(FChaosVDSceneParticleCustomization::GeometryCategoryName);

	MainTabWeakPtr = InMainTab;

	ResetCachedView();
}

FChaosVDSceneParticleCustomization::~FChaosVDSceneParticleCustomization()
{
	RegisterCVDScene(nullptr);
}

TSharedRef<IDetailCustomization> FChaosVDSceneParticleCustomization::MakeInstance(TWeakPtr<SChaosVDMainTab> InMainTab)
{
	return MakeShared<FChaosVDSceneParticleCustomization>(InMainTab);
}


void FChaosVDSceneParticleCustomization::AddParticleDataButtons(IDetailLayoutBuilder& DetailBuilder)
{
	IDetailCategoryBuilder& ParticleDataCategoryBuilder = DetailBuilder.EditCategory(ParticleDataCategoryName);

	ParticleDataCategoryBuilder.AddCustomRow(FText::GetEmpty()).
	                            WholeRowContent()
	[
		GenerateOpenInNewDetailsPanelButton().ToSharedRef()
	];

	const FText CollisionDataRowLabel = LOCTEXT("ParticleCollisionData", "Particle Collision Data");
	IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory("CollisionData", CollisionDataRowLabel);

	CategoryBuilder.AddCustomRow(CollisionDataRowLabel).
	                WholeRowContent()
	[
		GenerateShowCollisionDataButton().ToSharedRef()
	];
}


void FChaosVDSceneParticleCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	CachedDetailBuilder = &DetailBuilder;
	TGuardValue<bool> InsideCustomizeDetailsGuard(bIsInsideCustomizeDetails, true);

	FChaosVDDetailsCustomizationUtils::HideAllCategories(DetailBuilder, AllowedCategories);

	TSharedPtr<SChaosVDMainTab> MainTabPtr =  MainTabWeakPtr.Pin(); 
	TSharedPtr<FChaosVDScene> Scene = MainTabPtr ? MainTabPtr->GetChaosVDEngineInstance()->GetCurrentScene() : nullptr;

	RegisterCVDScene(Scene);

	if (!Scene)
	{
		ResetCachedView();
		return;
	}

	// We keep the particle data we need to visualize as a shared ptr because copying it each frame we advance/rewind to to an struct that lives in the particle actor it is not cheap.
	// Having a struct details view to which we set that pointer data each time the data in the particle is updated (meaning we assigned another ptr from the recording)
	// seems to be more expensive because it has to rebuild the entire layout from scratch.
	// So a middle ground I found is to have a Particle Data struct in this customization instance, which we add as external property. Then each time the particle data is updated we copy the data over.
	// This allows us to only perform the copy just for the particle that is being inspected and not every particle updated in that frame.

	TArray<TSharedPtr<FStructOnScope>> SelectedObjects;
	DetailBuilder.GetStructsBeingCustomized(SelectedObjects);
	if (SelectedObjects.Num() > 0)
	{
		//TODO: Add support for multi-selection.
		if (!ensure(SelectedObjects.Num() == 1))
		{
			UE_LOGF(LogChaosVDEditor, Warning, "[%ls] [%d] objects were selected but this customization panel only support single object selection.", ANSI_TO_TCHAR(__FUNCTION__), SelectedObjects.Num())
		}
		
		FChaosVDSceneParticle* CurrentParticleInstance = CurrentObservedParticle;
		FChaosVDSceneParticle* SelectedParticleInstance = nullptr;

		TSharedPtr<FStructOnScope>& SelectedStruct = SelectedObjects[0];
		if (SelectedStruct->GetStruct() == FChaosVDSceneParticle::StaticStruct())
		{
			SelectedParticleInstance = reinterpret_cast<FChaosVDSceneParticle*>(SelectedStruct->GetStructMemory());
		}

		if (CurrentParticleInstance && CurrentParticleInstance != SelectedParticleInstance)
		{
			ResetCachedView();
		}
		
		if (SelectedParticleInstance)
		{
			UpdateObserverParticlePtr(SelectedParticleInstance);

			HandleSceneUpdated();

			// CVD is a read-only viewer: mark all property rows as non-interactive so any
			// struct with EditAnywhere properties does not show editable input chrome.
			// The DisabledEffect gray-out is suppressed at the widget level by SChaosVDDetailsView.
			if (TSharedPtr<IDetailsView> View = DetailBuilder.GetDetailsViewSharedPtr())
			{
				if (!View->GetIsPropertyReadOnlyDelegate().IsBound())
				{
					View->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateLambda(
						[](const FPropertyAndParent&) { return true; }));
				}
			}

			if (TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = SelectedParticleInstance->GetParticleData())
			{
				// If bHasDebugName it means this is an old CVD recording where we didn't have the metadata system (or that it was a particle without a valid name)
				// In which case would be ok showing an empty metadata structure, even if it is an old CVD file
				if (!ParticleData->HasLegacyDebugName())
				{
					AddExternalStructure(CachedParticleMetadata, DetailBuilder, FName("Particle Metadata"), LOCTEXT("ParticleMetadataStructName", "Particle Metadata"));
				}
			}

			TSharedPtr<IPropertyHandle> InspectedDataPropertyHandlePtr;

			if (TSharedPtr<FChaosVDInstancedMeshData> SelectedGeometryInstance = SelectedParticleInstance->GetSelectedMeshInstance().Pin())
			{
				InspectedDataPropertyHandlePtr = AddExternalStructure(CachedGeometryDataInstanceCopy, DetailBuilder, GeometryCategoryName, LOCTEXT("GeometryShapeDataStructName", "Geometry Shape Data"));

				IDetailCategoryBuilder& ParticleDataCategoryBuilder = DetailBuilder.EditCategory(GeometryCategoryName);

				ParticleDataCategoryBuilder.AddCustomRow(FText::GetEmpty()).
											WholeRowContent()
				[
					GenerateOpenInNewDetailsPanelButton().ToSharedRef()
				];
			}
			else
			{
				InspectedDataPropertyHandlePtr = AddExternalStructure(CachedParticleData, DetailBuilder, ParticleDataCategoryName, LOCTEXT("ParticleDataStructName", "Particle Data"));
				AddParticleDataButtons(DetailBuilder);
			}

			if (InspectedDataPropertyHandlePtr)
			{
				TSharedRef<IPropertyHandle> InspectedDataPropertyHandleRef = InspectedDataPropertyHandlePtr.ToSharedRef();
				FChaosVDDetailsCustomizationUtils::HideInvalidCVDDataWrapperProperties({&InspectedDataPropertyHandleRef, 1}, DetailBuilder);
			}

			// Fire the extra data delegate so any registered component (e.g. UChaosVDParticleExtraDataComponent)
			// can append additional categories for this particle.
			if (TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = SelectedParticleInstance->GetParticleData())
			{
				const int32 SolverID = ParticleData->SolverID;
				const int32 ParticleID = ParticleData->ParticleIndex;
				if (SolverID != INDEX_NONE && ParticleID != INDEX_NONE)
				{
					Scene->OnParticleDetailsExtraDataRequested().Broadcast(SolverID, ParticleID, DetailBuilder);
				}
			}
		}
	}
	else
	{
		ResetCachedView();
	}

}

void FChaosVDSceneParticleCustomization::HandleSceneUpdated()
{
	if (!CurrentObservedParticle)
	{
		ResetCachedView();
		return;
	}

	// If we have selected a mesh instance, the only data being added to the details panel is the Shape Instance data, so can just update that data here
	if (TSharedPtr<FChaosVDInstancedMeshData> SelectedGeometryInstance = CurrentObservedParticle->GetSelectedMeshInstance().Pin())
	{
		CurrentObservedParticle->VisitGeometryInstances([this, SelectedGeometryInstance](const TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle)
		{
			if (MeshDataHandle == SelectedGeometryInstance)
			{
				CachedGeometryDataInstanceCopy = MeshDataHandle->GetState();
			}
		});
	}
	else
	{
		TSharedPtr<const FChaosVDParticleDataWrapper> ParticleDataPtr = CurrentObservedParticle->GetParticleData();
		CachedParticleData = ParticleDataPtr ? *ParticleDataPtr : FChaosVDParticleDataWrapper();

		if (ParticleDataPtr)
		{
			CachedParticleData = *ParticleDataPtr;
			if (const TSharedPtr<FChaosVDParticleMetadata>& MetadataInstance = ParticleDataPtr->GetMetadataInstance())
			{
				CachedParticleMetadata = *MetadataInstance;
			}
			else
			{
				CachedParticleMetadata = FChaosVDParticleMetadata();
			}
		}
		else
		{
			CachedParticleData = FChaosVDParticleDataWrapper();
			CachedParticleMetadata = FChaosVDParticleMetadata();
		}
	}

	// Notify extra data subscribers (e.g. UChaosVDParticleExtraDataComponent) that new frame data
	// is available. They update their cached FStructOnScope memory in-place so Slate reflects the
	// new values on the next redraw. If the data layout has changed a subscriber sets
	// bNeedsFullRebuild=true and we fall back to a ForceRefreshDetails.
	if (const TSharedPtr<FChaosVDScene> Scene = SceneWeakPtr.Pin())
	{
		if (Scene->OnParticleDetailsExtraDataRefreshRequested().IsBound())
		{
			if (TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = CurrentObservedParticle->GetParticleData())
			{
				const int32 SolverID = ParticleData->SolverID;
				const int32 ParticleID = ParticleData->ParticleIndex;
				if (SolverID != INDEX_NONE && ParticleID != INDEX_NONE)
				{
					bool bNeedsFullRebuild = false;
					Scene->OnParticleDetailsExtraDataRefreshRequested().Broadcast(SolverID, ParticleID, bNeedsFullRebuild);

					if (bNeedsFullRebuild && CachedDetailBuilder && !bIsInsideCustomizeDetails)
					{
						IDetailLayoutBuilder* BuilderToRefresh = CachedDetailBuilder;
						CachedDetailBuilder = nullptr;
						BuilderToRefresh->ForceRefreshDetails();
					}
				}
			}
		}
	}
}

bool FChaosVDSceneParticleCustomization::GetCollisionDataButtonEnabled() const
{
	return CurrentObservedParticle && CurrentObservedParticle->HasCollisionData();
}

FReply FChaosVDSceneParticleCustomization::ShowCollisionDataForSelectedObject()
{
	if (!CurrentObservedParticle)
	{
		return FReply::Handled();
	}
	
	TSharedPtr<SChaosVDMainTab> OwningTabPtr = MainTabWeakPtr.Pin();
	if (!OwningTabPtr.IsValid())
	{
		return FReply::Handled();
	}

	if (const TSharedPtr<FChaosVDCollisionDataDetailsTab> CollisionDataTab = OwningTabPtr->GetTabSpawnerInstance<FChaosVDCollisionDataDetailsTab>(FChaosVDTabID::CollisionDataDetails).Pin())
	{
		if (const TSharedPtr<FTabManager> TabManager = OwningTabPtr->GetTabManager())
		{
			TabManager->TryInvokeTab(FChaosVDTabID::CollisionDataDetails);

			if (const TSharedPtr<SChaosVDCollisionDataInspector> CollisionInspector = CollisionDataTab->GetCollisionInspectorInstance().Pin())
			{
				CollisionInspector->SetCollisionDataListToInspect(CurrentObservedParticle->GetCollisionData());
			}
		}
	}

	return FReply::Handled();
}

// Builds a tab label for a stand-alone particle details panel.
// Combines solver name and particle owner name so panels for particles from different
// network endpoints can be distinguished at a glance even when the strip truncates labels.
static FText MakeParticleDetailsPanelLabel(const FChaosVDSceneParticle* Particle, const TSharedPtr<SChaosVDMainTab>& MainTab)
{
	const TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = Particle ? Particle->GetParticleData() : nullptr;
	if (!ParticleData)
	{
		return LOCTEXT("DetailsPanel", "Details");
	}

	TSharedPtr<FChaosVDScene> Scene = MainTab ? MainTab->GetChaosVDEngineInstance()->GetCurrentScene() : nullptr;

	FName SolverName;
	if (Scene)
	{
		if (AChaosVDSolverInfoActor* SolverActor = Scene->GetSolverInfoActor(ParticleData->SolverID))
		{
			SolverName = SolverActor->GetSolverName();
		}
	}

	FName OwnerName;
	if (const TSharedPtr<FChaosVDParticleMetadata>& Metadata = ParticleData->GetMetadataInstance())
	{
		OwnerName = Metadata->OwnerName;
	}

	if (SolverName.IsNone() && OwnerName.IsNone())
	{
		return LOCTEXT("DetailsPanel", "Details");
	}

	// Start with the solver/endpoint name so it is visible even when the tab
	// label is truncated in the middle by the tab strip.
	FString LabelStr;
	if (!SolverName.IsNone())
	{
		LabelStr = SolverName.ToString();
	}
	if (!OwnerName.IsNone())
	{
		if (!LabelStr.IsEmpty())
		{
			LabelStr += TEXT(" | ");
		}
		LabelStr += OwnerName.ToString();
	}
	return FText::FromString(LabelStr);
}

FReply FChaosVDSceneParticleCustomization::OpenNewDetailsPanel()
{
	if (!CurrentObservedParticle)
	{
		return FReply::Handled();
	}

	TSharedPtr<SChaosVDMainTab> OwningTabPtr = MainTabWeakPtr.Pin();
	if (!OwningTabPtr.IsValid())
	{
		return FReply::Handled();
	}

	if (TSharedPtr<FChaosVDIndependentDetailsPanelManager> IndependentDetailsPanelManager = OwningTabPtr->GetIndependentDetailsPanelManager())
	{
		if (TSharedPtr<FChaosVDStandAloneObjectDetailsTab> DetailsTab = IndependentDetailsPanelManager->GetAvailableStandAloneDetailsPanelTab())
		{
			DetailsTab->SetStructToInspect(CurrentObservedParticle);
			DetailsTab->UpdateTabLabel(MakeParticleDetailsPanelLabel(CurrentObservedParticle, OwningTabPtr));

			return FReply::Handled();
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("OpenDetailsPanelError", "No (selection independent) Details Panel slot available.\n\nPlease close a panel and try again."));
		}
	}

	return FReply::Handled();
}

void FChaosVDSceneParticleCustomization::ResetCachedView()
{
	if (CurrentObservedParticle)
	{
		CurrentObservedParticle->ParticleDestroyedDelegate.Unbind();
	}

	CurrentObservedParticle = nullptr;
	CachedParticleData = FChaosVDParticleDataWrapper();
	CachedGeometryDataInstanceCopy = FChaosVDMeshDataInstanceState();
	CachedParticleMetadata = FChaosVDParticleMetadata();
}

void FChaosVDSceneParticleCustomization::RegisterCVDScene(const TSharedPtr<FChaosVDScene>& InScene)
{
	TSharedPtr<FChaosVDScene> CurrentScene = SceneWeakPtr.Pin();
	if (InScene != CurrentScene)
	{
		if (CurrentScene)
		{
			CurrentScene->OnSceneUpdated().RemoveAll(this);
		}

		if (InScene)
		{
			InScene->OnSceneUpdated().AddSP(this, &FChaosVDSceneParticleCustomization::HandleSceneUpdated);
		}

		SceneWeakPtr = InScene;
	}
}

static TSharedRef<SWidget> MakeCVDDetailsButton(const FText& Tooltip, const FText& Label, FOnClicked OnClicked, TAttribute<bool> IsEnabled = true)
{
	return SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	.Padding(12.0f, 7.0f, 12.0f, 7.0f)
	.FillWidth(1.0f)
	[
		SNew(SButton)
		.ToolTip(SNew(SToolTip).Text(Tooltip))
		.IsEnabled(IsEnabled)
		.ContentPadding(FMargin(0, 5.f, 0, 4.f))
		.OnClicked(OnClicked)
		.Content()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(3, 0, 0, 0))
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallButtonText")
				.Text(Label)
			]
		]
	];
}

TSharedPtr<SWidget> FChaosVDSceneParticleCustomization::GenerateShowCollisionDataButton()
{
	return MakeCVDDetailsButton(
		LOCTEXT("OpenCollisionDataDesc", "Click here to open the collision data for this particle on the collision data inspector."),
		LOCTEXT("ShowCollisionDataOnInspector", "Show Collision Data in Inspector"),
		FOnClicked::CreateRaw(this, &FChaosVDSceneParticleCustomization::ShowCollisionDataForSelectedObject),
		TAttribute<bool>::CreateRaw(this, &FChaosVDSceneParticleCustomization::GetCollisionDataButtonEnabled));
}

TSharedPtr<SWidget> FChaosVDSceneParticleCustomization::GenerateOpenInNewDetailsPanelButton()
{
	return MakeCVDDetailsButton(
		LOCTEXT("OpenDetailsPanelDesc", "Click here to open a new (selection independent) details panel for this particle."),
		LOCTEXT("OpenDetailsPanelText", "Show Data in New Panel"),
		FOnClicked::CreateRaw(this, &FChaosVDSceneParticleCustomization::OpenNewDetailsPanel));
}

void FChaosVDSceneParticleCustomization::UpdateObserverParticlePtr(FChaosVDSceneParticle* NewObservedParticle)
{
	if (CurrentObservedParticle)
	{
		CurrentObservedParticle->ParticleDestroyedDelegate.Unbind();
	}

	if (NewObservedParticle)
	{
		NewObservedParticle->ParticleDestroyedDelegate.BindSP(this, &FChaosVDSceneParticleCustomization::HandleObservedParticleInstanceDestroyed);
		CurrentObservedParticle = NewObservedParticle;	
	}
	else
	{
		ResetCachedView();
	}
}


void FChaosVDSceneParticleCustomization::HandleObservedParticleInstanceDestroyed()
{
	ResetCachedView();
}

#undef LOCTEXT_NAMESPACE
