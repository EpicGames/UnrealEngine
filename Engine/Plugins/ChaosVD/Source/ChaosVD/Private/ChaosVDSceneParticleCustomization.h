// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDGeometryDataComponent.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailPropertyRow.h"
#include "Templates/SharedPointer.h"
#include "IDetailCustomization.h"
#include "PropertyHandle.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"

struct FChaosVDSceneParticle;
class SChaosVDMainTab;
struct FChaosVDParticleDataWrapper;

/** Custom details panel for the ChaosVD Particle Actor */
class FChaosVDSceneParticleCustomization : public IDetailCustomization
{
public:
	FChaosVDSceneParticleCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab);
	virtual ~FChaosVDSceneParticleCustomization() override;

	inline static FName ParticleDataCategoryName = FName("Particle Data");
	inline static FName GeometryCategoryName = FName("Geometry Shape Data");

	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<SChaosVDMainTab> InMainTab);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:

	void AddParticleDataButtons(IDetailLayoutBuilder& DetailBuilder);

	template<typename TStruct>
	TSharedPtr<IPropertyHandle> AddExternalStructure(TStruct& CachedStruct, IDetailLayoutBuilder& DetailBuilder, FName CategoryName, const FText& InPropertyName);

	TSet<FName> AllowedCategories;
	FChaosVDSceneParticle* CurrentObservedParticle = nullptr;
	TSharedPtr<FStructOnScope> CurrentlyObserverParticleDataStruct;

	void ResetCachedView();

	void RegisterCVDScene(const TSharedPtr<FChaosVDScene>& InScene);

	void HandleSceneUpdated();

	bool GetCollisionDataButtonEnabled() const;

	FReply ShowCollisionDataForSelectedObject();
	FReply OpenNewDetailsPanel();

	TSharedPtr<SWidget> GenerateShowCollisionDataButton();
	TSharedPtr<SWidget> GenerateOpenInNewDetailsPanelButton();

	void UpdateObserverParticlePtr(FChaosVDSceneParticle* NewObservedParticle);

	void HandleObservedParticleInstanceDestroyed();

	/* Copy of the last known geometry shape data structure of a selected particle and mesh instance -  Used to avoid rebuild the layout every time we change frame in CVD */
	FChaosVDParticleDataWrapper CachedParticleData;
	/* Copy of the last known particle data structure of a selected particle -  Used to avoid rebuild the layout every time we change frame in CVD */
	FChaosVDMeshDataInstanceState CachedGeometryDataInstanceCopy;
	/* Copy of the last known particle metadata data structure of a selected particle -  Used to avoid rebuild the layout every time we change frame in CVD */
	FChaosVDParticleMetadata CachedParticleMetadata;
	
	TWeakPtr<FChaosVDScene> SceneWeakPtr = nullptr;

	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr = nullptr;

	/** Cached builder pointer used to trigger a ForceRefreshDetails if an extra data subscriber
	 *  reports a layout mismatch during an in-place refresh. Nulled before the call to avoid
	 *  dangling use if the rebuild is synchronous. */
	IDetailLayoutBuilder* CachedDetailBuilder = nullptr;

	/** Guard that prevents HandleSceneUpdated from issuing a ForceRefreshDetails while
	 *  CustomizeDetails is already on the call stack, avoiding an infinite rebuild loop. */
	bool bIsInsideCustomizeDetails = false;
};

template <typename TStruct>
TSharedPtr<IPropertyHandle> FChaosVDSceneParticleCustomization::AddExternalStructure(TStruct& CachedStruct, IDetailLayoutBuilder& DetailBuilder, FName CategoryName, const FText& InPropertyName)
{
	DetailBuilder.EditCategory(CategoryName, FText::GetEmpty(), ECategoryPriority::Important);
	IDetailCategoryBuilder& CVDMainCategoryBuilder = DetailBuilder.EditCategory(CategoryName).InitiallyCollapsed(false);

	const TSharedPtr<FStructOnScope> DataView = MakeShared<FStructOnScope>(TStruct::StaticStruct(), reinterpret_cast<uint8*>(&CachedStruct));

	FAddPropertyParams AddParams;
	AddParams.CreateCategoryNodes(true);

	if (IDetailPropertyRow* PropertyRow = CVDMainCategoryBuilder.AddExternalStructureProperty(DataView, NAME_None, EPropertyLocation::Default, AddParams))
	{
		PropertyRow->ShouldAutoExpand(true);
		PropertyRow->DisplayName(InPropertyName);
		// CVD data is read-only: the reset-to-default arrow has no meaning here and would
		// confuse users. Hide it for the entire struct tree (propagates to all children).
		PropertyRow->OverrideResetToDefault(FResetToDefaultOverride::Hide(/* bPropagateToChildren= */ true));
		return PropertyRow->GetPropertyHandle();
	}

	return nullptr;
}
