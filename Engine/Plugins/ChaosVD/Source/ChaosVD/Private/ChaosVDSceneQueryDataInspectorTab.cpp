// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSceneQueryDataInspectorTab.h"

#include "ChaosVDStyle.h"
#include "EditorModeManager.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Widgets/SChaosVDSceneQueryDataInspector.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDSceneQueryDataInspectorTab::~FChaosVDSceneQueryDataInspectorTab()
{
}

TSharedRef<SDockTab> FChaosVDSceneQueryDataInspectorTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> DetailsPanelTab =
	SNew(SDockTab)
	.TabRole(ETabRole::PanelTab)
	.Label(LOCTEXT("SceneQueryInspectorTab", "Scene Query Data Inspector"))
	.ToolTipText(LOCTEXT("SceneQueryInspectorTabTip", "See the details of the any scene query selected in the viewport"));

	if (const TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin())
	{
		DetailsPanelTab->SetContent
		(
			SAssignNew(SceneQueryDataInspector, SChaosVDSceneQueryDataInspector, GetChaosVDScene(), MainTabPtr.ToSharedRef())
		);
	}
	else
	{
		DetailsPanelTab->SetContent(GenerateErrorWidget());
	}

	DetailsPanelTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("SceneQueriesInspectorIcon"));

	HandleTabSpawned(DetailsPanelTab);

	return DetailsPanelTab;
}

void FChaosVDSceneQueryDataInspectorTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	FChaosVDTabSpawnerBase::HandleTabClosed(InTabClosed);

	SceneQueryDataInspector.Reset();
}

#undef LOCTEXT_NAMESPACE
