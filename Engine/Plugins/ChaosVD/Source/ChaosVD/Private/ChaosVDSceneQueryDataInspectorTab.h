// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDTabSpawnerBase.h"
#include "Templates/SharedPointer.h"

class SChaosVDSceneQueryDataInspector;

/** Spawns and handles and instance for the visual debugger Scene Query Data Inspector tab */
class FChaosVDSceneQueryDataInspectorTab : public FChaosVDTabSpawnerBase
{
public:
	FChaosVDSceneQueryDataInspectorTab(const FName& InTabID, const TSharedPtr<FTabManager>& InTabManager, const TWeakPtr<SChaosVDMainTab>& InOwningTabWidget)
		: FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{
	}

	virtual ~FChaosVDSceneQueryDataInspectorTab() override;
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

	TWeakPtr<SChaosVDSceneQueryDataInspector> GetSceneQueryDataInspectorInstance() const { return SceneQueryDataInspector; }

protected:
	TSharedPtr<SChaosVDSceneQueryDataInspector> SceneQueryDataInspector;
};
