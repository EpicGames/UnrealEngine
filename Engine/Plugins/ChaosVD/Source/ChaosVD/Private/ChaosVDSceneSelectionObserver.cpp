// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSceneSelectionObserver.h"

#include "Elements/Framework/TypedElementSelectionSet.h"

FChaosVDSceneSelectionObserver::FChaosVDSceneSelectionObserver()
{
}

FChaosVDSceneSelectionObserver::~FChaosVDSceneSelectionObserver()
{
	if (ObservedSelection.IsValid())
	{
		ObservedSelection->OnPreChange().RemoveAll(this);
		ObservedSelection->OnChanged().RemoveAll(this);
	}
}

void FChaosVDSceneSelectionObserver::RegisterSelectionSetObject(UTypedElementSelectionSet* SelectionSetObject)
{
	if (ObservedSelection.IsValid())
	{
		ObservedSelection->OnPreChange().RemoveAll(this);
		ObservedSelection->OnChanged().RemoveAll(this);
	}

	if (SelectionSetObject)
	{
		ObservedSelection = TWeakObjectPtr<UTypedElementSelectionSet>(SelectionSetObject);
		ObservedSelection->OnPreChange().AddRaw(this, &FChaosVDSceneSelectionObserver::HandlePreSelectionChange);
		ObservedSelection->OnChanged().AddRaw(this, &FChaosVDSceneSelectionObserver::HandlePostSelectionChange);
	}
}
