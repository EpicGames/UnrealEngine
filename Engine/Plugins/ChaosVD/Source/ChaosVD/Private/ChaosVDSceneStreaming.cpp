// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSceneStreaming.h"

#include "ChaosVDBaseSceneObject.h"
#include "ChaosVDModule.h"
#include "ChaosVDScene.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDSettingsManager.h"
#include "Misc/ScopeRWLock.h"
#include "Settings/ChaosVDGeneralSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSceneStreaming)

FChaosVDSceneStreaming::FChaosVDSceneStreaming()
{
	UpdateStreamingQueryShape();
	StreamingAccelerationStructure.SetTreeToDynamic();
}

FChaosVDSceneStreaming::~FChaosVDSceneStreaming()
{
}

void FChaosVDSceneStreaming::Initialize()
{
	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		Settings->OnSettingsChanged().AddRaw(this, &FChaosVDSceneStreaming::HandleSettingsChanged);

		HandleSettingsChanged(Settings);
	}
}

void FChaosVDSceneStreaming::DeInitialize()
{
	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		Settings->OnSettingsChanged().RemoveAll(this);
	}

	Reset();
}

bool FChaosVDSceneStreaming::Tick(float DeltaTime)
{
	if (!bIsStreamingSystemEnabled)
	{
		return true;
	}

	if (bProcessPendingOperationsQueueInWorkerThread && CurrentProcessingTaskHandle.IsCompleted() && !PendingOperationsPerParticle.IsEmpty())
	{
		CurrentProcessingTaskHandle = UE::Tasks::Launch(TEXT("UpdatingCVDStreamingAccel"), [this]()
		{
			ProcessPendingOperations();
		});
	}

	if (!bProcessPendingOperationsQueueInWorkerThread)
	{
		ProcessPendingOperations();
	}

	{
		bool bHasDirtyFlags = false;
		{
			FReadScopeLock ReadLock(DirtyFlagsLock);
			bHasDirtyFlags = DirtyFlags != EChaosVDStreamingDirtyFlags::None;
		}

		const bool bIsOverTheThreshold = LastStreamingLocationUpdate.IsSet() ? FVector::Distance(CurrentStreamingSourceLocation, LastStreamingLocationUpdate.GetValue()) > MovementThreshold : true;
		if (bIsOverTheThreshold || bHasDirtyFlags)
		{
			{
				FWriteScopeLock WriteLock(DirtyFlagsLock);
				DirtyFlags = EChaosVDStreamingDirtyFlags::None;
			}
			
			LastStreamingLocationUpdate = CurrentStreamingSourceLocation;
			UpdateStreamingState();
		}
	}

	return true;
}

void FChaosVDSceneStreaming::EnqueuePendingTrackingOperation(const FPendingTrackingOperation&& Operation)
{
	if (!bIsStreamingSystemEnabled)
	{
		return;
	}

	PendingOperationsPerParticle.Enqueue(Operation);
}

void FChaosVDSceneStreaming::EnqueuePendingTrackingOperation(const TSharedRef<FChaosVDBaseSceneObject>& InSceneObject, FPendingTrackingOperation::EType Type)
{
	if (!bIsStreamingSystemEnabled)
	{
		return;
	}

	PendingOperationsPerParticle.Enqueue(CreateStreamingTrackingOperation(InSceneObject, FPendingTrackingOperation::EType::AddOrUpdate));
}

void FChaosVDSceneStreaming::UpdateStreamingSourceLocation(const FVector& NewLocation)
{
	CurrentStreamingSourceLocation = NewLocation;
	SteamingViewBox = SteamingViewBox.MoveTo(NewLocation);
}

bool FChaosVDSceneStreaming::IsInStreamingRange(const FBox& Bounds) const
{
	if (bIsStreamingSystemEnabled)
	{
		return Bounds.IsValid ? SteamingViewBox.Intersect(Bounds) : false;
	}

	return true;
}

void FChaosVDSceneStreaming::UpdateStreamingStateForObject(const TSharedRef<FChaosVDBaseSceneObject>& InSceneObject) const
{
	if (IsInStreamingRange(InSceneObject->GetStreamingBounds()))
	{
		if (InSceneObject->GetStreamingState() == FChaosVDSceneParticle::EStreamingState::Hidden)
		{
			InSceneObject->SetStreamingState(FChaosVDSceneParticle::EStreamingState::Visible);
		}
	}
	else
	{
		if (InSceneObject->GetStreamingState() == FChaosVDSceneParticle::EStreamingState::Visible)
		{
			InSceneObject->SetStreamingState(FChaosVDSceneParticle::EStreamingState::Hidden);
		}
	}
}

FChaosVDSceneStreaming::FPendingTrackingOperation FChaosVDSceneStreaming::CreateStreamingTrackingOperation(const TSharedRef<FChaosVDBaseSceneObject>& InSceneObject, FPendingTrackingOperation::EType Type)
{
	return {.Bounds =  InSceneObject->GetStreamingBounds(), .ObjectID = InSceneObject->GetStreamingID(), .OperationType = Type };
}

void FChaosVDSceneStreaming::Reset()
{
	ensure(IsInGameThread());

	constexpr double TimeoutSeconds = 10.0;
	if (!ensure(CurrentProcessingTaskHandle.Wait(FTimespan::FromSeconds(TimeoutSeconds))))
	{
		UE_LOGF(LogChaosVDEditor, Error, "Failed to cancel pending operations processing queue after [%f]", TimeoutSeconds)
	}

	{
		FWriteScopeLock WriteLock(StreamingAccelerationOperationsLock);
		StreamingAccelerationStructure.Reset();
		StreamingAccelerationStructure.SetTreeToDynamic();
	}
}

void FChaosVDSceneStreaming::SetStreamingDataSource(IChaosVDStreamingDataSource* InStreamingDataSource)
{
	StreamingDataSource = InStreamingDataSource;
}

void FChaosVDSceneStreaming::SetStreamingEnabled(bool bNewEnabled)
{
	if (bIsStreamingSystemEnabled != bNewEnabled)
	{
		if (bNewEnabled)
		{
			ReBuildAccelStructureFromSourceDataArray();
			
			SetDirtyFlags(EChaosVDStreamingDirtyFlags::StreamingEnabled);
		}
		else
		{
			PendingOperationsPerParticle.Empty();
			MakeEverythingVisible();
		}

		bIsStreamingSystemEnabled = bNewEnabled;
	}
}

void FChaosVDSceneStreaming::HandleSettingsChanged(UObject* SettingsObject)
{
	if (UChaosVDGeneralSettings* Settings = Cast<UChaosVDGeneralSettings>(SettingsObject))
	{
		if (StreamingExtent != Settings->StreamingBoxExtentSize)
		{
			SetStreamingExtent(Settings->StreamingBoxExtentSize);
		}

		if (bProcessPendingOperationsQueueInWorkerThread && !Settings->bProcessPendingOperationsQueueInWorkerThread)
		{
			CurrentProcessingTaskHandle.Wait();
		}

		bProcessPendingOperationsQueueInWorkerThread = Settings->bProcessPendingOperationsQueueInWorkerThread;
		
		SetStreamingEnabled(Settings->bStreamingSystemEnabled);
	}
}

void FChaosVDSceneStreaming::SetDirtyFlags(EChaosVDStreamingDirtyFlags Flag)
{
	FWriteScopeLock WriteScopeLock(DirtyFlagsLock);
	EnumAddFlags(DirtyFlags, Flag);
}

void FChaosVDSceneStreaming::RemoveDirtyFlag(EChaosVDStreamingDirtyFlags Flag)
{
	FWriteScopeLock WriteScopeLock(DirtyFlagsLock);
	EnumRemoveFlags(DirtyFlags, Flag);
}

void FChaosVDSceneStreaming::ProcessPendingOperations()
{
	while (!PendingOperationsPerParticle.IsEmpty())
	{
		FPendingTrackingOperation PendingOperation;
		PendingOperationsPerParticle.Dequeue(PendingOperation);

		{
			FWriteScopeLock WriteLock(StreamingAccelerationOperationsLock);
			if (PendingOperation.OperationType == FPendingTrackingOperation::EType::AddOrUpdate)
			{
				StreamingAccelerationStructure.UpdateElement(PendingOperation.ObjectID, Chaos::TAABB<double,3>(PendingOperation.Bounds.Min, PendingOperation.Bounds.Max), true);
			}
			else if (PendingOperation.OperationType == FPendingTrackingOperation::EType::Remove)
			{
				StreamingAccelerationStructure.RemoveElement(PendingOperation.ObjectID);
			}
		}

		SetDirtyFlags(EChaosVDStreamingDirtyFlags::AccelerationStructure);
	}
}

void FChaosVDSceneStreaming::UpdateStreamingState()
{
	if (!StreamingDataSource)
	{
		return;
	}

	FMemMark StackMarker(FMemStack::Get());
	TSet<int32> InStreamingRangeObjects;
	{
		FQueryVisitor Collector(InStreamingRangeObjects);
		FReadScopeLock ReadLock(StreamingAccelerationOperationsLock);
		StreamingAccelerationStructure.Overlap(Chaos::TAABB<double,3>(SteamingViewBox.Min, SteamingViewBox.Max), Collector);
	}

	TQueue<TSharedPtr<FChaosVDBaseSceneObject>, EQueueMode::Mpsc> ObjectsThatNeedStreamingStateUpdate;

	{
		TConstArrayView<TSharedRef<FChaosVDBaseSceneObject>> StreamableObjectsArray = StreamingDataSource->GetStreamableSceneObjects();
		FReadScopeLock ObjectsReadLock(StreamingDataSource->GetObjectsLock());

		ParallelFor(StreamableObjectsArray.Num(),[&StreamableObjectsArray, &InStreamingRangeObjects, &ObjectsThatNeedStreamingStateUpdate, this](int32 Index)
		{
			const TSharedRef<FChaosVDBaseSceneObject>& SceneObject = StreamableObjectsArray[Index];

			if (InStreamingRangeObjects.Contains(SceneObject->GetStreamingID()))
			{
				if (SceneObject->GetStreamingState() == FChaosVDSceneParticle::EStreamingState::Hidden)
				{
					SceneObject->SetStreamingState(FChaosVDSceneParticle::EStreamingState::Visible);
					ObjectsThatNeedStreamingStateUpdate.Enqueue(SceneObject);
				}
			}
			else
			{
				if (SceneObject->GetStreamingState() == FChaosVDSceneParticle::EStreamingState::Visible)
				{
					SceneObject->SetStreamingState(FChaosVDSceneParticle::EStreamingState::Hidden);
					ObjectsThatNeedStreamingStateUpdate.Enqueue(SceneObject);
				}
			}
		});
	}

	bool bWasStreamingUpdated = !ObjectsThatNeedStreamingStateUpdate.IsEmpty();

	while (!ObjectsThatNeedStreamingStateUpdate.IsEmpty())
	{
		TSharedPtr<FChaosVDBaseSceneObject> SceneObject;
		ObjectsThatNeedStreamingStateUpdate.Dequeue(SceneObject);

		if (SceneObject)
		{
			SceneObject->SyncStreamingState();
		}
	}

	if (bWasStreamingUpdated)
	{
		if (TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
		{
			ScenePtr->RequestUpdate();
		}
	}
}

void FChaosVDSceneStreaming::SetStreamingExtent(float NewExtent)
{
	StreamingExtent = NewExtent;
	UpdateStreamingQueryShape();
}

void FChaosVDSceneStreaming::UpdateStreamingQueryShape()
{
	SteamingViewBox = FBox(ForceInitToZero).ExpandBy(StreamingExtent).MoveTo(CurrentStreamingSourceLocation);

	SetDirtyFlags(EChaosVDStreamingDirtyFlags::StreamingExtents);
}

void FChaosVDSceneStreaming::ReBuildAccelStructureFromSourceDataArray()
{
	Reset();

	if (!StreamingDataSource)
	{
		return;
	}

	{
		TConstArrayView<TSharedRef<FChaosVDBaseSceneObject>> StreamableObjectsArray = StreamingDataSource->GetStreamableSceneObjects();
		FReadScopeLock ObjectsReadLock(StreamingDataSource->GetObjectsLock());

		FWriteScopeLock AccelStructLock(StreamingAccelerationOperationsLock);
		for (const TSharedRef<FChaosVDBaseSceneObject>& SceneObject : StreamableObjectsArray)
		{
			FBox ObjectStreamingBounds = SceneObject->GetStreamingBounds();
			StreamingAccelerationStructure.UpdateElement(SceneObject->GetStreamingID(), Chaos::TAABB<double, 3>(ObjectStreamingBounds.Min, ObjectStreamingBounds.Max), true);
		}
	}

	if (TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->RequestUpdate();
	}
}

void FChaosVDSceneStreaming::MakeEverythingVisible()
{
	Reset();

	if (!StreamingDataSource)
	{
		return;
	}

	{
		TConstArrayView<TSharedRef<FChaosVDBaseSceneObject>> StreamableObjectsArray = StreamingDataSource->GetStreamableSceneObjects();
		FReadScopeLock ObjectsReadLock(StreamingDataSource->GetObjectsLock());
		for (const TSharedRef<FChaosVDBaseSceneObject>& SceneObject : StreamableObjectsArray)
		{
			if (SceneObject->GetStreamingState() != FChaosVDSceneParticle::EStreamingState::Visible)
			{
				SceneObject->SetStreamingState(FChaosVDBaseSceneObject::EStreamingState::Visible);
			}

			SceneObject->SyncStreamingState();
		}
	}

	if (TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->RequestUpdate();
	}
}
