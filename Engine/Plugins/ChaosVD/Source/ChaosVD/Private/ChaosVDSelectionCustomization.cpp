// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSelectionCustomization.h"

#include "ChaosVDScene.h"

bool FChaosVDSelectionCustomization::DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	const bool bSelectionChanged = FTypedElementSelectionCustomization::DeselectElement(InElementSelectionHandle, InSelectionSet, InSelectionOptions);

	if (TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->HandleDeSelectElement(InElementSelectionHandle, InSelectionSet, InSelectionOptions);
	}

	return bSelectionChanged;
}


bool FChaosVDSelectionCustomization::SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions)
{
	const bool bSelectionChanged = FTypedElementSelectionCustomization::SelectElement(InElementSelectionHandle, InSelectionSet, InSelectionOptions);

	// Currently we need to know if an element was tried to be de-selected or selected regardless if the selection set changed, in case we are re-selecting the object but in a different way (like selecting a particle but to show a specific geometry instance)
	if (TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->HandleSelectElement(InElementSelectionHandle, InSelectionSet, InSelectionOptions);
	}

	return bSelectionChanged;
}
