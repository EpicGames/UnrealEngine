// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Templates/SharedPointer.h"

class FChaosVDScene;

/** Customization class used to intercept selection actions and route Selection Events into CVD's Scene*/
class FChaosVDSelectionCustomization : public FTypedElementSelectionCustomization
{
public:
	FChaosVDSelectionCustomization(const TWeakPtr<FChaosVDScene>& InScene) : SceneWeakPtr(InScene)
	{
	}
	
	virtual bool DeselectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;
	virtual bool SelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions) override;

protected:
	TWeakPtr<FChaosVDScene> SceneWeakPtr;
};
