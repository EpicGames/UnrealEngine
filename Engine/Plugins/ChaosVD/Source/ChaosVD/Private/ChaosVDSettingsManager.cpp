// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSettingsManager.h"

#include "Misc/ConfigContext.h"
#include "Misc/LazySingleton.h"
#include "Settings/ChaosVDCoreSettings.h"

FChaosVDSettingsManager::FChaosVDSettingsManager()
{
	SettingsOuter = NewObject<UChaosVDSettingsObjectsOuter>();
}

void FChaosVDSettingsManager::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(SettingsOuter);
	Collector.AddStableReferenceMap(AvailableSettingsObject);
}

FString FChaosVDSettingsManager::GetReferencerName() const
{
	return TEXT("FChaosVDSettingsManager");
}

FChaosVDSettingsManager& FChaosVDSettingsManager::Get()
{
	return TLazySingleton<FChaosVDSettingsManager>::Get();
}

FChaosVDSettingsManager* FChaosVDSettingsManager::TryGet()
{
	return TLazySingleton<FChaosVDSettingsManager>::TryGet();
}

void FChaosVDSettingsManager::TearDown()
{
	TLazySingleton<FChaosVDSettingsManager>::TearDown();
}

UChaosVDSettingsObjectBase* FChaosVDSettingsManager::GetSettingsObject(UClass* SettingsClass)
{
	if (TObjectPtr<UChaosVDSettingsObjectBase>* ObjectPtrPtr = AvailableSettingsObject.Find(SettingsClass))
	{
		return ObjectPtrPtr->Get();
	}

	UChaosVDSettingsObjectBase* NewSettingsObject = NewObject<UChaosVDSettingsObjectBase>(SettingsOuter, SettingsClass);

	// Needed so any changes are added to the undo buffer
	NewSettingsObject->SetFlags(RF_Transactional);

	AvailableSettingsObject.Add(SettingsClass, NewSettingsObject);

	return NewSettingsObject;
}

void FChaosVDSettingsManager::ResetSettings(UClass* SettingsClass)
{
	UChaosVDSettingsObjectBase* SettingsObject = GetSettingsObject(SettingsClass);
	if (ensure(GConfig && SettingsClass && SettingsObject))
	{
		FString ConfigFile = SettingsClass->GetConfigName();
		FStringView Section = SettingsObject->GetConfigSectionName();

		GConfig->EmptySection(Section.GetData(), ConfigFile);
		GConfig->Flush(false);

		FConfigContext::ForceReloadIntoGConfig().Load(*FPaths::GetBaseFilename(ConfigFile));
	
		RestoreConfigPropertiesValuesFromCDO(SettingsObject);

		SettingsObject->BroadcastSettingsChanged();
	}
}

void FChaosVDSettingsManager::RestoreConfigPropertiesValuesFromCDO(UChaosVDSettingsObjectBase* TargetSettingsObject)
{
	const UChaosVDSettingsObjectBase* CDOSettingsObject = TargetSettingsObject ? GetDefault<UChaosVDSettingsObjectBase>(TargetSettingsObject->GetClass()) : nullptr;
	if (!ensure(CDOSettingsObject && TargetSettingsObject))
	{
		return;
	}

	for (FProperty* Property = CDOSettingsObject->GetClass()->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		if (!Property->HasAnyPropertyFlags(CPF_Config) )
		{
			continue;
		}

		const void* CDOPropertyAddress = Property->ContainerPtrToValuePtr<void>(CDOSettingsObject);
		void* TargetPropertyAddr = Property->ContainerPtrToValuePtr<void>(TargetSettingsObject);

		Property->CopyCompleteValue(TargetPropertyAddr, CDOPropertyAddress);
	}
}
