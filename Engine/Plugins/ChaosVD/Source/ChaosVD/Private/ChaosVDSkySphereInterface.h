// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ChaosVDSkySphereInterface.generated.h"

class ADirectionalLight;

UINTERFACE()
class UChaosVDSkySphereInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Simple interface to allow update the light source in the BP SkySphere from C++
 */
class IChaosVDSkySphereInterface
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintNativeEvent, CallInEditor)
	void SetDirectionalLightSource(ADirectionalLight* NewLightSource);

	UFUNCTION(BlueprintNativeEvent, CallInEditor)
	void Refresh();
};
