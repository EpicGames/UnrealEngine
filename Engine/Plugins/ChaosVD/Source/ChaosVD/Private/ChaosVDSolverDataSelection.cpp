// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSolverDataSelection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSolverDataSelection)

void FChaosVDSolverDataSelectionHandle::SetOwner(const TSharedPtr<FChaosVDSolverDataSelection>& InOwner)
{
	Owner = InOwner;
}

bool FChaosVDSolverDataSelectionHandle::IsSelected()
{
	if (TSharedPtr<FChaosVDSolverDataSelection> OwnerPtr = Owner.Pin())
	{
		return OwnerPtr->IsSelectionHandleSelected(AsShared());
	}

	return false;
}

bool FChaosVDSolverDataSelectionHandle::IsValid() const
{
	return SelectedDataStruct.IsValid() && SelectedDataStruct->IsValid();
}

void FChaosVDSolverDataSelection::SelectData(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle)
{
	CurrentSelectedSolverDataHandle = InSelectionHandle;

	SolverDataSelectionChangeDelegate.Broadcast(CurrentSelectedSolverDataHandle);
}

bool FChaosVDSolverDataSelection::IsSelectionHandleSelected(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle) const
{
	const bool bBothHandlesAreValid = CurrentSelectedSolverDataHandle && InSelectionHandle;

	if (!bBothHandlesAreValid)
	{
		return false;
	}

	return (*CurrentSelectedSolverDataHandle) == (*InSelectionHandle);
}
