// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSolversTracksTab.h"

#include "ChaosVDEngine.h"
#include "ChaosVDStyle.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Widgets/SChaosVDSolverTracks.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

TSharedRef<SDockTab> FChaosVDSolversTracksTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SolverTracksTab =
	SNew(SDockTab)
	.TabRole(ETabRole::PanelTab)
	.Label(LOCTEXT("SolverTracksTabLabel", "Solver Tracks"))
	.ToolTipText(LOCTEXT("SolverTracksTabToolTip", "Playback controls for the available solvers on the current Frame"));

	if (const TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin())
	{
		SolverTracksTab->SetContent
		(
			SAssignNew(SolverTracksWidget, SChaosVDSolverTracks, MainTabPtr->GetChaosVDEngineInstance()->GetPlaybackController(), OwningTabWidget)
		);
	}
	else
	{
		SolverTracksTab->SetContent(GenerateErrorWidget());
	}

	SolverTracksTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("TabIconSolverTracks"));

	HandleTabSpawned(SolverTracksTab);

	return SolverTracksTab;
}

void FChaosVDSolversTracksTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	SolverTracksWidget.Reset();
}

#undef LOCTEXT_NAMESPACE
