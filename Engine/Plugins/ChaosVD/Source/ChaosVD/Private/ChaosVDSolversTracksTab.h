// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDTabSpawnerBase.h"

class SChaosVDSolverTracks;

/** Spawns and handles and instance for the visual debugger Solvers Tracks tab
 * Which provides playback controls for each recorded solver
 */
class FChaosVDSolversTracksTab : public FChaosVDTabSpawnerBase, public TSharedFromThis<FChaosVDSolversTracksTab>
{
public:
	FChaosVDSolversTracksTab(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget) : FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{	
	}
	
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;

	virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

protected:
	TSharedPtr<SChaosVDSolverTracks> SolverTracksWidget;
};
