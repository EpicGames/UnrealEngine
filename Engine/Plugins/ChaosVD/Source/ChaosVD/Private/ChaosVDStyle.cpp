// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Interfaces/IPluginManager.h"
#include "Slate/SlateGameResources.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleMacros.h"
#include "Styling/SlateStyleRegistry.h"

#define RootToContentDir Style->RootToContentDir

TSharedPtr<FSlateStyleSet> FChaosVDStyle::StyleInstance = nullptr;

void FChaosVDStyle::Initialize()
{
	if (!StyleInstance.IsValid())
	{
		StyleInstance = Create();
		FSlateStyleRegistry::RegisterSlateStyle(*StyleInstance);
	}
}

void FChaosVDStyle::Shutdown()
{
	FSlateStyleRegistry::UnRegisterSlateStyle(*StyleInstance);
	ensure(StyleInstance.IsUnique());
	StyleInstance.Reset();
}

FName FChaosVDStyle::GetStyleSetName()
{
	static FName StyleSetName(TEXT("ChaosVDStyle"));
	return StyleSetName;
}

const FVector2D Icon16x16(16.0f, 16.0f);
const FVector2D Icon20x20(20.0f, 20.0f);
const FVector2D Icon24x24(24.0f, 24.0f);
const FVector2D Icon50x50(50.0f, 50.0f);

TSharedRef< FSlateStyleSet > FChaosVDStyle::Create()
{
	TSharedRef< FSlateStyleSet > Style = MakeShareable(new FSlateStyleSet("ChaosVDStyle"));
	Style->SetContentRoot(IPluginManager::Get().FindPlugin("ChaosVD")->GetBaseDir() / TEXT("Resources"));

	Style->Set("ChaosVD.OpenPluginWindow", new IMAGE_BRUSH_SVG(TEXT("PlaceholderButtonIcon"), Icon20x20));

	Style->Set("PlayIcon", new IMAGE_BRUSH_SVG(TEXT("PlayControlsPlayForward"), Icon16x16));
	Style->Set("StopIcon",  new IMAGE_BRUSH_SVG(TEXT("PlayControlsStop"), Icon16x16));
	Style->Set("NextIcon",  new IMAGE_BRUSH_SVG(TEXT("PlayControlsToNext"), Icon16x16));
	Style->Set("PrevIcon",  new IMAGE_BRUSH_SVG( TEXT("PlayControlsToPrevious"), Icon16x16));
	Style->Set("PauseIcon",  new IMAGE_BRUSH_SVG( TEXT("PlayControlsPause"), Icon16x16));
	
	Style->Set("TabIconDetailsPanel",  new IMAGE_BRUSH_SVG(TEXT("Details"), Icon16x16));
	Style->Set("TabIconPlaybackViewport",  new IMAGE_BRUSH_SVG(TEXT("ChaosVisualDebugger_16"), Icon16x16));
	Style->Set("TabIconWorldOutliner", new IMAGE_BRUSH_SVG(TEXT("WorldOutliner"), Icon16x16));
	Style->Set("TabIconOutputLog", new IMAGE_BRUSH_SVG(TEXT("OutputLog"), Icon16x16));
	Style->Set("TabIconSolverTracks", new IMAGE_BRUSH_SVG(TEXT("ChaosSolverTrack_16"), Icon16x16));

	Style->Set("ChaosVisualDebugger", new IMAGE_BRUSH_SVG(TEXT("ChaosVisualDebugger_16"), Icon16x16));
	Style->Set("OpenFileIcon", new IMAGE_BRUSH_SVG(TEXT("File_20"), Icon20x20));
	Style->Set("OpenSessionIcon", new IMAGE_BRUSH_SVG(TEXT("Session_20"), Icon20x20));
	Style->Set("SceneQueriesInspectorIcon", new IMAGE_BRUSH_SVG(TEXT("ChaosSceneQueries_16"), Icon16x16));
	
	Style->Set("LockIcon", new IMAGE_BRUSH_SVG(TEXT("lock"), Icon16x16));
	Style->Set("UnlockedIcon", new IMAGE_BRUSH_SVG(TEXT("lock-unlocked"), Icon16x16));
	Style->Set("RecordIcon", new IMAGE_BRUSH_SVG(TEXT("PlayControlsRecord"), Icon16x16));
	Style->Set("RecordToFileIcon", new IMAGE_BRUSH(TEXT("RecordToFile"), Icon16x16));
	Style->Set("RecordToLiveIcon", new IMAGE_BRUSH(TEXT("RecordLiveSession"), Icon16x16));
	Style->Set("ConnectionIcon", new IMAGE_BRUSH_SVG(TEXT("Connection"), Icon16x16));

	Style->Set("LinkedIcon", new IMAGE_BRUSH_SVG(TEXT("Linked"), Icon16x16));
	Style->Set("UnLinkedIcon", new IMAGE_BRUSH_SVG(TEXT("UnLinked"), Icon16x16));

	Style->Set("RigidBodyIcon", new IMAGE_BRUSH(TEXT("RigidBodyIcon_16x"), Icon16x16));
	Style->Set("SolverIcon", new IMAGE_BRUSH(TEXT("SolverIcon_16x"), Icon16x16));

	return Style;
}

void FChaosVDStyle::ReloadTextures()
{
	if (FSlateApplication::IsInitialized())
	{
		FSlateApplication::Get().GetRenderer()->ReloadTextureResources();
	}
}

const ISlateStyle& FChaosVDStyle::Get()
{
	return *StyleInstance;
}
