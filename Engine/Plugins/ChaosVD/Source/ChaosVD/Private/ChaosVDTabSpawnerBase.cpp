// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDTabSpawnerBase.h"

#include "ChaosVDEngine.h"
#include "ChaosVDScene.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDTabSpawnerBase::FChaosVDTabSpawnerBase(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget)
{
	TabID = InTabID;
	OwningTabWidget = InOwningTabWidget;
	InTabManager->RegisterTabSpawner(InTabID, FOnSpawnTab::CreateRaw(this, &FChaosVDTabSpawnerBase::HandleTabSpawnRequest));
}

void FChaosVDTabSpawnerBase::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	OnTabDestroyed().Broadcast(InTabClosed);
	InTabClosed->SetOnTabClosed(SDockTab::FOnTabClosedCallback());
}

void FChaosVDTabSpawnerBase::HandleTabSpawned(TSharedRef<SDockTab> InTabSpawned)
{
	OnTabSpawned().Broadcast(InTabSpawned);
	InTabSpawned->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateRaw(this, &FChaosVDTabSpawnerBase::HandleTabClosed));
}

TSharedRef<SWidget> FChaosVDTabSpawnerBase::GenerateErrorWidget()
{
	return SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("ChaosVDEditorTabSpawner", "Failed to generate Tab content"))
		];
}

UWorld* FChaosVDTabSpawnerBase::GetChaosVDWorld() const
{
	if (TSharedPtr<FChaosVDScene> ScenePtr = GetChaosVDScene().Pin())
	{
		return ScenePtr->GetUnderlyingWorld();
	}

	return nullptr;
}

TWeakPtr<FChaosVDScene> FChaosVDTabSpawnerBase::GetChaosVDScene() const
{
	const TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin();
	if (ensure(MainTabPtr))
	{
		const TSharedRef<FChaosVDEngine> ChaosVDEngine = MainTabPtr->GetChaosVDEngineInstance();
		if (TSharedPtr<FChaosVDScene> ScenePtr = ChaosVDEngine->GetCurrentScene())
		{
			return ScenePtr;
		}
	}
	return nullptr;
}

#undef LOCTEXT_NAMESPACE
