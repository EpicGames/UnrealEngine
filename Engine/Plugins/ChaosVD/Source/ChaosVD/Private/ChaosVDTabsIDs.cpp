// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDTabsIDs.h"

const FName FChaosVDTabID::ChaosVisualDebuggerTab = TEXT("ChaosVisualDebuggerTab");
const FName FChaosVDTabID::PlaybackViewport = TEXT("PlaybackViewport");
const FName FChaosVDTabID::WorldOutliner = TEXT("WorldOutliner");
const FName FChaosVDTabID::DetailsPanel = TEXT("DetailsPanel");
const FName FChaosVDTabID::IndependentDetailsPanel1 = TEXT("IndependentDetailsPanel1");
const FName FChaosVDTabID::IndependentDetailsPanel2 = TEXT("IndependentDetailsPanel2");
const FName FChaosVDTabID::IndependentDetailsPanel3 = TEXT("IndependentDetailsPanel3");
const FName FChaosVDTabID::IndependentDetailsPanel4 = TEXT("IndependentDetailsPanel4");
const FName FChaosVDTabID::OutputLog = TEXT("ChaosVDOutputLog");
const FName FChaosVDTabID::SolversTrack = TEXT("SolversTrack");
const FName FChaosVDTabID::StatusBar = TEXT("StatusBar");
const FName FChaosVDTabID::CollisionDataDetails = TEXT("CollisionDataDetails");
const FName FChaosVDTabID::SceneQueryDataDetails = TEXT("SceneQueryDataDetails");
const FName FChaosVDTabID::ConstraintsInspector = TEXT("ConstraintsDetails");
const FName FChaosVDTabID::SceneQueryBrowser = TEXT("SceneQueryBrowser");
const FName FChaosVDTabID::RecordedOutputLog = TEXT("RecordedOutputLog");