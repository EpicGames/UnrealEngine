// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDTriMeshGenerator.h"
#include "Chaos/TriangleMeshImplicitObject.h"

void FChaosVDTriMeshGenerator::GenerateFromTriMesh(const Chaos::FTriangleMeshImplicitObject& InTriMesh)
{
	const int32 NumTriangles = InTriMesh.Elements().GetNumTriangles();
	const int32 NumVertices = InTriMesh.Particles().Size();

	const int32 NormalsNum = NumTriangles * 3;
	constexpr int32 UVsNum = 0;
	SetBufferSizes(NumVertices, NumTriangles , UVsNum, NormalsNum);

	EParallelForFlags Flags = NumVertices > MaxElementsNumToProcessInSingleThread ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None;
	// Fill the vertex buffer with the transformed vertices of the TriMesh Shape
	ParallelFor(NumVertices, [this, &InTriMesh](int32 VertexIndex)
	{
		Vertices[VertexIndex] = FVector3d(UE::Math::TVector<double>(InTriMesh.Particles().GetX(VertexIndex)));
	}, Flags);

	const Chaos::FTrimeshIndexBuffer& IdxBuffer = InTriMesh.Elements();
	if (IdxBuffer.RequiresLargeIndices())
	{
		ProcessTriangles(IdxBuffer.GetLargeIndexBuffer(), NumTriangles, InTriMesh);
	}
	else
	{
		ProcessTriangles(IdxBuffer.GetSmallIndexBuffer(), NumTriangles, InTriMesh);
	}
	
	bIsGenerated = true;
}

UE::Geometry::FMeshShapeGenerator& FChaosVDTriMeshGenerator::Generate()
{
	ensureAlwaysMsgf(bIsGenerated, TEXT("You need to call FChaosVDTriMeshGenerator::GenerateFromTriMesh before calling Generate"));
	return *this;
}
