// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Generators/MeshShapeGenerator.h"

namespace Chaos
{
	class FTriangleMeshImplicitObject;
}

/** Generates a Dynamic mesh based on a Triangle Mesh Implicit object*/
class FChaosVDTriMeshGenerator : public UE::Geometry::FMeshShapeGenerator
{

public:
	/** Triangle Mesh Implicit object used as data source to generate the dynamic mesh */
	void GenerateFromTriMesh(const Chaos::FTriangleMeshImplicitObject& InTriMesh);

	virtual FMeshShapeGenerator& Generate() override;

private:

	template<typename BufferIndexType>
	void ProcessTriangles(const TArray<BufferIndexType>& InTriangles, const int32 NumTriangles, const Chaos::FTriangleMeshImplicitObject& InTriMesh);

	bool bIsGenerated = false;

	/** Max number of elements after which we will process elements in parallel using worker threads
	 * Note: This value is not tuned yet
	 */
	static constexpr int32 MaxElementsNumToProcessInSingleThread = 64;
};

template <typename BufferIndexType>
void FChaosVDTriMeshGenerator::ProcessTriangles(const TArray<BufferIndexType>& InTriangles, const int32 NumTriangles, const Chaos::FTriangleMeshImplicitObject& InTriMesh)
{
	EParallelForFlags Flags = NumTriangles > MaxElementsNumToProcessInSingleThread ? EParallelForFlags::ForceSingleThread : EParallelForFlags::None;
	ParallelFor(NumTriangles, [this, &InTriangles, &InTriMesh](int32 TriangleIndex)
	{
		UE::Geometry::FIndex3i Triangle(InTriangles[TriangleIndex][0],InTriangles[TriangleIndex][1],InTriangles[TriangleIndex][2]);

		FVector3f FaceNormal(InTriMesh.GetFaceNormal(TriangleIndex));
		int32 StartNormalIndexNumber = TriangleIndex * 3;
		// Create a Normal entry per triangle vertex.
		for (int32 LocalVertexIndex = 0; LocalVertexIndex < 3; ++LocalVertexIndex)
		{
			// Use the normal of the face for all it's vertices
			Normals[StartNormalIndexNumber + LocalVertexIndex] = FaceNormal;
		}

		SetTrianglePolygon(TriangleIndex, TriangleIndex);
		SetTriangleNormals(TriangleIndex, StartNormalIndexNumber, StartNormalIndexNumber + 1, StartNormalIndexNumber + 2);
		SetTriangle(TriangleIndex, MoveTemp(Triangle));
	}, Flags);
}
