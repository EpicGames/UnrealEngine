// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDWorldOutlinerMode.h"

#include "ActorTreeItem.h"
#include "ChaosVDModule.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDScene.h"
#include "ChaosVDSettingsManager.h"
#include "TedsOutlinerItem.h"
#include "DataStorage/Features.h"
#include "Elements/Actor/ActorElementData.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/EngineElementsLibrary.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Settings/ChaosVDGeneralSettings.h"
#include "Styling/StyleColors.h"
#include "TEDS/ChaosVDParticleEditorDataFactory.h"
#include "TEDS/ChaosVDSelectionInterface.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "TEDS/ChaosVDTedsUtils.h"

struct FTypedElementScriptStructTypeInfoColumn;

namespace Chaos::VD::SceneOutlinerUtils::Private
{
	FChaosVDSceneParticle* GetParticleInstanceFromOutlinerItem(const TSharedRef<ISceneOutlinerTreeItem>& TreeItemRef)
	{
		using namespace UE::Editor::Outliner;
		using namespace UE::Editor::DataStorage;
		if (FTedsOutlinerTreeItem* TEDSItem = TreeItemRef->CastTo<FTedsOutlinerTreeItem>())
		{
			ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			if (TEDSItem->IsValid() && Storage)
			{
				const FTypedElementExternalObjectColumn* RawObjectColumn = Storage->GetColumn<FTypedElementExternalObjectColumn>(TEDSItem->GetRowHandle());
				const FTypedElementScriptStructTypeInfoColumn* TypeInfoColumn = Storage->GetColumn<FTypedElementScriptStructTypeInfoColumn>(TEDSItem->GetRowHandle());
				if (RawObjectColumn && TypeInfoColumn)
				{
					if (RawObjectColumn->Object && TypeInfoColumn->TypeInfo == FChaosVDSceneParticle::StaticStruct())
					{
						return static_cast<FChaosVDSceneParticle*>(RawObjectColumn->Object);
					}
				}
			}
		}
	
		return nullptr;
	}

	AActor* GetActorFromOutlinerItem(const TSharedRef<ISceneOutlinerTreeItem>& TreeItemRef)
	{
		using namespace UE::Editor::Outliner;
		using namespace UE::Editor::DataStorage;

		if (FTedsOutlinerTreeItem* TEDSItem = TreeItemRef->CastTo<FTedsOutlinerTreeItem>())
		{
			ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
			if (TEDSItem->IsValid() && Storage)
			{
				const FTypedElementUObjectColumn* RawObjectColumn = Storage->GetColumn<FTypedElementUObjectColumn>(TEDSItem->GetRowHandle());
				return RawObjectColumn ? Cast<AActor>(RawObjectColumn->Object) : nullptr;
			}
		}
	
		return nullptr;
	}

	AActor* GetActorFromOutlinerItem(const TWeakPtr<ISceneOutlinerTreeItem>& WeakTreeItem)
	{
		if (const TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakTreeItem.Pin())
		{
			return GetActorFromOutlinerItem(TreeItem.ToSharedRef());
		}
	
		return nullptr;
	}

	FChaosVDSceneParticle* GetParticleInstanceFromOutlinerItem(const TWeakPtr<ISceneOutlinerTreeItem>& WeakTreeItem)
	{
		if (const TSharedPtr<ISceneOutlinerTreeItem> TreeItem = WeakTreeItem.Pin())
		{
			return GetParticleInstanceFromOutlinerItem(TreeItem.ToSharedRef());
		}
		return nullptr;
	}
}

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

bool FChaosVDParticleOutlinerSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, FChaosVDSceneParticle*& ParticlePtrOut) const
{
	if (FChaosVDSceneParticle* ParticleInstance = Chaos::VD::SceneOutlinerUtils::Private::GetParticleInstanceFromOutlinerItem(Item))
	{
		ParticlePtrOut = ParticleInstance;
		return true;
	}

	return false;
}

bool FChaosVDActorOutlinerSelector::operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, AActor*& ActorPtrOut) const
{
	if (AActor* Actor = Chaos::VD::SceneOutlinerUtils::Private::GetActorFromOutlinerItem(Item))
	{
		ActorPtrOut = Actor;
		return true;
	}

	return false;
}

FChaosVDWorldOutlinerMode::FChaosVDWorldOutlinerMode(const UE::Editor::Outliner::FTedsOutlinerParams& InModeParams, TWeakPtr<FChaosVDScene> InScene, TWeakPtr<FChaosVDPlaybackController> InPlaybackController)
	: FTedsOutlinerMode(InModeParams)
	, CVDScene(InScene)
	, PlaybackController(InPlaybackController)
{
	TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin();
	if (!ensure(ScenePtr.IsValid()))
	{
		return;
	}

	if (SceneOutliner)
	{
		TAttribute<bool> ConditionalEnabledAttribute;
		ConditionalEnabledAttribute.BindRaw(this, &FChaosVDWorldOutlinerMode::CanPopulate);

		SceneOutliner->SetEnabled(ConditionalEnabledAttribute);
	}
	
	RegisterSelectionSetObject(ScenePtr->GetElementSelectionSet());

	if (UE::Editor::DataStorage::ICoreProvider* DataStorage = TedsOutlinerImpl ? TedsOutlinerImpl->GetStorage() : nullptr)
	{
		using namespace UE::Editor::DataStorage;
		using namespace UE::Editor::DataStorage::Queries;
		using namespace UE::Editor::Outliner;

		FQueryDescription RowAdditionQueryDescription =
			Select(
				TEXT("Add CVD Row to Outliner"),
				FObserver::OnAdd<FChaosVDActiveObjectTag>().SetExecutionMode(EExecutionMode::GameThread),
				[this](IQueryContext& Context, RowHandle Row)
				{
					RowsPendingAddition.Emplace(Row);
					RowsPendingRemoval.Remove(Row);
				})
			.Compile();

		CustomRowAdditionQueryHandle = DataStorage->RegisterQuery(MoveTemp(RowAdditionQueryDescription));
	
		// Row to track removal of rows from the Outliner
		FQueryDescription RowRemovalQueryDescription =
			Select(
					TEXT("Remove CVD Row from Outliner"),
					FObserver::OnRemove<FChaosVDActiveObjectTag>().SetExecutionMode(EExecutionMode::GameThread),
					[this](IQueryContext& Context, RowHandle Row)
					{
						RowsPendingAddition.Remove(Row);
						RowsPendingRemoval.Emplace(Row);
					})
				.Compile();

		CustomRowRemovalQueryHandle = DataStorage->RegisterQuery(MoveTemp(RowRemovalQueryDescription));

		// Use TEDS' update instead of a regular tick to make sure all processors have run and the data is correct
		DataStorage->OnUpdateCompleted().AddRaw(this, &FChaosVDWorldOutlinerMode::OnDataStorageUpdateCompleted);
	}

}

FChaosVDWorldOutlinerMode::~FChaosVDWorldOutlinerMode()
{
	if (SceneOutliner)
	{
		TAttribute<bool> EmptyConditionalEnabledAttribute;
		SceneOutliner->SetEnabled(EmptyConditionalEnabledAttribute);
	}

	if (UE::Editor::DataStorage::ICoreProvider* DataStorage = TedsOutlinerImpl ? TedsOutlinerImpl->GetStorage() : nullptr)
	{
		DataStorage->UnregisterQuery(CustomRowAdditionQueryHandle);
		DataStorage->UnregisterQuery(CustomRowRemovalQueryHandle);
		
		DataStorage->OnUpdateCompleted().RemoveAll(this);
	}
}

void FChaosVDWorldOutlinerMode::OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection)
{
	if (SelectionType == ESelectInfo::Direct)
	{
		return;
	}

	TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin();
	if (!ScenePtr.IsValid())
	{
		return;
	}

	using namespace Chaos::VD::TypedElementDataUtil;

	TArray<FChaosVDSceneParticle*> OutlinerSelectedParticles = Selection.GetData<FChaosVDSceneParticle*>(FChaosVDParticleOutlinerSelector());
	if (!OutlinerSelectedParticles.IsEmpty())
	{
		ScenePtr->SetSelected(AcquireTypedElementHandleForStruct(OutlinerSelectedParticles[0], true));
		return;
	}

	TArray<AActor*> OutlinerSelectedActors = Selection.GetData<AActor*>(FChaosVDActorOutlinerSelector());
	
	if (!OutlinerSelectedActors.IsEmpty())
	{
		ScenePtr->SetSelected(UEngineElementsLibrary::AcquireEditorActorElementHandle(OutlinerSelectedActors[0]));
		return;
	}

	ScenePtr->SetSelected(FTypedElementHandle());
}

void FChaosVDWorldOutlinerMode::OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item)
{
	TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin();
	if (!ScenePtr.IsValid())
	{
		return;
	}

	if (FChaosVDSceneParticle* ParticleInstance = Chaos::VD::SceneOutlinerUtils::Private::GetParticleInstanceFromOutlinerItem(Item))
	{
		ScenePtr->OnFocusRequest().Broadcast(ParticleInstance->GetBoundingBox());
	}
	else if (AActor* ActorItem = Chaos::VD::SceneOutlinerUtils::Private::GetActorFromOutlinerItem(Item))
	{
		ScenePtr->OnFocusRequest().Broadcast(ActorItem->GetComponentsBoundingBox(false));
	}
}

bool FChaosVDWorldOutlinerMode::CanInteract(const ISceneOutlinerTreeItem& Item) const
{
	// This option is not supported in CVD yet
	//ensure(!bCanInteractWithSelectableActorsOnly);

	return true;
}

bool FChaosVDWorldOutlinerMode::CanPopulate() const
{
	if (TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		const bool bIsContinuousPlayback = PlaybackControllerPtr->IsScrubbingTimeline() || PlaybackControllerPtr->IsPlaying();

		if (!bIsContinuousPlayback)
		{
			return true;
		}

		// Updating the scene outliner during playback it is very expensive and can tank framerate,
		// as it need to re-build the hierarchy when things are added in ad removed. So if we are playing we want to pause any updates to the outliner
		if (const UChaosVDGeneralSettings* GeneralSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
		{
			return GeneralSettings->bUpdateSceneOutlinerDuringPlayback && !PlaybackControllerPtr->IsScrubbingTimeline();
		}

		return false;	
	}

	return true;
}

void FChaosVDWorldOutlinerMode::HandlePostSelectionChange(const UTypedElementSelectionSet* ChangesSelectionSet)
{
	TArray<FTypedElementHandle> SelectedParticlesHandles = ChangesSelectionSet->GetSelectedElementHandles(UChaosVDSelectionInterface::StaticClass());

	if (SelectedParticlesHandles.Num() > 0)
	{
		// We don't support multi selection yet
		ensure(SelectedParticlesHandles.Num() == 1);
		
		using namespace UE::Editor::DataStorage;
		using namespace Chaos::VD::TypedElementDataUtil;
		
		if (TTypedElement<ITedsTypedElementBridgeInterface> BridgeInterface =  UTypedElementRegistry::GetInstance()->GetElement<ITedsTypedElementBridgeInterface>(SelectedParticlesHandles[0]))
		{
			UE::Editor::DataStorage::RowHandle RowHandle = BridgeInterface.GetRowHandle();
			if (FSceneOutlinerTreeItemPtr TreeItem = SceneOutliner->GetTreeItem(RowHandle, true))
			{
				SceneOutliner->ScrollItemIntoView(TreeItem);
			}
		}
	}
}

void FChaosVDWorldOutlinerMode::ProcessPendingActionsList(double TimeBudgetInSeconds, TSet<UE::Editor::DataStorage::RowHandle>& PendingItemsToProcess, EPendingOperationType Type)
{
	if (!TedsOutlinerImpl)
	{
		return;
	}

	const double StartTimeSeconds = FPlatformTime::Seconds();
	double CurrentTimeSpentSeconds = 0.0;
	int32 CurrentTasksProcessedNum = 0;

	FSceneOutlinerHierarchyChangedData EventData;
	EventData.Type = Type == EPendingOperationType::Add ? FSceneOutlinerHierarchyChangedData::Added : FSceneOutlinerHierarchyChangedData::Removed;

	for (TSet<UE::Editor::DataStorage::RowHandle>::TIterator PendingItemsIDIterator(PendingItemsToProcess); PendingItemsIDIterator; ++PendingItemsIDIterator)
	{
		// Only check the budget every 5 tasks as Getting the current time is a syscall and it is not free
		if (CurrentTasksProcessedNum % 5 == 0)
		{
			CurrentTimeSpentSeconds += FPlatformTime::Seconds() - StartTimeSeconds;

			if (CurrentTimeSpentSeconds > TimeBudgetInSeconds)
			{
				break;
			}
		}

		if (Type == EPendingOperationType::Add)
		{
			EventData.Items.Add(CreateItemFor<UE::Editor::Outliner::FTedsOutlinerTreeItem>(UE::Editor::Outliner::FTedsOutlinerTreeItem(*PendingItemsIDIterator, TedsOutlinerImpl.ToSharedRef())));
			
		}
		else
		{
			EventData.ItemIDs.Add(*PendingItemsIDIterator);
		}

		PendingItemsIDIterator.RemoveCurrent();
		CurrentTasksProcessedNum++;
	}

	GetHierarchy()->OnHierarchyChanged().Broadcast(MoveTemp(EventData));
}

void FChaosVDWorldOutlinerMode::OnDataStorageUpdateCompleted()
{
	if (!CanPopulate())
	{
		SceneOutliner->SetNextUIRefreshDelay(1.0f);
	}
	else
	{
		constexpr double TimeBudgetInSeconds = 0.002;
		ProcessPendingActionsList(TimeBudgetInSeconds, RowsPendingAddition, EPendingOperationType::Add);
		ProcessPendingActionsList(TimeBudgetInSeconds, RowsPendingRemoval, EPendingOperationType::Remove);
	}
}

FText FChaosVDWorldOutlinerMode::GetStatusText() const
{
	if (CanPopulate())
	{
		const int32 PendingUpdated = RowsPendingAddition.Num() + RowsPendingRemoval.Num();
		if (PendingUpdated > 0)
		{
			return FText::Format(LOCTEXT("SceneOutlinerUpdatingMessage", "Processing {0} changes..."), FText::AsNumber(PendingUpdated));
		}

		return FText::GetEmpty();
	}
	
	return LOCTEXT("SceneOutlinerDisabledWarning", "Outliner updates are disabled during playback or scrubbing");
}

FSlateColor FChaosVDWorldOutlinerMode::GetStatusTextColor() const
{
	return CanPopulate() ? FStyleColors::Foreground : FStyleColors::Warning;
}

#undef LOCTEXT_NAMESPACE
