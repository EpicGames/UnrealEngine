// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ActorBrowsingMode.h"
#include "ActorHierarchy.h"
#include "ActorTreeItem.h"
#include "ChaosVDScene.h"
#include "ChaosVDSceneSelectionObserver.h"
#include "SceneOutlinerGutter.h"
#include "TedsOutlinerMode.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"

class FChaosVDPlaybackController;
class FChaosVDScene;

/** Functor which can be used to get actors from a selection including component parents */
struct FChaosVDParticleOutlinerSelector
{
	FChaosVDParticleOutlinerSelector()
	{
	}

	bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, FChaosVDSceneParticle*& ParticlePtrOut) const;
};

struct FChaosVDActorOutlinerSelector
{
	FChaosVDActorOutlinerSelector()
	{
	}

	bool operator()(const TWeakPtr<ISceneOutlinerTreeItem>& Item, AActor*& ActorPtrOut) const;
};

/**
 * Scene outliner mode used to represent a CVD (Chaos Visual Debugger) world
 * It has a more limited view compared to the normal outliner, hiding features we don't support,
 * and it is integrated with the CVD local selection system
 */
class FChaosVDWorldOutlinerMode : public UE::Editor::Outliner::FTedsOutlinerMode, public FChaosVDSceneSelectionObserver
{
public:

	FChaosVDWorldOutlinerMode(const UE::Editor::Outliner::FTedsOutlinerParams& InModeParams, TWeakPtr<FChaosVDScene> InScene, TWeakPtr<FChaosVDPlaybackController> InPlaybackController);

	virtual ~FChaosVDWorldOutlinerMode() override;

	virtual void OnItemSelectionChanged(FSceneOutlinerTreeItemPtr Item, ESelectInfo::Type SelectionType, const FSceneOutlinerItemSelection& Selection) override;
	virtual void OnItemDoubleClick(FSceneOutlinerTreeItemPtr Item) override;

	virtual void CreateViewContent(FMenuBuilder& MenuBuilder) override
	{
		// Intentionally not implemented, as we don't support the built in menu to switch worlds
	}

	virtual bool ShouldShowFolders() const override
	{
		return true;
	}

	virtual bool CanInteract(const ISceneOutlinerTreeItem& Item) const override;
	virtual bool CanPopulate() const override;


	virtual ESelectionMode::Type GetSelectionMode() const override
	{
		return ESelectionMode::Single;
	}
	virtual bool CanSupportDragAndDrop() const override
	{
		return false;
	}

	virtual TSharedPtr<SWidget> CreateContextMenu() override
	{
		return SNullWidget::NullWidget;
	}

	void OnDataStorageUpdateCompleted();

	virtual bool ShowStatusBar() const override
	{
		return true;
	}

	virtual FText GetStatusText() const override;

	virtual FSlateColor GetStatusTextColor() const override;

private:

	virtual void HandlePostSelectionChange(const UTypedElementSelectionSet* ChangesSelectionSet) override;

	enum class EPendingOperationType
	{
		Add,
		Remove
	};
	void ProcessPendingActionsList(double TimeBudgetInSeconds, TSet<UE::Editor::DataStorage::RowHandle>& PendingItemsToProcess, EPendingOperationType Type);

	TSet<UE::Editor::DataStorage::RowHandle> RowsPendingRemoval; 
	TSet<UE::Editor::DataStorage::RowHandle> RowsPendingAddition;

	UE::Editor::DataStorage::QueryHandle CustomRowAdditionQueryHandle;
	UE::Editor::DataStorage::QueryHandle CustomRowRemovalQueryHandle;

	TWeakPtr<FChaosVDScene> CVDScene;
	TWeakPtr<FChaosVDPlaybackController> PlaybackController;

	TMap<FSceneOutlinerTreeItemID, FSceneOutlinerHierarchyChangedData> PendingOutlinerEventsMap;
};
