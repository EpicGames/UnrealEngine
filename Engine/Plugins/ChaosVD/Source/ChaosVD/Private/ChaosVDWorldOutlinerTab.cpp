// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDWorldOutlinerTab.h"

#include "ChaosVDEngine.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDWorldOutlinerMode.h"
#include "ChaosVDStyle.h"
#include "Modules/ModuleManager.h"
#include "SceneOutlinerModule.h"
#include "SceneOutlinerPublicTypes.h"
#include "TedsOutlinerImpl.h"
#include "TedsOutlinerItem.h"
#include "Compatibility/SceneOutlinerTedsBridge.h"

#include "DataStorage/Features.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "TEDS/ChaosVDParentDataStorageFactory.h"
#include "Widgets/Docking/SDockTab.h"
#include "TEDS/ChaosVDParticleEditorDataFactory.h"
#include "Widgets/SChaosVDMainTab.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void FChaosVDWorldOutlinerTab::CreateWorldOutlinerWidget()
{
	TWeakPtr<FChaosVDPlaybackController> PlaybackController;
	if (TSharedPtr<SChaosVDMainTab> MainTabPtr =  OwningTabWidget.Pin())
	{
		PlaybackController = MainTabPtr->GetChaosVDEngineInstance()->GetPlaybackController();
	}

	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::Outliner;

	FQueryDescription OutlinerQueryDescription =
		Select()
			.ReadOnly<FTypedElementLabelColumn>()
		.Where((TColumn<FChaosVDObjectDataTag>() || TColumn<FTypedElementActorTag>()) && TColumn<FTypedElementFromCVDWorldTag>() && TColumn<FChaosVDActiveObjectTag>())
		.Compile();

	FSceneOutlinerInitializationOptions InitOptions;
	InitOptions.bShowHeaderRow = true;
	InitOptions.FilterBarOptions.bHasFilterBar = true;
	InitOptions.FilterBarOptions.bUseSharedSettings = false;
	InitOptions.bShowTransient = true;
	InitOptions.OutlinerIdentifier = "CVDTEDSOutliner";

	FTedsOutlinerParams Params(nullptr);
	Params.QueryDescription = OutlinerQueryDescription;
	Params.bShowRowHandleColumn = false;
	Params.bUseDefaultObservers = false; // CVD uses custom observers to time slice addition and removal currently
	
	const TMap<TWeakObjectPtr<const UScriptStruct>, FTedsOutlinerColumnParams> ColumnParams
	{
		{FVisibleInEditorColumn::StaticStruct(), FTedsOutlinerColumnParams(FTedsOutlinerColumnParams::EColumnPriorityGroup::Left)}
	};

	Params.ColumnDescription = FTedsOutlinerColumnDescription({FVisibleInEditorColumn::StaticStruct()}, ColumnParams);

	const FTedsOutlinerHierarchyData::FGetParentRowHandle RowHandleGetter = FTedsOutlinerHierarchyData::FGetParentRowHandle::CreateLambda([](const void* InColumnData)
	{
		if(const FTableRowParentColumn* ParentColumn = static_cast<const FTableRowParentColumn *>(InColumnData))
		{
			return ParentColumn->Parent;
		}

		return InvalidRowHandle;
	});

	const FTedsOutlinerHierarchyData::FGetChildrenRowsHandles ChildrenRowHandlesGetter = FTedsOutlinerHierarchyData::FGetChildrenRowsHandles::CreateLambda([](void* InColumnData)
	{
		if (FChaosVDTableRowParentColumn* ParentColumn = static_cast<FChaosVDTableRowParentColumn *>(InColumnData))
		{
			ParentColumn->Children.Reset(ParentColumn->ChildrenSet.Num());
			for (RowHandle Handle : ParentColumn->ChildrenSet)
			{
				ParentColumn->Children.Emplace(Handle);
			}
			return TArrayView<RowHandle>(ParentColumn->Children);
		}

		return TArrayView<RowHandle>();
	});

	const FTedsOutlinerHierarchyData::FSetParentRowHandle RowHandleSetter = FTedsOutlinerHierarchyData::FSetParentRowHandle::CreateLambda([](void* InColumnData,
		RowHandle InRowHandle)
		{
			if (FChaosVDTableRowParentColumn* ParentColumn = static_cast<FChaosVDTableRowParentColumn*>(InColumnData))
			{
				ParentColumn->ParentObject = InRowHandle;
			}
		});
		
	Params.HierarchyData = MakeShared<FTedsOutlinerLegacyHierarchyInterface>(FTedsOutlinerHierarchyData(FChaosVDTableRowParentColumn::StaticStruct(), RowHandleGetter, RowHandleSetter, ChildrenRowHandlesGetter));
	
	if (TSharedPtr<FChaosVDScene> CvdScene = GetChaosVDScene().Pin())
	{
		Params.SelectionSet = CvdScene->GetElementSelectionSet();
	}
	

	InitOptions.ModeFactory = FCreateSceneOutlinerMode::CreateLambda([this, PlaybackController, &Params](SSceneOutliner* Outliner)
	{
		Params.SceneOutliner = Outliner;

		// The mode is deleted by the Outliner when it is destroyed 
		return new FChaosVDWorldOutlinerMode(Params, GetChaosVDScene(), PlaybackController);
	});

	InitOptions.ColumnMap.Add(FSceneOutlinerBuiltInColumnTypes::Label(), FSceneOutlinerColumnInfo(ESceneOutlinerColumnVisibility::Visible, 10));
	
	TSharedRef<ISceneOutliner> TedsOutliner = SNew(SSceneOutliner, InitOptions);

	using namespace UE::Editor::DataStorage;
	ICoreProvider* Storage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	SceneOutlinerWidget = TedsOutliner;

	// TODO: There is an issue where the actors created with CVD's world, do not appear until the outliner does a new query (by searching something for example)
	// It seems to be a timing issue with TEDS. For now, lest just hack it by rebuilding the outliner in next tick. There should only be ~5 actors.
	DeferredOutlinerUpdateHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([WeakOutliner = SceneOutlinerWidget.ToWeakPtr()](float DeltaTime)
	{
		if (TSharedPtr<ISceneOutliner> OutlinerPtr = WeakOutliner.Pin())
		{
			OutlinerPtr->FullRefresh();
		}

		return false;
	}));
}

TSharedRef<SDockTab> FChaosVDWorldOutlinerTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	CreateWorldOutlinerWidget();

	TSharedRef<SDockTab> OutlinerTab =
		SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("Physics World Outliner", "Physics World Outliner"))
		.ToolTipText(LOCTEXT("PhysicsWorldOutlinerTabToolTip", "Hierarchy view of the physics objects by category"));
	
	OutlinerTab->SetContent
	(
		SceneOutlinerWidget.ToSharedRef()
	);

	OutlinerTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("TabIconWorldOutliner"));

	HandleTabSpawned(OutlinerTab);

	return OutlinerTab;
}

void FChaosVDWorldOutlinerTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	FTSTicker::RemoveTicker(DeferredOutlinerUpdateHandle);
	DeferredOutlinerUpdateHandle = FTSTicker::FDelegateHandle();

	FChaosVDTabSpawnerBase::HandleTabClosed(InTabClosed);

	SceneOutlinerWidget.Reset();
}

#undef LOCTEXT_NAMESPACE
