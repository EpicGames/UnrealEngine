// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDSceneSelectionObserver.h"
#include "ChaosVDTabSpawnerBase.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"

struct FChaosVDOutlinerItem;

class AActor;
class ISceneOutliner;
class FChaosVDScene;
class FSpawnTabArgs;
class FTabManager;
class SChaosVDSceneOutliner;
class SDockTab;

enum class EChaosVDParticleType : uint8;

DECLARE_DELEGATE_OneParam(FChaosVDSelectionChanged, AActor* SelectedActor)

/** Spawns and handles and instance for the visual debugger World Outliner */
class FChaosVDWorldOutlinerTab : public FChaosVDTabSpawnerBase, public TSharedFromThis<FChaosVDWorldOutlinerTab>
{
public:
	FChaosVDWorldOutlinerTab(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget) : FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{
	}

	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

private:
	void CreateWorldOutlinerWidget();

	TSharedPtr<ISceneOutliner> SceneOutlinerWidget;

	FTSTicker::FDelegateHandle DeferredOutlinerUpdateHandle;
};
