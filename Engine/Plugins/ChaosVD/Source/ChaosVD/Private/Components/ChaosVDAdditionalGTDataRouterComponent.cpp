// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDAdditionalGTDataRouterComponent.h"

#include "ChaosVDRecording.h"
#include "ChaosVDScene.h"
#include "Actors/ChaosVDSolverInfoActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDAdditionalGTDataRouterComponent)

void UChaosVDAdditionalGTDataRouterComponent::UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	if (!EnumHasAnyFlags(InSolverFrameData.GetAttributes(), EChaosVDSolverFrameAttributes::HasGTDataToReRoute))
	{
		return;
	}

	if (AChaosVDSolverInfoActor* Owner = Cast<AChaosVDSolverInfoActor>(GetOwner()))
	{
		if (TSharedPtr<FChaosVDScene> CVDScene = Owner->GetScene().Pin())
		{
			if (TSharedPtr<FChaosVDGameFrameDataWrapper> GTFrameDataWrapper = InSolverFrameData.GetCustomData().GetData<FChaosVDGameFrameDataWrapper>())
			{
				CVDScene->OnSceneUpdated().AddUObject(this, &UChaosVDAdditionalGTDataRouterComponent::ApplyDelayedSolverFrameDataUpdate, GTFrameDataWrapper);
			}
		}
	}
}

void UChaosVDAdditionalGTDataRouterComponent::ClearData()
{
	AChaosVDSolverInfoActor* Owner = Cast<AChaosVDSolverInfoActor>(GetOwner());
	TSharedPtr<FChaosVDScene> CVDScene = Owner ? Owner->GetScene().Pin() : nullptr;
	if (!CVDScene)
	{
		return;
	}

	CVDScene->OnSceneUpdated().RemoveAll(this);
}

void UChaosVDAdditionalGTDataRouterComponent::ApplyDelayedSolverFrameDataUpdate(TSharedPtr<FChaosVDGameFrameDataWrapper> PendingFrameUpdate)
{
	AChaosVDSolverInfoActor* Owner = Cast<AChaosVDSolverInfoActor>(GetOwner());
	TSharedPtr<FChaosVDScene> CVDScene = Owner ? Owner->GetScene().Pin() : nullptr;
	if (!CVDScene)
	{
		return;
	}

	CVDScene->OnSceneUpdated().RemoveAll(this);

	if (PendingFrameUpdate)
	{
		if (TSharedPtr<FChaosVDGameFrameDataWrapperContext> FrameDataContext = PendingFrameUpdate->FrameData->GetCustomDataHandler().GetData<FChaosVDGameFrameDataWrapperContext>())
		{
			bool bHasNewData = false;
			for (int32 SupportedSolverID : FrameDataContext->SupportedSolverIDs)
			{
				if (AChaosVDSolverInfoActor* SolverInfoActor = CVDScene->GetSolverInfoActor(SupportedSolverID))
				{
					AChaosVDDataContainerBaseActor::FScopedGameFrameDataReRouting ScopedGTDataUpdate(SolverInfoActor);
					SolverInfoActor->UpdateFromNewGameFrameData(*PendingFrameUpdate->FrameData);
					bHasNewData = true;
				}
				else
				{
					AChaosVDDataContainerBaseActor::FScopedGameFrameDataReRouting ScopedGTDataUpdate(Owner);
					Owner->UpdateFromNewGameFrameData(*PendingFrameUpdate->FrameData);
					bHasNewData = true;
				}
			}

			if (bHasNewData)
			{
				CVDScene->RequestUpdate();
			}
		}
	}
}
