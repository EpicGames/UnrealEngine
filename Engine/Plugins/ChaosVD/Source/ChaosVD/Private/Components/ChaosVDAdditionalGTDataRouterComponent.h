// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChaosVDSolverDataComponent.h"
#include "ChaosVDAdditionalGTDataRouterComponent.generated.h"

struct FChaosVDGameFrameDataWrapper;

/** Helper component used to re-route GT data loaded as a Solver Frame.
 * This is part of a compatibility layer to support GT data from multiple files
 * Until we refactor CVD to either support multiple GT Tracks, or remove the difference between GT track and solver track
 */
UCLASS()
class UChaosVDAdditionalGTDataRouterComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()

	virtual void UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData) override;

	virtual void ClearData() override;

	void ApplyDelayedSolverFrameDataUpdate(TSharedPtr<FChaosVDGameFrameDataWrapper> PendingFrameUpdate);
};
