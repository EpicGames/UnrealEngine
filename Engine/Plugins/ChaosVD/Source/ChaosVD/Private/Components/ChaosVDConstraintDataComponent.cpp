// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDConstraintDataComponent.h"

#include "ChaosVDConstraintDataHelpers.h"
#include "ChaosVDMiscSceneObjectTypeNamesForTesting.h"
#include "ChaosVDSceneCompositionReport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDConstraintDataComponent)

UChaosVDConstraintDataComponent::UChaosVDConstraintDataComponent()
{
	SetCanEverAffectNavigation(false);
	bNavigationRelevant = false;
	PrimaryComponentTick.bCanEverTick = false;
}

const FChaosVDConstraintDataArray* UChaosVDConstraintDataComponent::GetConstraintsForParticle(int32 ParticleID, EChaosVDParticlePairSlot Options) const
{
	return Chaos::VisualDebugger::Utils::GetDataFromParticlePairMaps<FChaosVDConstraintDataByParticleMap, TSharedPtr<FChaosVDConstraintDataWrapperBase>>(ConstraintByParticle0, ConstraintByParticle1, ParticleID, Options);
}

TSharedPtr<FChaosVDConstraintDataWrapperBase> UChaosVDConstraintDataComponent::GetConstraintByIndex(int32 ConstraintIndex)
{
	if (TSharedPtr<FChaosVDConstraintDataWrapperBase>* ConstraintPtrPtr = ConstraintByConstraintIndex.Find(ConstraintIndex))
	{
		return *ConstraintPtrPtr;
	}

	return nullptr;
}

void UChaosVDConstraintDataComponent::ClearData()
{
	AllConstraints.Empty();
	ConstraintByParticle0.Empty();
	ConstraintByParticle1.Empty();
	ConstraintByConstraintIndex.Empty();
}

void UChaosVDConstraintDataComponent::AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData)
{
	Super::AppendSceneCompositionTestData(OutStateTestData);

	int32& CurrentCount = OutStateTestData.ObjectsCountByType.FindOrAdd(Chaos::VD::Test::SceneObjectTypes::CollisionConstraint);
	CurrentCount+= AllConstraints.Num();
}

