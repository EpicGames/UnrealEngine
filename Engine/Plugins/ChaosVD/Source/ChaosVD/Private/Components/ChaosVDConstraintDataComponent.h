// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChaosVDConstraintDataHelpers.h"
#include "Components/ChaosVDSolverDataComponent.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "ChaosVDConstraintDataComponent.generated.h"

struct FChaosVDJointConstraint;
typedef TMap<int32, TArray<TSharedPtr<FChaosVDConstraintDataWrapperBase>>> FChaosVDConstraintDataByParticleMap;
typedef TMap<int32, TSharedPtr<FChaosVDConstraintDataWrapperBase>> FChaosVDConstraintDataByConstraintIndexMap;
typedef TArray<TSharedPtr<FChaosVDConstraintDataWrapperBase>> FChaosVDConstraintDataArray;

UCLASS()
class UChaosVDConstraintDataComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()

public:
	UChaosVDConstraintDataComponent();

	template<typename ConstraintDataType>
	void UpdateConstraintData(const TArray<TSharedPtr<ConstraintDataType>>& InData);
	
	const FChaosVDConstraintDataArray& GetAllConstraints() const { return AllConstraints; }
	const FChaosVDConstraintDataArray* GetConstraintsForParticle(int32 ParticleID, EChaosVDParticlePairSlot Options) const;

	virtual void ClearData() override;

	virtual void AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData) override;

protected:

	TSharedPtr<FChaosVDConstraintDataWrapperBase> GetConstraintByIndex(int32 ConstraintIndex);

	FChaosVDConstraintDataArray AllConstraints;
	
	FChaosVDConstraintDataByParticleMap ConstraintByParticle0;
	FChaosVDConstraintDataByParticleMap ConstraintByParticle1;

	FChaosVDConstraintDataByConstraintIndexMap ConstraintByConstraintIndex;

};

template <typename ConstraintDataType>
void UChaosVDConstraintDataComponent::UpdateConstraintData(const TArray<TSharedPtr<ConstraintDataType>>& InData)
{
	ClearData();

	AllConstraints.Reserve(InData.Num());
	ConstraintByParticle0.Reserve(InData.Num());
	ConstraintByParticle1.Reserve(InData.Num());
	ConstraintByConstraintIndex.Reserve(InData.Num());

	for (const TSharedPtr<FChaosVDConstraintDataWrapperBase>& Constraint : InData)
	{
		if (!Constraint)
		{
			continue;
		}

		AllConstraints.Emplace(Constraint);
		ConstraintByConstraintIndex.Add(Constraint->GetConstraintIndex(), Constraint);
		Chaos::VisualDebugger::Utils::AddDataDataToParticleIDMap(ConstraintByParticle0, Constraint, Constraint->GetParticleIDAtSlot(EChaosVDParticlePairIndex::Index_0));
		Chaos::VisualDebugger::Utils::AddDataDataToParticleIDMap(ConstraintByParticle1, Constraint, Constraint->GetParticleIDAtSlot(EChaosVDParticlePairIndex::Index_1));
	}
}
