// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"

class FChaosVDCollisionDataVisualizer;
class FChaosVDScene;
class FPrimitiveDrawInterface;

enum class EChaosVDParticlePairSlot : uint8
{
	Primary,
	Secondary,
	Any
};

namespace Chaos::VisualDebugger::Utils
{
	template <typename MapType, typename TData>
	void AddDataDataToParticleIDMap(MapType& MapToUpdate, const TData& MidPhaseData, int32 ParticleID)
	{
		if (TArray<TData>* ParticleCollisionData = MapToUpdate.Find(ParticleID))
		{
			ParticleCollisionData->Add(MidPhaseData);
		}
		else
		{
			MapToUpdate.Add(ParticleID, { MidPhaseData });
		}
	}

	template <typename MapType, typename TData>
	const TArray<TData>* GetDataFromParticlePairMaps(const MapType& Map0ToQuery, const MapType& Map1ToQuery, int32 ParticleID, EChaosVDParticlePairSlot Options)
	{
		switch (Options)
		{
		case EChaosVDParticlePairSlot::Primary:
			{
				return Map0ToQuery.Find(ParticleID);
				break;
			}
		case EChaosVDParticlePairSlot::Secondary:
			{
				return Map1ToQuery.Find(ParticleID);
				break;
			}
		case EChaosVDParticlePairSlot::Any:
			{
				const TArray<TData>* FoundMidPhasesContainer = Map0ToQuery.Find(ParticleID);
				return FoundMidPhasesContainer ? FoundMidPhasesContainer : Map1ToQuery.Find(ParticleID);
				break;
			}
		}
		return nullptr;
	}
}
