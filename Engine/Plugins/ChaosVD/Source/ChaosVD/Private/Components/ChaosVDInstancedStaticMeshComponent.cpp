// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDInstancedStaticMeshComponent.h"

#include "ChaosVDGeometryBuilder.h"
#include "ChaosVDModule.h"
#include "Actors/ChaosVDGeometryContainer.h"
#include "Widgets/SChaosVDPlaybackViewport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDInstancedStaticMeshComponent)


void UChaosVDInstancedStaticMeshComponent::UpdateVisibilityForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	if (InInstanceHandle->GetMeshComponent() != this)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Attempted to update a mesh instance using a handle from another component. No instances were updated | Handle Component [%ls] | Current Component [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(InInstanceHandle->GetMeshComponent()), *GetNameSafe(this));
		return;
	}

	EnqueueMeshInstanceOperation(InInstanceHandle, EChaosVDMeshInstanceOperationsFlags::TransformUpdate);
}

void UChaosVDInstancedStaticMeshComponent::UpdateSelectionStateForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	if (InInstanceHandle->GetMeshComponent() != this)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Attempted to update a mesh instance using a handle from another component. No instances were updated | Handle Component [%ls] | Current Component [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(InInstanceHandle->GetMeshComponent()), *GetNameSafe(this));
		return;
	}

	EnqueueMeshInstanceOperation(InInstanceHandle, EChaosVDMeshInstanceOperationsFlags::SelectionUpdate);
}

void UChaosVDInstancedStaticMeshComponent::UpdateColorForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	FLinearColor NewColor = InInstanceHandle->GetInstanceColor();
	// Check that this mesh component supports the intended visualization
	// We can't change the material of Instanced mesh components because we might have other instances that are not intended to be translucent (or the other way around).
	// The Mesh handle instance system should have detected we need to migrate the instance to another component before ever reaching this point
	const bool bIsSolidColor = FMath::IsNearlyEqual(NewColor.A, 1.0f);
	const bool bMeshComponentSupportsTranslucentInstances = EnumHasAnyFlags(static_cast<EChaosVDMeshAttributesFlags>(MeshComponentAttributeFlags), EChaosVDMeshAttributesFlags::TranslucentGeometry);
	if (bIsSolidColor)
	{
		if (!ensure(!bMeshComponentSupportsTranslucentInstances))
		{
			UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Desired Color is not supported in this mesh component [%ls]...", ANSI_TO_TCHAR(__FUNCTION__), *NewColor.ToString());
		}
	}
	else
	{
		if (!ensure(bMeshComponentSupportsTranslucentInstances))
		{
			UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Desired Color is not supported in this mesh component [%ls]...", ANSI_TO_TCHAR(__FUNCTION__), *NewColor.ToString());
		}
	}

	EnqueueMeshInstanceOperation(InInstanceHandle, EChaosVDMeshInstanceOperationsFlags::ColorUpdate);
}

void UChaosVDInstancedStaticMeshComponent::UpdateWorldTransformForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	if (InInstanceHandle->GetMeshComponent() != this)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Attempted to update a mesh instance using a handle from another component. No instances were updated | Handle Component [%ls] | Current Component [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(InInstanceHandle->GetMeshComponent()), *GetNameSafe(this));
		return;
	}

	EnqueueMeshInstanceOperation(InInstanceHandle, EChaosVDMeshInstanceOperationsFlags::TransformUpdate);
}

void UChaosVDInstancedStaticMeshComponent::Reset()
{
	bIsMeshReady = false;
	bIsDestroyed = false;
	MeshReadyDelegate = FChaosVDMeshReadyDelegate();
	ComponentEmptyDelegate = FChaosVDMeshComponentEmptyDelegate();

	SetStaticMesh(nullptr);

	EmptyOverrideMaterials();

	ClearInstanceHandles();

	CurrentGeometryKey = 0;
}

void UChaosVDInstancedStaticMeshComponent::Initialize()
{
	// We need to set the reverse culling flag correctly
	// And the material

	bReverseCulling = EnumHasAnyFlags(MeshComponentAttributeFlags, EChaosVDMeshAttributesFlags::MirroredGeometry);

	TSharedPtr<FChaosVDGeometryBuilder> GeometryBuilder = GeometryBuilderWeakPtr.Pin();
	if (ensure(GeometryBuilder))
	{
		if (!ensure(!HasOverrideMaterials()))
		{
			UE_LOGF(LogChaosVDEditor, Error, "[%ls] | Component [%ls] already had a material, which is not expected!. It likely it was modified after being disposed.", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(this));
			
			EmptyOverrideMaterials();
		}

		GeometryBuilder->RequestMaterialUpdate(this);
	}
}

void UChaosVDInstancedStaticMeshComponent::SetGeometryBuilder(TWeakPtr<FChaosVDGeometryBuilder> GeometryBuilder)
{
	GeometryBuilderWeakPtr = GeometryBuilder;
}

EChaosVDMaterialType UChaosVDInstancedStaticMeshComponent::GetMaterialType() const
{
	if (EnumHasAnyFlags(MeshComponentAttributeFlags, EChaosVDMeshAttributesFlags::TranslucentGeometry))
	{
		return EChaosVDMaterialType::ISMCTranslucent;
	}

	return EChaosVDMaterialType::ISMCOpaque;
}

void UChaosVDInstancedStaticMeshComponent::OnDisposed()
{
	Reset();

	bIsDestroyed = true;
	
	SetRelativeTransform(FTransform::Identity);

	if (IsRegistered())
	{
		UnregisterComponent();
	}

	if (AActor* Owner = GetOwner())
	{
		Owner->RemoveOwnedComponent(this);
	}

	FTSTicker::RemoveTicker(ExternalTickerHandle);
}

bool UChaosVDInstancedStaticMeshComponent::ExternalTick(float DeltaTime)
{
	ProcessChanges();
	return true;
}

bool UChaosVDInstancedStaticMeshComponent::UpdateGeometryKey(const uint32 NewHandleGeometryKey)
{
	if (CurrentGeometryKey != 0 && CurrentGeometryKey != NewHandleGeometryKey)
	{
		ensure(false);

		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Attempted to add a mesh instance belonging to another geometry key. No instance was added | CurrentKey [%u] | New Key [%u]", ANSI_TO_TCHAR(__FUNCTION__), CurrentGeometryKey, NewHandleGeometryKey);
		return false;
	}
	else
	{
		CurrentGeometryKey = NewHandleGeometryKey;
	}
	
	return true;
}

TSharedPtr<FChaosVDInstancedMeshData> UChaosVDInstancedStaticMeshComponent::AddMeshInstance(const FTransform InstanceTransform, bool bIsWorldSpace, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InGeometryHandle, int32 ParticleID, int32 SolverID)
{
	const TSharedPtr<FChaosVDInstancedMeshData> InstanceHandle = MakeShared<FChaosVDInstancedMeshData>(INDEX_NONE, this, ParticleID, SolverID, InGeometryHandle);

	const uint32 NewHandleGeometryKey = InGeometryHandle->GetGeometryKey();

	if (!UpdateGeometryKey(NewHandleGeometryKey))
	{
		return nullptr;
	}

	EnqueueMeshInstanceOperation(InstanceHandle.ToSharedRef(), EChaosVDMeshInstanceOperationsFlags::Add);

	return InstanceHandle;
}

void UChaosVDInstancedStaticMeshComponent::AddExistingMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InMeshDataHandle)
{
	const uint32 NewHandleGeometryKey = InMeshDataHandle->ExtractedGeometryHandle->GetGeometryKey();

	if (!UpdateGeometryKey(NewHandleGeometryKey))
	{
		return;
	}

	InMeshDataHandle->SetMeshInstanceIndex(INDEX_NONE);
	InMeshDataHandle->SetMeshComponent(this);

	EnqueueMeshInstanceOperation(InMeshDataHandle, EChaosVDMeshInstanceOperationsFlags::Add);
}

void UChaosVDInstancedStaticMeshComponent::RemoveMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InHandleToRemove, ERemovalMode Mode)
{
	if (InHandleToRemove->GetMeshComponent() != this)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Attempted to remove a mesh instace using a handle from another component. No instanced were removed | Handle Component [%ls] | Current Component [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(InHandleToRemove->GetMeshComponent()), *GetNameSafe(this));
		return;
	}

	if (Mode == ERemovalMode::Deferred)
	{
		EnqueueMeshInstanceOperation(InHandleToRemove, EChaosVDMeshInstanceOperationsFlags::Remove);
	}
	else
	{
		PendingOperationsByInstance.Remove(InHandleToRemove);

		int32 InstanceIndex = InHandleToRemove->GetMeshInstanceIndex();

		if (InstanceIndex == INDEX_NONE)
		{
			// The mesh instance wasn't added yet
			return;
		}

		if (!ensure(!InHandleToRemove->IsPendingDestroy()))
		{
			return;
		}
		
		{
			const int32 CurrentInstanceCount = GetInstanceCount();
			if (!ensure(IsValidInstance(InstanceIndex)))
			{
				UE_LOGF(LogChaosVDEditor, Error, "[%ls] Mesh Data Handle has an invalid instance index. No instanced were removed | Handle Instance Index [%d] | Current Instance Count [%d]", ANSI_TO_TCHAR(__FUNCTION__), InHandleToRemove->GetMeshInstanceIndex(), CurrentInstanceCount);
				return;
			}
		}

		RemoveInstance(InstanceIndex);
		InHandleToRemove->SetMeshInstanceIndex(INDEX_NONE);
	}
}

void UChaosVDInstancedStaticMeshComponent::UpdateInstanceHandle(int32 OldIndex, int32 NewIndex)
{
	TSharedPtr<FChaosVDInstancedMeshData> RelocatedHandle;
	ensure(CurrentInstanceHandlesByIndex.RemoveAndCopyValue(OldIndex, RelocatedHandle));

	if (!ensure(RelocatedHandle->GetMeshComponent() == this))
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Trying to update an instance from another component | Handle Component [%ls] | This component [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(RelocatedHandle->GetMeshComponent()), *GetNameSafe(this));
	}

    if (RelocatedHandle)
    {
        RelocatedHandle->SetMeshInstanceIndex(NewIndex);

        if (NewIndex != INDEX_NONE)
        {
        	CurrentInstanceHandlesByIndex.Add(NewIndex, RelocatedHandle);
        }
    }
}

void UChaosVDInstancedStaticMeshComponent::HandleInstanceIndexUpdated(TArrayView<const FInstancedStaticMeshDelegates::FInstanceIndexUpdateData> InIndexUpdates)
{
	bool bComponentCleared = false;
	for (const FInstancedStaticMeshDelegates::FInstanceIndexUpdateData& IndexUpdateData : InIndexUpdates)
	{
		switch (IndexUpdateData.Type)
		{
		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Added:
			break; // We don't need to process 'Added' updates as they can't affect existing IDs
		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Relocated:
			{
				UpdateInstanceHandle(IndexUpdateData.OldIndex, IndexUpdateData.Index);
				break;
			}
		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Removed:
			{
				UpdateInstanceHandle(IndexUpdateData.Index, INDEX_NONE);
				break;
			}
		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Destroyed:
		case FInstancedStaticMeshDelegates::EInstanceIndexUpdateType::Cleared:
			{
				bComponentCleared = true;
				break;
			}
		default:
			break;
		}
	};

	if (bComponentCleared)
	{
		ClearInstanceHandles();
	}
}

bool UChaosVDInstancedStaticMeshComponent::Modify(bool bAlwaysMarkDirty)
{
	// CVD Mesh Components are not saved to any assets or require undo
	return false;
}

bool UChaosVDInstancedStaticMeshComponent::IsNavigationRelevant() const
{
	return false;
}

void UChaosVDInstancedStaticMeshComponent::ClearInstanceHandles()
{
	CurrentInstanceHandlesByIndex.Reset();
	PendingOperationsByInstance.Reset();
}

void UChaosVDInstancedStaticMeshComponent::OnAcquired()
{
	bIsDestroyed = false;

	ExternalTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateUObject(this, &UChaosVDInstancedStaticMeshComponent::ExternalTick));
}

void UChaosVDInstancedStaticMeshComponent::EnqueueMeshInstanceOperation(const TSharedRef<FChaosVDInstancedMeshData>& MeshInstanceHandle, EChaosVDMeshInstanceOperationsFlags Operation)
{
	if (!ensure(!bIsDestroyed))
	{
		return;
	}
	
	if (!ensure(!MeshInstanceHandle->IsPendingDestroy()))
	{
		return;
	}

	if (EChaosVDMeshInstanceOperationsFlags* PendingOperation = PendingOperationsByInstance.Find(MeshInstanceHandle))
	{
		if (EnumHasAnyFlags(Operation, EChaosVDMeshInstanceOperationsFlags::Remove))
		{
			const int32 CurrentInstanceCount = GetInstanceCount();

			if (CurrentInstanceCount == 0)
			{
				// If the instance count is 0 it means we have pending operations but we didn't process them, therefore there is nothing to do
				PendingOperationsByInstance.Remove(MeshInstanceHandle);
			}
			else
			{
				if (EnumHasAnyFlags(*PendingOperation, EChaosVDMeshInstanceOperationsFlags::Add))
				{
					ensure(MeshInstanceHandle->GetMeshInstanceIndex() == INDEX_NONE);
					PendingOperationsByInstance.Remove(MeshInstanceHandle);
					return;
				}

				// If this new operation is a removal, discard all other operations
				*PendingOperation = Operation;
			}
		}
		else if (EnumHasAnyFlags(Operation, EChaosVDMeshInstanceOperationsFlags::Add))
		{
			EnumRemoveFlags(*PendingOperation, EChaosVDMeshInstanceOperationsFlags::Remove);
		}
		else
		{
			EnumRemoveFlags(*PendingOperation, EChaosVDMeshInstanceOperationsFlags::Remove);
			EnumAddFlags(*PendingOperation, Operation);
		}
	}
	else
	{
		PendingOperationsByInstance.Add(MeshInstanceHandle, Operation);
	}
}

void UChaosVDInstancedStaticMeshComponent::CancelMeshInstanceOperation(const TSharedRef<FChaosVDInstancedMeshData>& MeshInstanceHandle, EChaosVDMeshInstanceOperationsFlags Operation)
{
	if (EChaosVDMeshInstanceOperationsFlags* PendingOperation = PendingOperationsByInstance.Find(MeshInstanceHandle))
	{
		EnumRemoveFlags(*PendingOperation, Operation);

		if (*PendingOperation == EChaosVDMeshInstanceOperationsFlags::None)
		{
			PendingOperationsByInstance.Remove(MeshInstanceHandle);
		}
	}
}

bool UChaosVDInstancedStaticMeshComponent::CanExecuteOperation(EChaosVDMeshInstanceOperationsFlags Operation, EChaosVDMeshInstanceOperationsFlags PendingOperations)
{
	if (!EnumHasAnyFlags(PendingOperations, Operation))
	{
		return false;
	}

	if (EnumHasAnyFlags(Operation, EChaosVDMeshInstanceOperationsFlags::ColorUpdate) ||
		EnumHasAnyFlags(Operation, EChaosVDMeshInstanceOperationsFlags::SelectionUpdate) ||
		EnumHasAnyFlags(Operation, EChaosVDMeshInstanceOperationsFlags::TransformUpdate))
	{
		return !EnumHasAnyFlags(PendingOperations, EChaosVDMeshInstanceOperationsFlags::Add);
	}

	return true;
}

void UChaosVDInstancedStaticMeshComponent::ProcessChanges()
{
	if (bIsMeshReady)
	{
		bool bHasSelectionChange = false;
		for (TMap<TSharedPtr<FChaosVDInstancedMeshData>, EChaosVDMeshInstanceOperationsFlags>::TIterator HandleOperationsIterator = PendingOperationsByInstance.CreateIterator(); HandleOperationsIterator; ++HandleOperationsIterator)
		{
			TPair<TSharedPtr<FChaosVDInstancedMeshData>, EChaosVDMeshInstanceOperationsFlags>& OperationHandle = *HandleOperationsIterator;
			
			if (CanExecuteOperation(EChaosVDMeshInstanceOperationsFlags::Add, OperationHandle.Value))
			{
				int32 InstanceIndex = AddInstance(OperationHandle.Key->GetWorldTransform(), true);
				OperationHandle.Key->SetMeshInstanceIndex(InstanceIndex);

				CurrentInstanceHandlesByIndex.Add(InstanceIndex, OperationHandle.Key);

				EnumRemoveFlags(OperationHandle.Value, EChaosVDMeshInstanceOperationsFlags::Add);
			}
			
			if (CanExecuteOperation(EChaosVDMeshInstanceOperationsFlags::TransformUpdate, OperationHandle.Value))
			{
				constexpr bool bIsWorldSpaceTransform = true;
				constexpr bool bMarkRenderDirty = true;
				constexpr bool bTeleport = true;

				FTransform Transform = OperationHandle.Key->GetWorldTransform();
				if (!OperationHandle.Key->GetVisibility())
				{
					// Setting the scale to 0 will hide this instance while keeping it on the component
					Transform.SetScale3D(FVector::ZeroVector);
				}
	
				UpdateInstanceTransform(OperationHandle.Key->GetMeshInstanceIndex(), Transform, bIsWorldSpaceTransform, bMarkRenderDirty, bTeleport);

				EnumRemoveFlags(OperationHandle.Value, EChaosVDMeshInstanceOperationsFlags::TransformUpdate);
			}

			if (CanExecuteOperation(EChaosVDMeshInstanceOperationsFlags::ColorUpdate, OperationHandle.Value))
			{
				SetNumCustomDataFloats(4);
				FLinearColor NewColor = OperationHandle.Key->GetInstanceColor();
				SetCustomData(OperationHandle.Key->GetMeshInstanceIndex(), TArrayView<float>(reinterpret_cast<float*>(&NewColor), 4));

				EnumRemoveFlags(OperationHandle.Value, EChaosVDMeshInstanceOperationsFlags::ColorUpdate);
			}

			if (CanExecuteOperation(EChaosVDMeshInstanceOperationsFlags::SelectionUpdate, OperationHandle.Value))
			{
				const int32 InstanceIndex = OperationHandle.Key->GetMeshInstanceIndex();

				if (!ensure(IsValidInstance(InstanceIndex)))
				{
					UE_LOGF(LogChaosVDEditor, Error, "[%ls] Attempted to update a mesh instance using a handle with an invalid instance index | Handle Instance Index [%d] | Current Instance Conut [%d]", ANSI_TO_TCHAR(__FUNCTION__), InstanceIndex, GetInstanceCount());
					EnumRemoveFlags(OperationHandle.Value, EChaosVDMeshInstanceOperationsFlags::SelectionUpdate);
				}
				else
				{
					NotifySMInstanceSelectionChanged({this, InstanceIndex}, OperationHandle.Key->InstanceState.bIsSelected);
					EnumRemoveFlags(OperationHandle.Value, EChaosVDMeshInstanceOperationsFlags::SelectionUpdate);
					bHasSelectionChange = true;
				}
			}

			if (CanExecuteOperation(EChaosVDMeshInstanceOperationsFlags::Remove, OperationHandle.Value))
			{
				int32 InstanceIndex = OperationHandle.Key->GetMeshInstanceIndex();
				const int32 CurrentInstanceCount = GetInstanceCount();

				if (!ensure(IsValidInstance(InstanceIndex)))
				{
					UE_LOGF(LogChaosVDEditor, Error, "[%ls] Mesh Data Handle has an invalid instance index. No instanced were removed | Handle Instance Index [%d] | Current Instance Count [%d]", ANSI_TO_TCHAR(__FUNCTION__), OperationHandle.Key->GetMeshInstanceIndex(), CurrentInstanceCount);
				}
				else
				{
					RemoveInstance(InstanceIndex);
					OperationHandle.Key->SetMeshInstanceIndex(INDEX_NONE);
				}

				// We just removed this instance, clear all flags
				OperationHandle.Value = EChaosVDMeshInstanceOperationsFlags::None;
			}

			if (OperationHandle.Value == EChaosVDMeshInstanceOperationsFlags::None)
			{
				HandleOperationsIterator.RemoveCurrent();
			}
		}

		if (bHasSelectionChange)
		{
			SChaosVDPlaybackViewport::ExecuteExternalViewportInvalidateRequest();
		}

		if (CurrentInstanceHandlesByIndex.Num() == 0 && PendingOperationsByInstance.Num() == 0)
		{
			ensure(GetInstanceCount() == 0);

			ComponentEmptyDelegate.Broadcast(this);
		}
	}
}

uint32 UChaosVDInstancedStaticMeshComponent::GetGeometryKey() const
{
	return CurrentGeometryKey;
}

TSharedPtr<FChaosVDInstancedMeshData> UChaosVDInstancedStaticMeshComponent::GetMeshDataInstanceHandle(int32 InstanceIndex) const
{
	if (const TSharedPtr<FChaosVDInstancedMeshData>* FoundHandle = CurrentInstanceHandlesByIndex.Find(InstanceIndex))
	{
		return *FoundHandle;
	}

	return nullptr;
}

void UChaosVDInstancedStaticMeshComponent::SetIsMeshReady(bool bIsReady)
{
	bIsMeshReady = bIsReady;
}
