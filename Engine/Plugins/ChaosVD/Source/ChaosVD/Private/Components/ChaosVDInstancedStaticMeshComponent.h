// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/InstancedStaticMeshComponent.h"
#include "ChaosVDGeometryDataComponent.h"
#include "Components/HierarchicalInstancedStaticMeshComponent.h"
#include "Containers/Ticker.h"
#include "Interfaces/ChaosVDPooledObject.h"
#include "ChaosVDInstancedStaticMeshComponent.generated.h"

UENUM()
enum class EChaosVDMeshInstanceOperationsFlags
{
	None = 0,
	Add = 1 << 0,
	Remove = 1 << 1,
	ColorUpdate = 1 << 2,
	SelectionUpdate = 1 << 3,
	TransformUpdate = 1 << 4,
};
ENUM_CLASS_FLAGS(EChaosVDMeshInstanceOperationsFlags);

/** CVD version of an Instance Static Mesh Component that holds additional CVD data */
UCLASS(HideCategories=("Transform"))
class UChaosVDInstancedStaticMeshComponent : public UInstancedStaticMeshComponent, public IChaosVDGeometryComponent, public IChaosVDPooledObject
{
	GENERATED_BODY()
public:
	UChaosVDInstancedStaticMeshComponent(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	{
		SetRemoveSwap();
		SetCanEverAffectNavigation(false);
		bHasPerInstanceHitProxies = true;

		bOverrideWireframeColor = true;
		WireframeColorOverride = FColor::White;
	}

	void EnqueueMeshInstanceOperation(const TSharedRef<FChaosVDInstancedMeshData>& MeshInstanceHandle, EChaosVDMeshInstanceOperationsFlags Operation);
	void CancelMeshInstanceOperation(const TSharedRef<FChaosVDInstancedMeshData>& MeshInstanceHandle, EChaosVDMeshInstanceOperationsFlags Operation);

	bool CanExecuteOperation(EChaosVDMeshInstanceOperationsFlags Operation, EChaosVDMeshInstanceOperationsFlags PendingOperations);

	void ProcessChanges();
	
	// BEGIN IChaosVDGeometryDataComponent Interface
	
	virtual uint32 GetGeometryKey() const override;

	virtual TSharedPtr<FChaosVDInstancedMeshData> GetMeshDataInstanceHandle(int32 InstanceIndex) const override;

	virtual bool IsMeshReady() const override
	{
		return bIsMeshReady;
	}
	
	virtual void SetIsMeshReady(bool bIsReady) override;

	virtual FChaosVDMeshReadyDelegate* OnMeshReady() override
	{
		return &MeshReadyDelegate;
	}

	virtual FChaosVDMeshComponentEmptyDelegate* OnComponentEmpty() override
	{
		return &ComponentEmptyDelegate;
	}

	virtual void UpdateVisibilityForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) override;

	virtual void UpdateSelectionStateForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) override;

	virtual void UpdateColorForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) override;
	
	virtual void UpdateWorldTransformForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) override;

	virtual void SetMeshComponentAttributeFlags(EChaosVDMeshAttributesFlags Flags) override
	{
		MeshComponentAttributeFlags = Flags;
	}
	virtual EChaosVDMeshAttributesFlags GetMeshComponentAttributeFlags() const override
	{
		return MeshComponentAttributeFlags;
	}

	virtual bool GetIsDestroyed() const override
	{
		return bIsDestroyed;
	}

	virtual void SetIsDestroyed(bool bNewIsDestroyed) override
	{
		bIsDestroyed = bNewIsDestroyed;
	}

	virtual void Reset() override;

	virtual void Initialize() override;

	virtual void SetGeometryBuilder(TWeakPtr<FChaosVDGeometryBuilder> GeometryBuilder) override;
	
	virtual EChaosVDMaterialType GetMaterialType() const override;
	// END IChaosVDGeometryDataComponent Interface

	virtual TSharedPtr<FChaosVDInstancedMeshData> AddMeshInstance(const FTransform InstanceTransform, bool bIsWorldSpace, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InGeometryHandle, int32 ParticleID, int32 SolverID) override;
	virtual void AddExistingMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InMeshDataHandle) override;
	virtual void RemoveMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InHandleToRemove, ERemovalMode Mode = ERemovalMode::Deferred) override;

	void UpdateInstanceHandle(int32 OldIndex, int32 NewIndex);

	/** Handles a mesh instance index update reported by the mesh component used to render this mesh instance */
	void HandleInstanceIndexUpdated(TArrayView<const FInstancedStaticMeshDelegates::FInstanceIndexUpdateData> InIndexUpdates);

	virtual bool Modify(bool bAlwaysMarkDirty) override;

	virtual bool IsNavigationRelevant() const override;

	void ClearInstanceHandles();

	// BEGIN IChaosVDPooledObject Interface
	virtual void OnAcquired() override;
	virtual void OnDisposed() override;
	// END IChaosVDPooledObject Interface

	bool ExternalTick(float DeltaTime);

protected:
	
	bool UpdateGeometryKey(uint32 NewHandleGeometryKey);

	EChaosVDMeshAttributesFlags MeshComponentAttributeFlags = EChaosVDMeshAttributesFlags::None;

	uint32 CurrentGeometryKey = 0;

	uint8 bIsMeshReady:1 = false;	
	uint8 bIsDestroyed:1 = false;

	FChaosVDMeshReadyDelegate MeshReadyDelegate;
	FChaosVDMeshComponentEmptyDelegate ComponentEmptyDelegate;
	
	TWeakPtr<FChaosVDGeometryBuilder> GeometryBuilderWeakPtr;

	TMap<TSharedPtr<FChaosVDInstancedMeshData>, EChaosVDMeshInstanceOperationsFlags> PendingOperationsByInstance;

	TMap<int32, TSharedPtr<FChaosVDInstancedMeshData>> CurrentInstanceHandlesByIndex;

	FTSTicker::FDelegateHandle ExternalTickerHandle;
};
