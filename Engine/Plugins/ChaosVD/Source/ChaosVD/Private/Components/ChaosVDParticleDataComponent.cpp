// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDParticleDataComponent.h"

#include "ChaosVDRecording.h"
#include "ChaosVDScene.h"
#include "ChaosVDSceneCompositionReport.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDSettingsManager.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Async/ParallelFor.h"
#include "DataStorage/Features.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Misc/ScopedSlowTask.h"
#include "Settings/ChaosVDParticleVisualizationSettings.h"
#include "TEDS/ChaosVDParticleEditorDataFactory.h"
#include "TEDS/ChaosVDSelectionInterface.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "TEDS/ChaosVDTedsUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDParticleDataComponent)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

namespace ChaosVDSceneUIOptions
{
	constexpr float DelayToShowProgressDialogThreshold = 1.0f;
	constexpr bool bShowCancelButton = false;
	constexpr bool bAllowInPIE = false;
}

namespace Chaos::VisualDebugger::Cvars
{
	static bool bUnloadParticleDataUsingKeyFrameDiff = true;
	static FAutoConsoleVariableRef CVarChaosVDUnloadParticleDataUsingKeyFrameDiff(
		TEXT("p.Chaos.VD.Tool.UnloadParticleDataUsingKeyFrameDiff"),
		bUnloadParticleDataUsingKeyFrameDiff,
		TEXT("If false, CVD will only rely on the particle destroyed events to figure out what needs to be removed from the visualization, instead of doing a diff based on keyframe data"));
}

namespace Chaos::VD::Test::SceneObjectTypes
{
	FName ActiveParticles = FName("ActiveParticles");
}

UChaosVDParticleDataComponent::UChaosVDParticleDataComponent()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (UChaosVDParticleVisualizationSettings* ParticleVisualizationSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationSettings>())
	{
		ParticleVisualizationSettings->OnSettingsChanged().AddUObject(this, &UChaosVDParticleDataComponent::HandleVisibilitySettingsUpdated);
	}
	
	if (UChaosVDParticleVisualizationColorSettings* ColorVisualizationSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationColorSettings>())
	{
		ColorVisualizationSettings->OnSettingsChanged().AddUObject(this, &UChaosVDParticleDataComponent::HandleColorsSettingsUpdated);
	}

	StreamingSystem.SetStreamingDataSource(this);

	StreamingSystem.Initialize();
}

void UChaosVDParticleDataComponent::SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr)
{
	Super::SetScene(InSceneWeakPtr);
	
	StreamingSystem.SetScene(InSceneWeakPtr);
}

void UChaosVDParticleDataComponent::ClearData()
{
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		return;
	}

	if (UChaosVDParticleVisualizationSettings* ParticleVisualizationSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationSettings>())
	{
		ParticleVisualizationSettings->OnSettingsChanged().RemoveAll(this);
	}
	
	if (UChaosVDParticleVisualizationColorSettings* ColorVisualizationSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationColorSettings>())
	{
		ColorVisualizationSettings->OnSettingsChanged().RemoveAll(this);
	}

	TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (ScenePtr && GetSelectedParticle())
	{
		ScenePtr->ClearSelectionAndNotify();
	}

	StreamingSystem.SetStreamingDataSource(nullptr);
	StreamingSystem.DeInitialize();
	
	{
		FReadScopeLock ReadLock(ParticleSceneObjectsLock);
		using namespace Chaos::VD::TypedElementDataUtil;
		for (const TPair<int32, TSharedRef<FChaosVDSceneParticle>>& ParticleWithID : SolverParticlesByID)
		{
			DestroyTypedElementHandleForStruct(&ParticleWithID.Value.Get());

			Chaos::VD::TedsUtils::RemoveObjectToFromDataStorage(&ParticleWithID.Value.Get());
		}

		for (const TPair<EChaosVDParticleType, TSharedPtr<FChaosVDBaseSceneObject>>& ContainerWithID : ParticleSceneContainersByType)
		{
			Chaos::VD::TedsUtils::RemoveObjectToFromDataStorage(ContainerWithID.Value.Get());
		}
	}

	{
		FWriteScopeLock WriteScopeLock(ParticleSceneObjectsLock);
		SolverParticlesByID.Reset();
		SolverParticlesArray.Reset();
	}
}

FChaosVDSceneParticle* UChaosVDParticleDataComponent::GetSelectedParticle() const
{
	TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return nullptr;
	}
	
	using namespace Chaos::VD::TypedElementDataUtil;

	constexpr int32 MaxElements = 1;

	TArray<FTypedElementHandle, TInlineAllocator<MaxElements>> SelectedParticlesHandles;
	ScenePtr->GetElementSelectionSet()->GetSelectedElementHandles(SelectedParticlesHandles, UChaosVDSelectionInterface::StaticClass());

	return SelectedParticlesHandles.Num() > 0 ? GetStructDataFromTypedElementHandle<FChaosVDSceneParticle>(SelectedParticlesHandles[0]) : nullptr;
}

void UChaosVDParticleDataComponent::HandleWorldStreamingLocationUpdated(const FVector& InLocation)
{
	StreamingSystem.UpdateStreamingSourceLocation(InLocation);
}

void UChaosVDParticleDataComponent::AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData)
{
	Super::AppendSceneCompositionTestData(OutStateTestData);

	int32& CurrentCount = OutStateTestData.ObjectsCountByType.FindOrAdd(Chaos::VD::Test::SceneObjectTypes::ActiveParticles);

	for (const TPair<int32, TSharedRef<FChaosVDSceneParticle>>& ParticleWithID : SolverParticlesByID)
	{
		// Only take into account active particles because inactive particles are destroyed ones we keep instantiated as part of a pseudo pool. 
		if (ParticleWithID.Value->IsActive())
		{
			CurrentCount++;
		}
	}
}

int32 UChaosVDParticleDataComponent::GetMaxElementCountForBatchTEDSUpdate_AssumesLocked() const
{
	// All Particles + their container objects + this component owner
	return SolverParticlesByID.Num() + ParticleSceneContainersByType.Num() + 1;
}

void UChaosVDParticleDataComponent::ProcessRemovedParticles_AssumessLocked(const TSharedRef<FChaosVDScene>& SceneRef, const FChaosVDSolverFrameData& InSolverFrameData,
	const FChaosVDFrameStageData& InSolverFrameStageData, const TSet<int32>& ParticlesUpdatedIDsForKeyFrameDiff, FTedsHandlesForBatchArray& OutRemovedTedsHandles)
{
	// TODO: We will need handle multi selection in the future, we can do by checking if each particle is selected directly with
	// the selection set, but given that we know now that we only have one particle, we can just compare against that as this is faster
	FChaosVDSceneParticle* SelectedParticleInstance = GetSelectedParticle();
	TSharedPtr<const FChaosVDParticleDataWrapper> SelectedParticleInstanceData = SelectedParticleInstance ? SelectedParticleInstance->GetParticleData() : nullptr;
	if (SelectedParticleInstanceData && InSolverFrameData.ParticlesDestroyedIDs.Contains(SelectedParticleInstanceData->ParticleIndex))
	{
		SceneRef->ClearSelectionAndNotify();

		Chaos::VD::TypedElementDataUtil::DestroyTypedElementHandleForStruct(SelectedParticleInstance);
	}

	// Evaluating and setting a particle as inactive can be done in parallel, but updating visibility and TEDS based on the new state, needs to be done in the GT,
	TQueue<TSharedPtr<FChaosVDSceneParticle>, EQueueMode::Mpsc> DestroyedParticlesToProcessInGT;

	// To be able to fully remove this diff removal, we need to start recording particle created events so we know what to unload if the user is scrubbing backwards
	if (Chaos::VisualDebugger::Cvars::bUnloadParticleDataUsingKeyFrameDiff && InSolverFrameData.bIsKeyFrame && EnumHasAnyFlags(InSolverFrameStageData.StageFlags, EChaosVDSolverStageFlags::ExplicitStage))
	{
		TArray<int32, TInlineAllocator<256, TMemStackAllocator<>>> AvailableParticlesIDs;	
		SolverParticlesByID.GenerateKeyArray(AvailableParticlesIDs);

		ParallelFor(SolverParticlesByID.Num(),[&InSolverFrameData, &AvailableParticlesIDs, &ParticlesUpdatedIDsForKeyFrameDiff, &DestroyedParticlesToProcessInGT, this](int32 Index)
		{
			int32 ParticleID = AvailableParticlesIDs[Index];

			if (bool bShouldDestroyParticle = InSolverFrameData.ParticlesDestroyedIDs.Contains(Index) || !ParticlesUpdatedIDsForKeyFrameDiff.Contains(ParticleID))
			{
				if (TSharedRef<FChaosVDSceneParticle>* FoundParticle = SolverParticlesByID.Find(ParticleID))
				{
					(*FoundParticle)->SetIsActive(false);

					DestroyedParticlesToProcessInGT.Enqueue(*FoundParticle);
				}
			}
		});	
	}
	else
	{
		TArray<int32, TInlineAllocator<256, TMemStackAllocator<>>> DestroyedParticlesArray;
		for (TSet<int32>::TConstIterator SetIt(InSolverFrameData.ParticlesDestroyedIDs);SetIt;++SetIt)
		{
			DestroyedParticlesArray.Add(*SetIt);
		}

		ParallelFor(DestroyedParticlesArray.Num(),[&DestroyedParticlesArray, &DestroyedParticlesToProcessInGT, this](int32 ParticleIndex)
		{
			int32 DestroyedParticlesID = DestroyedParticlesArray[ParticleIndex];
			if (TSharedRef<FChaosVDSceneParticle>* FoundParticle = SolverParticlesByID.Find(DestroyedParticlesID))
			{
				(*FoundParticle)->SetIsActive(false);

				DestroyedParticlesToProcessInGT.Enqueue(*FoundParticle);
			}
		});
	}

	while (!DestroyedParticlesToProcessInGT.IsEmpty())
	{
		TSharedPtr<FChaosVDSceneParticle> ParticleToProcessInGT;
		DestroyedParticlesToProcessInGT.Dequeue(ParticleToProcessInGT);

		if (EnumHasAnyFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::Active))
		{
			OutRemovedTedsHandles.Emplace(ParticleToProcessInGT->GetTedsRowHandle());
			EnumRemoveFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::Active);
		}

		ParticleToProcessInGT->UpdateGeometryComponentsVisibility();
	}
}

void UChaosVDParticleDataComponent::BatchTEDSSyncFromWorldTag(FTedsHandlesForBatchArray& ParticlesPendingTEDSTagSync,
	const UE::Editor::DataStorage::ICompatibilityProvider* CompatibilityStorage,
	UE::Editor::DataStorage::ICoreProvider* DataStorage)
{
	if (CompatibilityStorage && DataStorage)
	{
		// We need to mark our owner as pending sync to keep the hierarchy up to date because our owner is not a CVDScene Object, we also need to manually flag
		// The containers we use to group particles by type, as pending sync
		ParticlesPendingTEDSTagSync.Emplace(CompatibilityStorage->FindRowWithCompatibleObject(GetOwner()));

		for (const TPair<EChaosVDParticleType, TSharedPtr<FChaosVDBaseSceneObject>>& ParticleContainer : ParticleSceneContainersByType)
		{
			ParticlesPendingTEDSTagSync.Emplace(ParticleContainer.Value->GetTedsRowHandle());
		}

		DataStorage->BatchAddRemoveColumns(ParticlesPendingTEDSTagSync, { FTypedElementSyncFromWorldTag::StaticStruct() }, {});
	}
}

void UChaosVDParticleDataComponent::BatchTEDSCombinedSyncFromWorldAndActiveTag(FTedsHandlesForBatchArray& ParticlesPendingCombinedTEDSTagSync, const UE::Editor::DataStorage::ICompatibilityProvider* CompatibilityStorage, UE::Editor::DataStorage::ICoreProvider* DataStorage)
{
	FMemMark StackMarker(FMemStack::Get());
	FTedsHandlesForBatchArray OwnersPendingTEDSWorldSyncTag;
	if (CompatibilityStorage && DataStorage)
	{
		// We need to mark our owner as pending sync to keep the hierarchy up to date because our owner is not a CVDScene Object, we also need to manually flag
		// The containers we use to group particles by type, as pending sync.
		// This is only needed for the World Sync tag, for the Active tag we don't need to update our owner's and container tags
		OwnersPendingTEDSWorldSyncTag.Emplace(CompatibilityStorage->FindRowWithCompatibleObject(GetOwner()));

		for (const TPair<EChaosVDParticleType, TSharedPtr<FChaosVDBaseSceneObject>>& ParticleContainer : ParticleSceneContainersByType)
		{
			OwnersPendingTEDSWorldSyncTag.Emplace(ParticleContainer.Value->GetTedsRowHandle());
		}

		BatchTEDSAddRemoveTags(DataStorage, OwnersPendingTEDSWorldSyncTag, { FTypedElementSyncFromWorldTag::StaticStruct() }, {});

		TArray<const UScriptStruct*, TInlineAllocator<2>> ColumnsToAdd = { FTypedElementSyncFromWorldTag::StaticStruct(),  FChaosVDActiveObjectTag::StaticStruct() };
		BatchTEDSAddRemoveTags(DataStorage, ParticlesPendingCombinedTEDSTagSync, ColumnsToAdd, {});
	}
}

void UChaosVDParticleDataComponent::BatchTEDSAddRemoveTags(UE::Editor::DataStorage::ICoreProvider* EditorDataStorage, FTedsHandlesForBatchArray& ParticlesPendingTEDSTagSync,
                                                           TConstArrayView<const UScriptStruct*> ColumnsToAdd, TConstArrayView<const UScriptStruct*> ColumnsToRemove)
{
	if (!EditorDataStorage)
	{
		return;
	}
	
	EditorDataStorage->BatchAddRemoveColumns(ParticlesPendingTEDSTagSync, ColumnsToAdd, ColumnsToRemove);
}

void UChaosVDParticleDataComponent::UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData)
{
	// Units of work representing each phase of the update loop (Pre-Processing Particle Data, Applying Particle data, Update TEDS Tags, and Removing Particles).
	// This needs to be updated if new phases are added or removed
	constexpr float AmountOfWork = 4.0f;

	const FText InitialProgressBarTitle = FText::Format(FTextFormat(LOCTEXT("UpdatingParticleDataMessage", "Updating Particle Data for {0} Solver with ID {1} ...")), FText::FromString(GetSolverName()), FText::AsNumber(SolverID));
	FScopedSlowTask UpdatingParticleData(AmountOfWork, InitialProgressBarTitle);
	UpdatingParticleData.MakeDialogDelayed(ChaosVDSceneUIOptions::DelayToShowProgressDialogThreshold, ChaosVDSceneUIOptions::bShowCancelButton, ChaosVDSceneUIOptions::bAllowInPIE);
	UpdatingParticleData.EnterProgressFrame(1.0f, InitialProgressBarTitle);

	TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return;
	}

	TQueue<TSharedPtr<FChaosVDSceneParticle>, EQueueMode::Mpsc> SceneParticleToProcessInGT;

	{
		using namespace UE::Editor::DataStorage;
		ICompatibilityProvider* Compatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
		ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

		// Make a pre-pass in parallel and process all the data that can be updated in parallel. This step will mostly copy data, and set flags of what needs to be updated in the Game Thread
		ParallelFor(InSolverFrameStageData.RecordedParticlesData.Num(),[this, &DataStorage, &Compatibility, &SceneParticleToProcessInGT, &InSolverFrameStageData, &InSolverFrameData](int32 ParticleIndex)
		{
			const TSharedPtr<FChaosVDParticleDataWrapper>& Particle = InSolverFrameStageData.RecordedParticlesData[ParticleIndex];
			if (!Particle)
			{
				return;
			}
	
			const int32 ParticleVDInstanceID = Particle ? Particle->ParticleIndex : INDEX_NONE;

			if (InSolverFrameStageData.ParticlesDestroyedIDs.Contains(ParticleVDInstanceID))
			{
				// Do not process the particle if it was destroyed in the same step
				return;
			}

			TSharedPtr<FChaosVDSceneParticle> ParticleInstanceToUpdate = nullptr;
			{
				FReadScopeLock ReadLock(ParticleSceneObjectsLock);
				ParticleInstanceToUpdate = GetParticleInstanceByID_AssumesLocked(ParticleVDInstanceID);
			}

			if (ParticleInstanceToUpdate)
			{
				// We have new data for this particle, so re-activate the existing actor
				if (!ParticleInstanceToUpdate->IsActive())
				{
					ParticleInstanceToUpdate->SetIsActive(true);
				}

				ParticleInstanceToUpdate->PreUpdateFromRecordedParticleData(Particle, InSolverFrameData.SimulationTransform);
			}
			else
			{
				{
					FWriteScopeLock WriteLock(ParticleSceneObjectsLock);

					TSharedRef<FChaosVDSceneParticle> NewParticleRef = CreateSceneParticle_AssumesLocked(Particle.ToSharedRef(), InSolverFrameData);

					NewParticleRef->SetTedsRowHandle(Chaos::VD::TedsUtils::AddObjectToDataStorage(&NewParticleRef.Get(), DataStorage, Compatibility));

					ParticleInstanceToUpdate = NewParticleRef;
				}

				ParticleInstanceToUpdate->PreUpdateFromRecordedParticleData(Particle, InSolverFrameData.SimulationTransform);

				StreamingSystem.EnqueuePendingTrackingOperation(ParticleInstanceToUpdate.ToSharedRef(), FChaosVDSceneStreaming::FPendingTrackingOperation::EType::AddOrUpdate);
			}

			SceneParticleToProcessInGT.Enqueue(ParticleInstanceToUpdate);
		});
	}

	using namespace UE::Editor::DataStorage;

	FMemMark StackMarker(FMemStack::Get());

	// Currently, updating TEDS tags is expensive, but the cost is reduced if we batch the update
	// therefore we will use this to gather all TEDS rows that need updating and do that in a single call later on
	TArray<RowHandle, TInlineAllocator<256, TMemStackAllocator<>>> ParticlesPendingTEDSSyncFromWorldTagSync;
	ParticlesPendingTEDSSyncFromWorldTagSync.Reserve(InSolverFrameStageData.RecordedParticlesData.Num());

	TSet<int32> ParticlesUpdatedIDsForKeyFrameDiff;
	if (Chaos::VisualDebugger::Cvars::bUnloadParticleDataUsingKeyFrameDiff)
	{
		ParticlesUpdatedIDsForKeyFrameDiff.Reserve(InSolverFrameStageData.RecordedParticlesData.Num());
	}

	FTedsHandlesForBatchArray ParticlesPendingTEDSActiveStateTagAddition;
	FTedsHandlesForBatchArray ParticlesPendingTEDSActiveStateAndSyncTagAddition;
	
	{
		const FText UpdatingSceneMessage = FText::Format(FTextFormat(LOCTEXT("UpdatingSceneMessage", "Updating Scene for {0} Solver with ID {1} ...")), FText::FromString(GetSolverName()), FText::AsNumber(SolverID));
		UpdatingParticleData.EnterProgressFrame(1.0f, UpdatingSceneMessage);

		// At this point all particle instances should have the latest data, and created where needed
		// So now we can go over them and perform any operation that needed to be executed in the GT (mostly related to interactions with the instances static mesh components
		// we use for visualization, and other GT only APIs in the editor)
		while (!SceneParticleToProcessInGT.IsEmpty())
		{
			TSharedPtr<FChaosVDSceneParticle> ParticleToProcessInGT;
			SceneParticleToProcessInGT.Dequeue(ParticleToProcessInGT);

			if (EnumHasAnyFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::Transform))
			{
				StreamingSystem.EnqueuePendingTrackingOperation(ParticleToProcessInGT.ToSharedRef(), FChaosVDSceneStreaming::FPendingTrackingOperation::EType::AddOrUpdate);	
			}

			ParticleToProcessInGT->ProcessPendingParticleDataUpdates();

			if (Chaos::VisualDebugger::Cvars::bUnloadParticleDataUsingKeyFrameDiff)
			{
				if (TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = ParticleToProcessInGT->GetParticleData())
				{
					ParticlesUpdatedIDsForKeyFrameDiff.Emplace(ParticleData->ParticleIndex);
				}
			}
			
			if (EnumHasAllFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::Active | EChaosVDSceneParticleDirtyFlags::TEDS))
			{
				ParticlesPendingTEDSActiveStateAndSyncTagAddition.Emplace(ParticleToProcessInGT->GetTedsRowHandle());
				EnumRemoveFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::Active | EChaosVDSceneParticleDirtyFlags::TEDS);
			}
			else
			{
				if (EnumHasAnyFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::Active))
				{
					// We could check if the particle is active before doing this, but at this point only newly active particles can be in the queue
					ParticlesPendingTEDSActiveStateTagAddition.Emplace(ParticleToProcessInGT->GetTedsRowHandle());
					EnumRemoveFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::Active);
				}

				if (EnumHasAnyFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::TEDS))
				{
					ParticlesPendingTEDSSyncFromWorldTagSync.Emplace(ParticleToProcessInGT->GetTedsRowHandle());
					EnumRemoveFlags(ParticleToProcessInGT->DirtyFlags, EChaosVDSceneParticleDirtyFlags::TEDS);
				}
			}
		}
	}

	FTedsHandlesForBatchArray ParticlesPendingTEDSActiveStateTagRemoval;
	ProcessRemovedParticles_AssumessLocked(ScenePtr.ToSharedRef(), InSolverFrameData, InSolverFrameStageData, ParticlesUpdatedIDsForKeyFrameDiff, ParticlesPendingTEDSActiveStateTagRemoval);

	const ICompatibilityProvider* EditorDataStorageCompatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	ICoreProvider* EditorDataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);

	BatchTEDSCombinedSyncFromWorldAndActiveTag(ParticlesPendingTEDSActiveStateAndSyncTagAddition, EditorDataStorageCompatibility, EditorDataStorage);
	BatchTEDSSyncFromWorldTag(ParticlesPendingTEDSSyncFromWorldTagSync, EditorDataStorageCompatibility, EditorDataStorage);
	BatchTEDSAddRemoveTags(EditorDataStorage, ParticlesPendingTEDSActiveStateAndSyncTagAddition, { FChaosVDActiveObjectTag::StaticStruct() } , {});
	BatchTEDSAddRemoveTags(EditorDataStorage, ParticlesPendingTEDSActiveStateTagRemoval, {}, { FChaosVDActiveObjectTag::StaticStruct() });
}

TSharedRef<FChaosVDSceneParticle> UChaosVDParticleDataComponent::CreateSceneParticle_AssumesLocked(const TSharedRef<FChaosVDParticleDataWrapper>& InParticleData, const FChaosVDSolverFrameData& InFrameData)
{
	using namespace Chaos;
	using namespace UE::Editor::DataStorage;

	TSharedRef<FChaosVDSceneParticle> NewRepresentation = MakeShared<FChaosVDSceneParticle>();

	TSharedPtr<FChaosVDBaseSceneObject> Container = GetParticleContainerByType_AssumesLocked(InParticleData->Type);
	NewRepresentation->SetParent(Container);

	NewRepresentation->SetScene(SceneWeakPtr);
	NewRepresentation->SetIsActive(true);
	NewRepresentation->SetIsServerParticle(IsServerData());

	if (!StreamingSystem.IsEnabled())
	{
		NewRepresentation->SetStreamingState(FChaosVDBaseSceneObject::EStreamingState::Visible);
	}

	SolverParticlesByID.Emplace(InParticleData->ParticleIndex, NewRepresentation);
	SolverParticlesArray.Emplace(StaticCastSharedRef<FChaosVDBaseSceneObject>(NewRepresentation));

	return NewRepresentation;
}

void UChaosVDParticleDataComponent::UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	Super::UpdateFromSolverFrameData(InSolverFrameData);
}

bool UChaosVDParticleDataComponent::IsServerData()
{
	if (bCachedIsServerData.IsSet())
	{
		return bCachedIsServerData.GetValue();
	}

	if (AChaosVDSolverInfoActor* AsSolverInfoActor = Cast<AChaosVDSolverInfoActor>(GetOwner()))
	{
		bCachedIsServerData = AsSolverInfoActor->GetIsServer();
		return bCachedIsServerData.GetValue();
	}

	// Particle data components should be part of a solver info actor, therefore we should not get here.
	return ensure(false);
}

const FString& UChaosVDParticleDataComponent::GetSolverName()
{
	if (bCachedSolverName.IsSet())
	{
		return bCachedSolverName.GetValue();
	}

	if (AChaosVDSolverInfoActor* AsSolverInfoActor = Cast<AChaosVDSolverInfoActor>(GetOwner()))
	{
		bCachedSolverName = AsSolverInfoActor->GetSolverName().ToString();
	}
	else
	{
		// Particle data components should be part of a solver info actor, therefore we should not get here.
		ensure(false);
		bCachedSolverName = TEXT("Unknown");
	}

	return bCachedSolverName.GetValue();
}

void UChaosVDParticleDataComponent::BatchApplySolverVisibilityToParticle(bool bNewIsVisible)
{
	using namespace UE::Editor::DataStorage;

	FReadScopeLock ReadLock(ParticleSceneObjectsLock);

	FMemMark StackMarker(FMemStack::Get());
	TArray<RowHandle, TInlineAllocator<256, TMemStackAllocator<>>> ParticlesPendingTEDSTagSync;
	ParticlesPendingTEDSTagSync.Reserve(GetMaxElementCountForBatchTEDSUpdate_AssumesLocked());

	for (const TPair<int32, TSharedRef<FChaosVDSceneParticle>>& ParticleWithIDPair : SolverParticlesByID)
	{
		if (bNewIsVisible)
		{
			ParticleWithIDPair.Value->RemoveHiddenFlag(EChaosVDHideParticleFlags::HiddenBySolverVisibility);
		}
		else
		{
			// Note: We should probably add a priority system for the hide requests
			// For now just clear the HideBySceneOutliner flag when a hide by solver request is done as this has priority
			ParticleWithIDPair.Value->RemoveHiddenFlag(EChaosVDHideParticleFlags::HiddenBySceneOutliner);
			ParticleWithIDPair.Value->AddHiddenFlag(EChaosVDHideParticleFlags::HiddenBySolverVisibility);
		}

		ParticlesPendingTEDSTagSync.Emplace(ParticleWithIDPair.Value->GetTedsRowHandle());

		ParticleWithIDPair.Value->UpdateGeometryComponentsVisibility();
	}

	const ICompatibilityProvider* EditorDataStorageCompatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
	ICoreProvider* EditorDataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
	BatchTEDSSyncFromWorldTag(ParticlesPendingTEDSTagSync, EditorDataStorageCompatibility, EditorDataStorage);
	
	if (TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->RequestUpdate();
	}
}

void UChaosVDParticleDataComponent::SetVisibility(bool bNewIsVisible)
{
	Super::SetVisibility(bNewIsVisible);

	BatchApplySolverVisibilityToParticle(bNewIsVisible);
}

TSharedPtr<FChaosVDSceneParticle> UChaosVDParticleDataComponent::GetParticleInstanceByID(int32 ParticleID) const
{
	FReadScopeLock ReadLock(ParticleSceneObjectsLock);
	return GetParticleInstanceByID_AssumesLocked(ParticleID);
}

TSharedPtr<FChaosVDSceneParticle> UChaosVDParticleDataComponent::GetParticleInstanceByID_AssumesLocked(int32 ParticleID) const
{
	const TSharedRef<FChaosVDSceneParticle>* FoundParticleActor = SolverParticlesByID.Find(ParticleID);
	return FoundParticleActor ? FoundParticleActor->ToSharedPtr() : nullptr;
}

void UChaosVDParticleDataComponent::HandleVisibilitySettingsUpdated(UObject* SettingsObject)
{
	BatchApplySolverVisibilityToParticle(IsVisible());
}

void UChaosVDParticleDataComponent::HandleColorsSettingsUpdated(UObject* SettingsObject)
{
	FReadScopeLock ReadLock(ParticleSceneObjectsLock);
	for (const TPair<int32, TSharedRef<FChaosVDSceneParticle>>& ParticleWithIDPair : SolverParticlesByID)
	{
		ParticleWithIDPair.Value->UpdateGeometryColors();
	}
}

TSharedPtr<FChaosVDBaseSceneObject> UChaosVDParticleDataComponent::GetParticleContainerByType(EChaosVDParticleType ParticleType)
{
	FReadScopeLock ReadLock(ParticleSceneObjectsLock);
	return GetParticleContainerByType_AssumesLocked(ParticleType);
}

TSharedPtr<FChaosVDBaseSceneObject> UChaosVDParticleDataComponent::GetParticleContainerByType_AssumesLocked(EChaosVDParticleType ParticleType)
{
	using namespace Chaos;
	using namespace UE::Editor::DataStorage;

	if (TSharedPtr<FChaosVDBaseSceneObject>* FoundContainerPtrPtr = ParticleSceneContainersByType.Find(ParticleType))
	{
		return *FoundContainerPtrPtr;
	}

	if (AActor* Owner = GetOwner())
	{
		TSharedPtr<FChaosVDBaseSceneObject> NewContainer = MakeShared<FChaosVDBaseSceneObject>();
	
		NewContainer->SetDisplayName(UEnum::GetDisplayValueAsText(ParticleType).ToString());
		NewContainer->SetParentParentActor(Owner);
		NewContainer->SetIconName(Owner->GetCustomIconName());

		NewContainer->SetTedsRowHandle(Chaos::VD::TedsUtils::AddObjectToDataStorage(NewContainer.Get()));

		ParticleSceneContainersByType.Add(ParticleType, NewContainer);

		return NewContainer;
	}

	return nullptr;
}

#undef LOCTEXT_NAMESPACE
