// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDParticleExtraDataComponent.h"

#include "ChaosVDModule.h"
#include "ChaosVDScene.h"
#include "ChaosVisualDebugger/ChaosVDParticleExtraData.h"
#include "ChaosVisualDebugger/ChaosVDStructCollectionMemWriterReader.h"
#include "DataWrappers/ChaosVDParticleExtraDataContainer.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "IDetailsView.h"
#include "UObject/Class.h"
#include "UObject/UObjectGlobals.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDParticleExtraDataComponent)

using namespace Chaos::VisualDebugger;

void UChaosVDParticleExtraDataComponent::ClearData()
{
	CurrentFrameContainer.Reset();
	CachedScopeEntries.Reset();
	CachedSolverID = INDEX_NONE;
	CachedParticleID = INDEX_NONE;
}

void UChaosVDParticleExtraDataComponent::UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	CurrentFrameContainer = InSolverFrameData.GetCustomData().GetData<FChaosVDParticleExtraDataContainer>();
}

void UChaosVDParticleExtraDataComponent::SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr)
{
	// Unbind from the previous scene
	if (TSharedPtr<FChaosVDScene> OldScene = SceneWeakPtr.Pin())
	{
		UnbindFromScene(OldScene);
	}

	Super::SetScene(InSceneWeakPtr);

	if (TSharedPtr<FChaosVDScene> NewScene = SceneWeakPtr.Pin())
	{
		ExtraDataDelegateHandle = NewScene->OnParticleDetailsExtraDataRequested().AddUObject(
			this, &UChaosVDParticleExtraDataComponent::HandleParticleDetailsExtraData);

		RefreshDelegateHandle = NewScene->OnParticleDetailsExtraDataRefreshRequested().AddUObject(
			this, &UChaosVDParticleExtraDataComponent::HandleParticleDetailsRefresh);
	}
}

void UChaosVDParticleExtraDataComponent::UnbindFromScene(const TSharedPtr<FChaosVDScene>& Scene)
{
	if (Scene)
	{
		if (ExtraDataDelegateHandle.IsValid())
		{
			Scene->OnParticleDetailsExtraDataRequested().Remove(ExtraDataDelegateHandle);
			ExtraDataDelegateHandle.Reset();
		}

		if (RefreshDelegateHandle.IsValid())
		{
			Scene->OnParticleDetailsExtraDataRefreshRequested().Remove(RefreshDelegateHandle);
			RefreshDelegateHandle.Reset();
		}
	}
}

void UChaosVDParticleExtraDataComponent::HandleParticleDetailsExtraData(int32 InSolverID, int32 InParticleID, IDetailLayoutBuilder& DetailBuilder)
{
	if (!CurrentFrameContainer)
	{
		return;
	}

	// Only handle data that belongs to our solver
	if (InSolverID != SolverID)
	{
		return;
	}

	const TMap<int32, TSharedPtr<FChaosVDParticleExtraData>>* DataByParticle =
		CurrentFrameContainer->DataBySolverAndParticleID.Find(SolverID);
	if (!DataByParticle)
	{
		return;
	}

	const TSharedPtr<FChaosVDParticleExtraData>* ExtraDataPtr = DataByParticle->Find(InParticleID);
	if (!ExtraDataPtr || !(*ExtraDataPtr))
	{
		return;
	}

	const FChaosVDParticleExtraData& ExtraData = **ExtraDataPtr;

	TSharedPtr<FChaosVDSerializableNameTable> NameTable = CurrentFrameContainer->NameTable;
	if (!NameTable)
	{
		return;
	}

	// Rebuild the scope cache when the observed particle changes.
	// For the same particle we reuse the existing FStructOnScope allocations so Slate can
	// pick up frame-to-frame value changes via HandleParticleDetailsRefresh without a rebuild.
	const bool bNeedsCacheRebuild = (InSolverID != CachedSolverID || InParticleID != CachedParticleID || CachedScopeEntries.IsEmpty());

	if (bNeedsCacheRebuild)
	{
		CachedScopeEntries.Reset();
		CachedSolverID = InSolverID;
		CachedParticleID = InParticleID;

		for (const FChaosVDExtraDataCategory& Category : ExtraData.Categories)
		{
			for (const FChaosVDExtraDataStructEntry& Entry : Category.Entries)
			{
				UScriptStruct* Struct = FindObject<UScriptStruct>(nullptr, *Entry.StructTypePath.ToString());
				if (!Struct)
				{
					UE_LOGF(LogChaosVDEditor, Warning,
						"UChaosVDParticleExtraDataComponent: Could not resolve UScriptStruct '%ls'  - type may not be loaded.",
						*Entry.StructTypePath.ToString());
					continue;
				}

				TSharedPtr<FStructOnScope> StructScope = MakeShared<FStructOnScope>(Struct);
				FChaosVDStructCollectionMemoryReader Reader(Entry.Bytes, NameTable.ToSharedRef());
				if (Entry.SerializationMode == EChaosVDExtraDataSerializationMode::SerializeBin)
				{
					Struct->SerializeBin(Reader, StructScope->GetStructMemory());
				}
				else
				{
					Struct->SerializeItem(Reader, StructScope->GetStructMemory(), nullptr);
				}

				if (Reader.IsError())
				{
					UE_LOGF(LogChaosVDEditor, Warning,
						"UChaosVDParticleExtraDataComponent: Failed to deserialize '%ls'  - skipping entry.",
						*Entry.StructTypePath.ToString());
					continue;
				}

				CachedScopeEntries.Add({ Category.CategoryName, Entry.StructTypePath, StructScope });
			}
		}
	}

	// CVD is a read-only viewer: mark all properties in this view as read-only so user-defined
	// structs with EditAnywhere properties do not show editable input chrome.
	if (TSharedPtr<IDetailsView> View = DetailBuilder.GetDetailsViewSharedPtr())
	{
		if (!View->GetIsPropertyReadOnlyDelegate().IsBound())
		{
			View->SetIsPropertyReadOnlyDelegate(FIsPropertyReadOnly::CreateLambda(
				[](const FPropertyAndParent&)
				{
					return true;
				}));
		}
	}

	// Add all cached scopes to the layout builder
	for (const FChaosVDExtraDataScopeEntry& ScopeEntry : CachedScopeEntries)
	{
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(ScopeEntry.CategoryName);
		if (IDetailPropertyRow* Row = CategoryBuilder.AddExternalStructure(ScopeEntry.Scope))
		{
			Row->DisplayName(ScopeEntry.Scope->GetStruct()->GetDisplayNameText());
			Row->ShouldAutoExpand(true);
			// CVD data is read-only: hide the reset-to-default arrow for the entire struct tree.
			Row->OverrideResetToDefault(FResetToDefaultOverride::Hide(/* bPropagateToChildren= */ true));
		}
	}
}

void UChaosVDParticleExtraDataComponent::HandleParticleDetailsRefresh(int32 InSolverID, int32 InParticleID, bool& bOutNeedsFullRebuild)
{
	// Only act if there are cached scopes for exactly this particle
	if (CachedScopeEntries.IsEmpty() || InSolverID != CachedSolverID || InParticleID != CachedParticleID)
	{
		return;
	}

	if (!CurrentFrameContainer || InSolverID != SolverID)
	{
		return;
	}

	const TMap<int32, TSharedPtr<FChaosVDParticleExtraData>>* DataByParticle =
		CurrentFrameContainer->DataBySolverAndParticleID.Find(InSolverID);
	if (!DataByParticle)
	{
		return;
	}

	const TSharedPtr<FChaosVDParticleExtraData>* ExtraDataPtr = DataByParticle->Find(InParticleID);
	if (!ExtraDataPtr || !(*ExtraDataPtr))
	{
		return;
	}

	const FChaosVDParticleExtraData& ExtraData = **ExtraDataPtr;

	TSharedPtr<FChaosVDSerializableNameTable> NameTable = CurrentFrameContainer->NameTable;
	if (!NameTable)
	{
		return;
	}

	// Walk the new frame data in lock-step with the cached scope entries.
	// If the structure is stable (same categories and struct types in the same order),
	// deserialize the new bytes directly into the existing FStructOnScope memory so
	// Slate picks up the changes on the next redraw  - no layout rebuild required.
	int32 ScopeIndex = 0;
	for (const FChaosVDExtraDataCategory& Category : ExtraData.Categories)
	{
		for (const FChaosVDExtraDataStructEntry& Entry : Category.Entries)
		{
			if (ScopeIndex >= CachedScopeEntries.Num())
			{
				// More entries in the new frame than in the cache  - layout has grown.
				bOutNeedsFullRebuild = true;
				return;
			}

			FChaosVDExtraDataScopeEntry& ScopeEntry = CachedScopeEntries[ScopeIndex];

			// If the layout has changed (different category or struct type), request a full rebuild
			// so CustomizeDetails re-runs and the new structure is reflected correctly.
			if (ScopeEntry.CategoryName != Category.CategoryName || ScopeEntry.StructTypePath != Entry.StructTypePath)
			{
				bOutNeedsFullRebuild = true;
				return;
			}

			if (UScriptStruct* Struct = const_cast<UScriptStruct*>(Cast<const UScriptStruct>(ScopeEntry.Scope->GetStruct())))
			{
				FChaosVDStructCollectionMemoryReader Reader(Entry.Bytes, NameTable.ToSharedRef());
				if (Entry.SerializationMode == EChaosVDExtraDataSerializationMode::SerializeBin)
				{
					Struct->SerializeBin(Reader, ScopeEntry.Scope->GetStructMemory());
				}
				else
				{
					Struct->SerializeItem(Reader, ScopeEntry.Scope->GetStructMemory(), nullptr);
				}
			}

			++ScopeIndex;
		}
	}
}
