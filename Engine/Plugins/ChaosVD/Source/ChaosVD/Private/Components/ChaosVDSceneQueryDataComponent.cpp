// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDSceneQueryDataComponent.h"

#include "ChaosVDMiscSceneObjectTypeNamesForTesting.h"
#include "ChaosVDRecording.h"
#include "ChaosVDScene.h"
#include "ChaosVDSceneCompositionReport.h"
#include "Actors/ChaosVDSolverInfoActor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSceneQueryDataComponent)

UChaosVDSceneQueryDataComponent::UChaosVDSceneQueryDataComponent()
{
	PrimaryComponentTick.bCanEverTick = false;
	
	SetCanEverAffectNavigation(false);
	bNavigationRelevant = false;
}

void UChaosVDSceneQueryDataComponent::ClearSceneQuerySelection()
{
	const AChaosVDSolverInfoActor* SolverInfo = Cast<AChaosVDSolverInfoActor>(GetOwner());
	const TSharedPtr<FChaosVDScene> CVDScene = SolverInfo ? SolverInfo->GetScene().Pin() : nullptr;

	if (TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelectionObject = CVDScene ? CVDScene->GetSolverDataSelectionObject().Pin() : nullptr)
	{
		TSharedPtr<FChaosVDSolverDataSelectionHandle> SelectionHandle = SolverDataSelectionObject->GetCurrentSelectionHandle();
		if (SelectionHandle && SelectionHandle->IsA<FChaosVDQueryDataWrapper>())
		{
			SolverDataSelectionObject->SelectData(nullptr);
		}
	}
}

void UChaosVDSceneQueryDataComponent::ProccessSQData(const TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>>* RecordedQueriesByQueryID)
{
	if (!RecordedQueriesByQueryID)
	{
		ClearSceneQuerySelection();
		ClearData();
		return;
	}
	
	const int32 RecordedQueriesNum = RecordedQueriesByQueryID->Num();

	RecordedQueriesByType.Empty(RecordedQueriesNum);
	RecordedQueriesByID.Empty(RecordedQueriesNum);
	RecordedQueries.Empty(RecordedQueriesNum);

	for (const TPair<int32, TSharedPtr<FChaosVDQueryDataWrapper>>& QueryIDPair : (*RecordedQueriesByQueryID))
	{
		if (TSharedPtr<FChaosVDQueryDataWrapper> QueryData = QueryIDPair.Value)
		{
			TArray<TSharedPtr<FChaosVDQueryDataWrapper>>& QueriesForType = RecordedQueriesByType.FindOrAdd(QueryData->Type);
			QueriesForType.Emplace(QueryData);

			RecordedQueriesByID.Add(QueryIDPair.Key, QueryData);
			RecordedQueries.Add(QueryData);
		}
	}

	ClearSceneQuerySelection();
}

void UChaosVDSceneQueryDataComponent::UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData)
{
	const TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>>* RecordedQueriesByQueryID = nullptr;

	if (TSharedPtr<FChaosVDSceneQueriesDataContainer> SQDataContainer = InGameFrameData.GetCustomDataHandler().GetData<FChaosVDSceneQueriesDataContainer>())
	{
		RecordedQueriesByQueryID = SQDataContainer->RecordedSceneQueriesBySolverID.Find(SolverID);
	}

	ProccessSQData(RecordedQueriesByQueryID);
}

TConstArrayView<TSharedPtr<FChaosVDQueryDataWrapper>> UChaosVDSceneQueryDataComponent::GetQueriesByType(EChaosVDSceneQueryType Type) const
{
	if (const TArray<TSharedPtr<FChaosVDQueryDataWrapper>>* FoundQueries = RecordedQueriesByType.Find(Type))
	{
		return MakeArrayView(*FoundQueries);
	}

	return TArrayView<TSharedPtr<FChaosVDQueryDataWrapper>>();
}

TConstArrayView<TSharedPtr<FChaosVDQueryDataWrapper>> UChaosVDSceneQueryDataComponent::GetAllQueries() const
{
	return RecordedQueries;
}

TSharedPtr<FChaosVDQueryDataWrapper> UChaosVDSceneQueryDataComponent::GetQueryByID(int32 QueryID) const
{
	if (const TSharedPtr<FChaosVDQueryDataWrapper>* FoundQuery = RecordedQueriesByID.Find(QueryID))
	{
		return *FoundQuery;
	}

	return nullptr;
}

void UChaosVDSceneQueryDataComponent::ClearData()
{
	RecordedQueriesByType.Reset();
	RecordedQueriesByID.Reset();
	RecordedQueries.Reset();
}

void UChaosVDSceneQueryDataComponent::AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData)
{
	Super::AppendSceneCompositionTestData(OutStateTestData);

	int32& CurrentCount = OutStateTestData.ObjectsCountByType.FindOrAdd(Chaos::VD::Test::SceneObjectTypes::SceneQuery);
	CurrentCount+= RecordedQueries.Num();
}
