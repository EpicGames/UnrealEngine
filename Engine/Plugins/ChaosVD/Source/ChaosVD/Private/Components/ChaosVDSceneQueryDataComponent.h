// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#include "Components/ChaosVDSolverDataComponent.h"
#include "ChaosVDSolverDataSelection.h"

#include "ChaosVDSceneQueryDataComponent.generated.h"

struct FChaosVDGameFrameData;

/** Actor Component that contains all the scene queries recorded at the current loaded frame */
UCLASS()
class UChaosVDSceneQueryDataComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()

public:
	UChaosVDSceneQueryDataComponent();
	void ClearSceneQuerySelection();

	virtual void UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData) override;

	TConstArrayView<TSharedPtr<FChaosVDQueryDataWrapper>> GetQueriesByType(EChaosVDSceneQueryType Type) const;
	TConstArrayView<TSharedPtr<FChaosVDQueryDataWrapper>> GetAllQueries() const;
	TSharedPtr<FChaosVDQueryDataWrapper> GetQueryByID(int32 QueryID) const;

	virtual void ClearData() override;

	virtual void AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData) override;

protected:

	void ProccessSQData(const TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>>* RecordedQueriesByQueryID);
	
	TMap<EChaosVDSceneQueryType, TArray<TSharedPtr<FChaosVDQueryDataWrapper>>> RecordedQueriesByType;
	TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>> RecordedQueriesByID;
	TArray<TSharedPtr<FChaosVDQueryDataWrapper>> RecordedQueries;
};
