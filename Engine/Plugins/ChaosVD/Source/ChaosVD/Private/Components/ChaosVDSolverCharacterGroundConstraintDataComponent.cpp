// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDSolverCharacterGroundConstraintDataComponent.h"

#include "ChaosVDCharacterGroundConstraintDataProviderInterface.h"
#include "ChaosVDMiscSceneObjectTypeNamesForTesting.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "ChaosVDScene.h"
#include "ChaosVDSceneCompositionReport.h"
#include "ChaosVDSettingsManager.h"
#include "Selection.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Settings/ChaosVDCharacterConstraintsVisualizationSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSolverCharacterGroundConstraintDataComponent)

void UChaosVDSolverCharacterGroundConstraintDataComponent::HandleSceneUpdated()
{
	if (const UChaosVDCharacterConstraintsVisualizationSettings* CharacterConstraintsVisualizationSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCharacterConstraintsVisualizationSettings>())
	{
		if (!CharacterConstraintsVisualizationSettings->bAutoSelectConstraintFromSelectedParticle)
		{
			return;
		}
	}

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return;
	}

	AChaosVDSolverInfoActor* OwnerSolverData = Cast<AChaosVDSolverInfoActor>(GetOwner());
	if (!OwnerSolverData)
	{
		return;
	}

	UChaosVDParticleDataComponent* ParticleDataComponent = OwnerSolverData->GetParticleDataComponent();
	if (!ParticleDataComponent)
	{
		return;
	}

	TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelection = ScenePtr->GetSolverDataSelectionObject().Pin();
	if(!SolverDataSelection)
	{
		return;
	}

	if (FChaosVDSceneParticle* ParticleInstance = ParticleDataComponent->GetSelectedParticle())
	{
		if (ParticleInstance->HasCharacterGroundConstraintData())
		{
			TArray<TSharedPtr<FChaosVDCharacterGroundConstraint>> FoundConstraintData;
			ParticleInstance->GetCharacterGroundConstraintData(FoundConstraintData);
			SolverDataSelection->SelectData(SolverDataSelection->MakeSelectionHandle(FoundConstraintData[0]));
		}
	}
}

void UChaosVDSolverCharacterGroundConstraintDataComponent::BeginDestroy()
{
	Super::BeginDestroy();

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return;
	}
	
	ScenePtr->OnSceneUpdated().RemoveAll(this);
}

void UChaosVDSolverCharacterGroundConstraintDataComponent::AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData)
{
	Super::AppendSceneCompositionTestData(OutStateTestData);

	int32& CurrentCount = OutStateTestData.ObjectsCountByType.FindOrAdd(Chaos::VD::Test::SceneObjectTypes::CharacterGroundConstraint);
	CurrentCount+= AllConstraints.Num();
}

void UChaosVDSolverCharacterGroundConstraintDataComponent::SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr)
{
	Super::SetScene(InSceneWeakPtr);
	
	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return;
	}
	
	ScenePtr->OnSceneUpdated().AddUObject(this, &UChaosVDSolverCharacterGroundConstraintDataComponent::HandleSceneUpdated);
}

void UChaosVDSolverCharacterGroundConstraintDataComponent::UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	UpdateConstraintData(InSolverFrameData.RecordedCharacterGroundConstraints);
}

