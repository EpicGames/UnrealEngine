// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDConstraintDataComponent.h"
#include "ChaosVDSolverCharacterGroundConstraintDataComponent.generated.h"

#define UE_API CHAOSVD_API

UCLASS(MinimalAPI)
class UChaosVDSolverCharacterGroundConstraintDataComponent : public UChaosVDConstraintDataComponent
{
	GENERATED_BODY()
public:
	UE_API virtual void SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr) override;

	UE_API virtual void UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData) override;

	UE_API void HandleSceneUpdated();

	UE_API virtual void BeginDestroy() override;

	virtual void AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData) override;
};

#undef UE_API
