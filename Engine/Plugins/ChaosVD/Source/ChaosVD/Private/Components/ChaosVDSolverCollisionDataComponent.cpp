// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDSolverCollisionDataComponent.h"

#include "ChaosVDRecording.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSolverCollisionDataComponent)

UChaosVDSolverCollisionDataComponent::UChaosVDSolverCollisionDataComponent()
{
	SetCanEverAffectNavigation(false);
	bNavigationRelevant = false;
}

void UChaosVDSolverCollisionDataComponent::UpdateCollisionData(const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>& InMidPhaseData)
{
	ClearData();

	AllMidPhases.Reserve(InMidPhaseData.Num());
	AllMidPhases = InMidPhaseData;

	MidPhasesByParticleID0.Reserve(InMidPhaseData.Num());
	MidPhasesByParticleID1.Reserve(InMidPhaseData.Num());

	for (const TSharedPtr<FChaosVDParticlePairMidPhase>& ParticleIDMidPhase : InMidPhaseData)
	{
		Chaos::VisualDebugger::Utils::AddDataDataToParticleIDMap(MidPhasesByParticleID0, ParticleIDMidPhase, ParticleIDMidPhase->Particle0Idx);
		Chaos::VisualDebugger::Utils::AddDataDataToParticleIDMap(MidPhasesByParticleID1, ParticleIDMidPhase, ParticleIDMidPhase->Particle1Idx);

		ConstraintsByParticleID0.Reserve(ParticleIDMidPhase->Constraints.Num());
		ConstraintsByParticleID1.Reserve(ParticleIDMidPhase->Constraints.Num());
		for (FChaosVDConstraint& Constraint : ParticleIDMidPhase->Constraints)
		{
			Chaos::VisualDebugger::Utils::AddDataDataToParticleIDMap<FChaosVDConstraintByParticleMap, FChaosVDConstraint*>(ConstraintsByParticleID0, &Constraint, Constraint.Particle0Index);
			Chaos::VisualDebugger::Utils::AddDataDataToParticleIDMap<FChaosVDConstraintByParticleMap, FChaosVDConstraint*>(ConstraintsByParticleID1, &Constraint, Constraint.Particle1Index);
		}
	}
}

void UChaosVDSolverCollisionDataComponent::UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData)
{
	if (EnumHasAnyFlags(InSolverFrameStageData.StageFlags, EChaosVDSolverStageFlags::ExplicitStage))
	{
		UpdateCollisionData(InSolverFrameStageData.RecordedMidPhases);
	}
}

const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>* UChaosVDSolverCollisionDataComponent::GetMidPhasesForParticle(int32 ParticleID, EChaosVDParticlePairSlot Options) const
{
	return Chaos::VisualDebugger::Utils::GetDataFromParticlePairMaps<FChaosVDMidPhaseByParticleMap, TSharedPtr<FChaosVDParticlePairMidPhase>>(MidPhasesByParticleID0, MidPhasesByParticleID1, ParticleID, Options);
}

const TArray<FChaosVDConstraint*>* UChaosVDSolverCollisionDataComponent::GetConstraintsForParticle(int32 ParticleID, EChaosVDParticlePairSlot Options) const
{
	return Chaos::VisualDebugger::Utils::GetDataFromParticlePairMaps<FChaosVDConstraintByParticleMap, FChaosVDConstraint*>(ConstraintsByParticleID0, ConstraintsByParticleID1, ParticleID, Options);
}

void UChaosVDSolverCollisionDataComponent::ClearData()
{
	AllMidPhases.Reset();
	MidPhasesByParticleID0.Reset();
	MidPhasesByParticleID1.Reset();
	ConstraintsByParticleID0.Reset();
	ConstraintsByParticleID1.Reset();
}
