// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDConstraintDataHelpers.h"

#include "Components/ChaosVDSolverDataComponent.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "GameFramework/Info.h"
#include "Templates/SharedPointer.h"

#include "ChaosVDSolverCollisionDataComponent.generated.h"

struct FChaosVDConstraint;

typedef TMap<int32, TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>> FChaosVDMidPhaseByParticleMap;
typedef TMap<int32, TArray<FChaosVDConstraint*>> FChaosVDConstraintByParticleMap;

UCLASS()
class UChaosVDSolverCollisionDataComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()

public:
	UChaosVDSolverCollisionDataComponent();

	virtual void UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData) override;

	const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>& GetMidPhases() const
	{
		return AllMidPhases;
	}

	const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>* GetMidPhasesForParticle(int32 ParticleID, EChaosVDParticlePairSlot Options) const;
	const TArray<FChaosVDConstraint*>* GetConstraintsForParticle(int32 ParticleID, EChaosVDParticlePairSlot Options) const;

	virtual void ClearData() override;

protected:
	
	void UpdateCollisionData(const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>& InMidPhaseData);

	TArray<TSharedPtr<FChaosVDParticlePairMidPhase>> AllMidPhases;
	FChaosVDMidPhaseByParticleMap MidPhasesByParticleID0;
	FChaosVDMidPhaseByParticleMap MidPhasesByParticleID1;

	FChaosVDConstraintByParticleMap ConstraintsByParticleID0;
	FChaosVDConstraintByParticleMap ConstraintsByParticleID1;

};