// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDSolverDataComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSolverDataComponent)

void UChaosVDSolverDataComponent::SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr)
{
	SceneWeakPtr = InSceneWeakPtr;
}

void UChaosVDSolverDataComponent::SetVisibility(bool bNewIsVisible)
{
	bIsVisible = bNewIsVisible;
}
