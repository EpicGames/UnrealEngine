// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDSolverJointConstraintDataComponent.h"

#include "ChaosVDMiscSceneObjectTypeNamesForTesting.h"
#include "ChaosVDRecording.h"
#include "ChaosVDSceneCompositionReport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSolverJointConstraintDataComponent)

void UChaosVDSolverJointConstraintDataComponent::UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData)
{
	if (EnumHasAnyFlags(InSolverFrameStageData.StageFlags, EChaosVDSolverStageFlags::ExplicitStage))
	{
		UpdateConstraintData(InSolverFrameStageData.RecordedJointConstraints);
	}
}

void UChaosVDSolverJointConstraintDataComponent::AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData)
{
	Super::AppendSceneCompositionTestData(OutStateTestData);

	int32& CurrentCount = OutStateTestData.ObjectsCountByType.FindOrAdd(Chaos::VD::Test::SceneObjectTypes::JointConstraint);
	CurrentCount+= AllConstraints.Num();
}
