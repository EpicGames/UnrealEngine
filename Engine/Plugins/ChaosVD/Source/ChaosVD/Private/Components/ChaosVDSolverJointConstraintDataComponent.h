// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDConstraintDataComponent.h"
#include "ChaosVDSolverJointConstraintDataComponent.generated.h"

#define UE_API CHAOSVD_API


UCLASS(MinimalAPI)
class UChaosVDSolverJointConstraintDataComponent : public UChaosVDConstraintDataComponent
{
	GENERATED_BODY()

	UE_API virtual void UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData) override;

	virtual void AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData) override;
};

#undef UE_API
