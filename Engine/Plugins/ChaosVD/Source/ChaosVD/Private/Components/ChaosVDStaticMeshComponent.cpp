// Copyright Epic Games, Inc. All Rights Reserved.

#include "Components/ChaosVDStaticMeshComponent.h"

#include "ChaosVDGeometryBuilder.h"
#include "ChaosVDModule.h"
#include "Materials/Material.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Widgets/SChaosVDEnumFlagsMenu.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDStaticMeshComponent)


uint32 UChaosVDStaticMeshComponent::GetGeometryKey() const
{
	return CurrentGeometryKey;
}

void UChaosVDStaticMeshComponent::UpdateVisibilityForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	if (InInstanceHandle->GetMeshComponent() != this)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Attempted to update a mesh instance using a handle from another component. No instances were updated | Handle Component [%ls] | Current Component [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(InInstanceHandle->GetMeshComponent()), *GetNameSafe(this));
		return;
	}

	SetVisibility(InInstanceHandle->GetState().bIsVisible);
}

void UChaosVDStaticMeshComponent::UpdateSelectionStateForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	bIsOwningParticleSelected = InInstanceHandle->GetState().bIsSelected;

	PushSelectionToProxy();
}

bool UChaosVDStaticMeshComponent::ShouldRenderSelected() const
{
	return bIsOwningParticleSelected;
}

void UChaosVDStaticMeshComponent::UpdateColorForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	if (InInstanceHandle->GetMeshComponent() != this)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Attempted to update a mesh instance using a handle from another component. No instances were updated | Handle Component [%ls] | Current Component [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(InInstanceHandle->GetMeshComponent()), *GetNameSafe(this));
		return;
	}

	FLinearColor NewColor = InInstanceHandle->GetInstanceColor();

	const bool bIsSolidColor = FMath::IsNearlyEqual(NewColor.A, 1.0f);

	bool bHasOpaqueMaterial = !EnumHasAnyFlags(MeshComponentAttributeFlags, EChaosVDMeshAttributesFlags::TranslucentGeometry);

	constexpr int32 ColorPrimitiveDataIndex = 0;
	SetCustomPrimitiveDataVector4(ColorPrimitiveDataIndex, NewColor);

	// If we want to show a color with transparency, we might need to change our material
	if (bHasOpaqueMaterial != bIsSolidColor)
	{
		TSharedPtr<FChaosVDGeometryBuilder> GeometryBuilder = GeometryBuilderWeakPtr.Pin();
		if (ensure(GeometryBuilder))
		{
			EmptyOverrideMaterials();

			Chaos::VisualDebugger::Utils::EnumAddToggleFlag(MeshComponentAttributeFlags, EChaosVDMeshAttributesFlags::TranslucentGeometry);
			
			GeometryBuilder->RequestMaterialUpdate(this);
		}
	}
}

void UChaosVDStaticMeshComponent::UpdateWorldTransformForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle)
{
	if (InInstanceHandle->GetMeshComponent() != this)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Attempted to update a mesh instance using a handle from another component. No instances were updated | Handle Component [%ls] | Current Component [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GetNameSafe(InInstanceHandle->GetMeshComponent()), *GetNameSafe(this));
		return;
	}

	SetWorldTransform(InInstanceHandle->GetWorldTransform());
}

void UChaosVDStaticMeshComponent::Reset()
{
	bIsMeshReady = false;
	bIsDestroyed = false;
	MeshReadyDelegate = FChaosVDMeshReadyDelegate();
	ComponentEmptyDelegate = FChaosVDMeshComponentEmptyDelegate();

	SetStaticMesh(nullptr);

	CurrentMeshDataHandle = nullptr;
	CurrentGeometryKey = 0;
}

TSharedPtr<FChaosVDInstancedMeshData> UChaosVDStaticMeshComponent::AddMeshInstance(const FTransform InstanceTransform, bool bIsWorldSpace, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InGeometryHandle, int32 ParticleID, int32 SolverID)
{
	// Static meshes only have one instance
	constexpr int32 StaticMeshInstanceIndex = 0;

	CurrentMeshDataHandle = MakeShared<FChaosVDInstancedMeshData>(StaticMeshInstanceIndex, this, ParticleID, SolverID, InGeometryHandle);

	const uint32 NewHandleGeometryKey = InGeometryHandle->GetGeometryKey();

	if (!UpdateGeometryKey(NewHandleGeometryKey))
	{
		return nullptr;
	}

	CurrentMeshDataHandle->SetWorldTransform(InstanceTransform);

	return CurrentMeshDataHandle;
}

void UChaosVDStaticMeshComponent::AddExistingMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InMeshDataHandle)
{
	// Static meshes only have one instance
	constexpr int32 StaticMeshInstanceIndex = 0;

	const uint32 NewHandleGeometryKey = InMeshDataHandle->ExtractedGeometryHandle->GetGeometryKey();
	if (!UpdateGeometryKey(NewHandleGeometryKey))
	{
		return;
	}

	InMeshDataHandle->SetMeshInstanceIndex(StaticMeshInstanceIndex);
	InMeshDataHandle->SetMeshComponent(this);
}

void UChaosVDStaticMeshComponent::RemoveMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InHandleToRemove, ERemovalMode Mode)
{
	SetStaticMesh(nullptr);
	CurrentMeshDataHandle = nullptr;
	ComponentEmptyDelegate.Broadcast(this);
}

void UChaosVDStaticMeshComponent::SetGeometryBuilder(TWeakPtr<FChaosVDGeometryBuilder> GeometryBuilder)
{
	GeometryBuilderWeakPtr = GeometryBuilder;
}

EChaosVDMaterialType UChaosVDStaticMeshComponent::GetMaterialType() const
{
	if (EnumHasAnyFlags(MeshComponentAttributeFlags, EChaosVDMeshAttributesFlags::TranslucentGeometry))
	{
		return  EChaosVDMaterialType::SMTranslucent;
	}

	return EChaosVDMaterialType::SMOpaque;
}

void UChaosVDStaticMeshComponent::OnDisposed()
{
	Reset();

	bIsDestroyed = true;

	SetRelativeTransform(FTransform::Identity);

	if (IsRegistered())
	{
		UnregisterComponent();
	}

	if (AActor* Owner =GetOwner())
	{
		Owner->RemoveOwnedComponent(this);
	}
}

bool UChaosVDStaticMeshComponent::UpdateGeometryKey(uint32 NewHandleGeometryKey)
{
	if (CurrentGeometryKey != 0 && CurrentGeometryKey != NewHandleGeometryKey)
	{
		ensure(false);

		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Attempted to add a mesh instance belonging to another geometry key. No instance was added | CurrentKey [%u] | New Key [%u]", ANSI_TO_TCHAR(__FUNCTION__), CurrentGeometryKey, NewHandleGeometryKey);
		return false;
	}
	else
	{
		CurrentGeometryKey = NewHandleGeometryKey;
	}

	return true;
}

TSharedPtr<FChaosVDInstancedMeshData> UChaosVDStaticMeshComponent::GetMeshDataInstanceHandle(int32 InstanceIndex) const
{
	return CurrentMeshDataHandle;
}

void UChaosVDStaticMeshComponent::Initialize()
{
	TSharedPtr<FChaosVDGeometryBuilder> GeometryBuilder = GeometryBuilderWeakPtr.Pin();
	if (ensure(GeometryBuilder))
	{
		GeometryBuilder->RequestMaterialUpdate(this);
	}
}
