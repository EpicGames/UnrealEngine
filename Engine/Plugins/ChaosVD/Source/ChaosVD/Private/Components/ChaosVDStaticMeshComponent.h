// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/StaticMeshComponent.h"
#include "ChaosVDGeometryDataComponent.h"
#include "Interfaces/ChaosVDPooledObject.h"
#include "ChaosVDStaticMeshComponent.generated.h"

/** CVD version of a Static Mesh Component that holds additional CVD data */
UCLASS(HideCategories=("Transform"))
class UChaosVDStaticMeshComponent : public UStaticMeshComponent, public IChaosVDGeometryComponent, public IChaosVDPooledObject
{
	GENERATED_BODY()

public:
	UChaosVDStaticMeshComponent(const FObjectInitializer& ObjectInitializer)
		: Super(ObjectInitializer)
	{
		SetCanEverAffectNavigation(false);
		bNavigationRelevant = false;
		bOverrideWireframeColor = true;
		WireframeColorOverride = FColor::White;
	}

	// BEGIN IChaosVDGeometryDataComponent Interface

	virtual bool IsMeshReady() const override { return bIsMeshReady; }
	
	virtual void SetIsMeshReady(bool bIsReady) override { bIsMeshReady = bIsReady; }

	virtual FChaosVDMeshReadyDelegate* OnMeshReady() override { return &MeshReadyDelegate; }
	
	virtual FChaosVDMeshComponentEmptyDelegate* OnComponentEmpty() override { return &ComponentEmptyDelegate; }

	virtual uint32 GetGeometryKey() const override;
	virtual void UpdateVisibilityForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) override;

	virtual void UpdateSelectionStateForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) override;

	virtual bool ShouldRenderSelected() const override;

	virtual void UpdateColorForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) override;
	virtual void UpdateWorldTransformForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) override;

	virtual void SetMeshComponentAttributeFlags(EChaosVDMeshAttributesFlags Flags) override { MeshComponentAttributeFlags = Flags; };
	virtual EChaosVDMeshAttributesFlags GetMeshComponentAttributeFlags() const override { return MeshComponentAttributeFlags; };

	virtual TSharedPtr<FChaosVDInstancedMeshData> GetMeshDataInstanceHandle(int32 InstanceIndex) const override;

	virtual void Initialize() override;
	virtual void Reset() override;

	virtual TSharedPtr<FChaosVDInstancedMeshData> AddMeshInstance(const FTransform InstanceTransform, bool bIsWorldSpace, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InGeometryHandle, int32 ParticleID, int32 SolverID) override;
	virtual void AddExistingMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InMeshDataHandle) override;
	virtual void RemoveMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InHandleToRemove, ERemovalMode Mode = ERemovalMode::Deferred) override;

	virtual void SetGeometryBuilder(TWeakPtr<FChaosVDGeometryBuilder> GeometryBuilder) override;

	virtual EChaosVDMaterialType GetMaterialType() const override;

	virtual bool GetIsDestroyed() const override
	{
		return bIsDestroyed;
	}

	virtual void SetIsDestroyed(bool bNewIsPendingDestroy) override
	{
		bIsDestroyed = bNewIsPendingDestroy;
	}

	// END IChaosVDGeometryDataComponent Interface

	// BEGIN IChaosVDPooledObject Interface
	virtual void OnAcquired() override
	{
		bIsDestroyed = false;
	}

	virtual void OnDisposed() override;
	// END IChaosVDPooledObject Interface

protected:

	bool UpdateGeometryKey(uint32 NewHandleGeometryKey);

	EChaosVDMeshAttributesFlags MeshComponentAttributeFlags = EChaosVDMeshAttributesFlags::None;
	uint32 CurrentGeometryKey = 0;
	bool bIsMeshReady = false;
	bool bIsOwningParticleSelected = false;
	FChaosVDMeshReadyDelegate MeshReadyDelegate;
	FChaosVDMeshComponentEmptyDelegate ComponentEmptyDelegate;
	
	bool bIsDestroyed = false;

	TSharedPtr<FChaosVDInstancedMeshData> CurrentMeshDataHandle = nullptr;

	TSharedPtr<FChaosVDExtractedGeometryDataHandle> CurrentGeometryHandle = nullptr;
	
	TWeakPtr<FChaosVDGeometryBuilder> GeometryBuilderWeakPtr;
};
