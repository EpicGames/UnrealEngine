// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/ChaosVDDetailsCustomizationUtils.h"

#include "ChaosVDModule.h"
#include "ChaosVDScene.h"
#include "CollisionQueryParams.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Engine/EngineTypes.h"
#include "IDetailGroup.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void FChaosVDDetailsCustomizationUtils::HideAllCategories(IDetailLayoutBuilder& DetailBuilder, const TSet<FName>& AllowedCategories)
{
	// Hide everything as the only thing we want to show in these actors is the Recorded debug data
	TArray<FName> CurrentCategoryNames;
	DetailBuilder.GetCategoryNames(CurrentCategoryNames);
	for (const FName& CategoryToHide : CurrentCategoryNames)
	{
		if (!AllowedCategories.Contains(CategoryToHide))
		{
			DetailBuilder.HideCategory(CategoryToHide);
		}
	}
}

void FChaosVDDetailsCustomizationUtils::HideInvalidCVDDataWrapperProperties(TConstArrayView<TSharedPtr<IPropertyHandle>> InPropertyHandles)
{
	if (InPropertyHandles.Num() == 0)
	{
		return;
	}

	for (const TSharedPtr<IPropertyHandle>& Handle : InPropertyHandles)
	{
		bool bIsParticleDataStruct;
		if (Handle && !HasValidCVDWrapperData(Handle, bIsParticleDataStruct))
		{
			// TODO: This doesn't work in all cases. It seems this just sets the IsCustom flag on, and that is why it is hidden but depends on how it is being customized
			// We need to find a more reliable way of hiding it
			Handle->MarkHiddenByCustomization();
		}
	}
}

void FChaosVDDetailsCustomizationUtils::HideInvalidCVDDataWrapperProperties(TConstArrayView<TSharedRef<IPropertyHandle>> InPropertyHandles, IDetailLayoutBuilder& DetailBuilder)
{
	for (const TSharedRef<IPropertyHandle>& PropertyHandle : InPropertyHandles)
	{
		bool bIsParticleDataStruct = false;
		if (!HasValidCVDWrapperData(PropertyHandle, bIsParticleDataStruct))
		{
			if (bIsParticleDataStruct)
			{
				DetailBuilder.HideProperty(PropertyHandle);
			}
		}

		uint32 NumChildren = 0;
		PropertyHandle->GetNumChildren(NumChildren);

		for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
		{
			TSharedRef<IPropertyHandle> ChildHandle = PropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
			HideInvalidCVDDataWrapperProperties({ &ChildHandle,1 }, DetailBuilder);
		}	
	}
}

bool FChaosVDDetailsCustomizationUtils::HasValidCVDWrapperData(const TSharedPtr<IPropertyHandle>& InPropertyHandle, bool& bOutIsCVDBaseDataStruct)
{
	if (FProperty* Property = InPropertyHandle ? InPropertyHandle->GetProperty() : nullptr)
	{
		const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
		if (StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(FChaosVDWrapperDataBase::StaticStruct()))
		{
			bOutIsCVDBaseDataStruct = true;

			void* Data = nullptr;
			InPropertyHandle->GetValueData(Data);
			if (Data)
			{
				const FChaosVDWrapperDataBase* DataViewer = static_cast<const FChaosVDWrapperDataBase*>(Data);

				// The Particle Data viewer struct has several fields that will have default values if there was no recorded data for them in the trace file
				// As these do not represent any real value, we should hide them in the details panel
				return DataViewer->HasValidData();
			}
		}
	}

	return true;
}

TSharedPtr<FChaosVDCollisionChannelsInfoContainer> FChaosVDDetailsCustomizationUtils::BuildDefaultCollisionChannelInfo()
{
	TSharedPtr<FChaosVDCollisionChannelsInfoContainer> NewCollisionChannelsInfoContainer = MakeShared<FChaosVDCollisionChannelsInfoContainer>();

	// Build the default channels name & type using the enum metadata
	UEnum * Enum = StaticEnum<ECollisionChannel>();
	if (!ensure(Enum))
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed to generate fallback collision channels entry ", ANSI_TO_TCHAR(__FUNCTION__));
		return NewCollisionChannelsInfoContainer;
	}

	const int32 NumEnum = Enum->NumEnums();
	constexpr int32 ExpectedChannels = FChaosVDDetailsCustomizationUtils::GetMaxCollisionChannelIndex();
	NewCollisionChannelsInfoContainer->CollisionChannelInfos.SetNum(ExpectedChannels);

	if (NumEnum < ExpectedChannels)
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed to generate fallback collision channels entry ", ANSI_TO_TCHAR(__FUNCTION__));
		return NewCollisionChannelsInfoContainer;
	}
	
	const FString KeyName = TEXT("DisplayName");
	const FString TraceType = TEXT("TraceQuery");

	for (int32 ChannelIndex = 0; ChannelIndex < ExpectedChannels; ++ChannelIndex)
	{
		FChaosVDCollisionChannelInfo Info;
		Info.DisplayName = Enum->GetDisplayNameTextByIndex(ChannelIndex).ToString();
		Info.CollisionChannel = static_cast<ECollisionChannel>(ChannelIndex);
		Info.bIsTraceType = Enum->GetMetaData(*TraceType, ChannelIndex) == TEXT("1");

		NewCollisionChannelsInfoContainer->CollisionChannelInfos[ChannelIndex] = Info;
	}

	return NewCollisionChannelsInfoContainer;
}

void FChaosVDDetailsCustomizationUtils::CreateCollisionChannelsMatrixRow(int32 ChannelIndex, const FChaosVDCollisionChannelStateGetter& InChannelStateGetter, const FText& InChannelName, IDetailGroup& CollisionGroup, const float RowWidthCustomization)
{
	// Currently all details panel in CVD are read-only
	constexpr bool bEnabledState = false;

	CollisionGroup.AddWidgetRow()
	.NameContent()
	[
		SNew(STextBlock)
		.IsEnabled(bEnabledState)
		.Text(InChannelName)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SHorizontalBox)
		.IsEnabled(bEnabledState)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			CreateCollisionResponseMatrixCheckbox(InChannelStateGetter, ChannelIndex, ECollisionResponse::ECR_Ignore, RowWidthCustomization)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			CreateCollisionResponseMatrixCheckbox(InChannelStateGetter, ChannelIndex, ECollisionResponse::ECR_Overlap, RowWidthCustomization)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			CreateCollisionResponseMatrixCheckbox(InChannelStateGetter, ChannelIndex, ECollisionResponse::ECR_Block, RowWidthCustomization)
		]
	];
}

void FChaosVDDetailsCustomizationUtils::BuildCollisionChannelMatrix(const FChaosVDCollisionChannelStateGetter& InCollisionChannelStateGetter, TConstArrayView<FChaosVDCollisionChannelInfo> CollisionChannelsInfo, IDetailGroup& ParentCategoryGroup)
{
	constexpr float RowWidthCustomization = 50;

	ParentCategoryGroup.AddWidgetRow()
	.ValueContent()
	.MaxDesiredWidth(0)
	.MinDesiredWidth(0)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.WidthOverride(RowWidthCustomization)
			.HAlign(HAlign_Left)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("IgnoreCollisionLabel", "Ignore"))
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.HAlign(HAlign_Left)
			.WidthOverride(RowWidthCustomization)
			.Content()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("OverlapCollisionLabel", "Overlap"))
				.Font(IDetailLayoutBuilder::GetDetailFontBold())
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("BlockCollisionLabel", "Block"))
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		]
	];

	constexpr bool bStartExpanded = true;
	IDetailGroup& TraceResponsesGroup = ParentCategoryGroup.AddGroup(TEXT("CollisionTraceResponses"), LOCTEXT("CollisionTraceResponsesLabel", "Trace Responses"), bStartExpanded);
	TraceResponsesGroup.EnableReset(false);

	for (int32 ChannelIndex = 0; ChannelIndex < CollisionChannelsInfo.Num(); ++ChannelIndex)
	{
		const FChaosVDCollisionChannelInfo& ChannelInfo = CollisionChannelsInfo[ChannelIndex];
		if (ChannelInfo.bIsTraceType)
		{
			FChaosVDDetailsCustomizationUtils::CreateCollisionChannelsMatrixRow(ChannelIndex, InCollisionChannelStateGetter, FText::FromString(ChannelInfo.DisplayName), TraceResponsesGroup, RowWidthCustomization);
		}
	}

	IDetailGroup& CollisionResponsesGroup = ParentCategoryGroup.AddGroup(TEXT("CollisionObjectResponses"), LOCTEXT("CollisionObjectResponses", "Object Responses"), bStartExpanded);
	TraceResponsesGroup.EnableReset(false);

	for (int32 ChannelIndex = 0; ChannelIndex < CollisionChannelsInfo.Num(); ++ChannelIndex)
	{
		const FChaosVDCollisionChannelInfo& ChannelInfo = CollisionChannelsInfo[ChannelIndex];
		if (!ChannelInfo.bIsTraceType)
		{
			FChaosVDDetailsCustomizationUtils::CreateCollisionChannelsMatrixRow(ChannelIndex, InCollisionChannelStateGetter, FText::FromString(ChannelInfo.DisplayName), CollisionResponsesGroup, RowWidthCustomization);
		}
	}
}

TSharedRef<SWidget> FChaosVDDetailsCustomizationUtils::CreateCollisionResponseMatrixCheckbox(const FChaosVDCollisionChannelStateGetter& InStateGetter, int32 ChannelIndex, ECollisionResponse TargetResponse, float Width)
{
	return SNew(SBox)
		.WidthOverride(Width)
		.Content()
		[
			SNew(SCheckBox)
			.IsChecked_Lambda([GetterCopy = InStateGetter, ChannelIndex, TargetResponse]()
			{
				if (GetterCopy.IsBound())
				{
					return GetterCopy.Execute(ChannelIndex) == TargetResponse ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
				}
				return ECheckBoxState::Undetermined;
			})
		];
}

void FChaosVDDetailsCustomizationUtils::ExtractResponseContainer(const uint64 BlockMask, const uint64 OverlapMask, FCollisionResponseContainer& OutResponses)
{
	constexpr uint32 ChannelCount = UE_ARRAY_COUNT(FCollisionResponseContainer::EnumArray);
	static_assert(ChannelCount == 64);

	for (uint32 ChannelIndex = 0; ChannelIndex < ChannelCount; ++ChannelIndex)
	{
		const uint64 ChannelBit = ECC_TO_BITFIELD(ChannelIndex);
		uint8 Response = ECR_Ignore;
		if (BlockMask & ChannelBit)
		{
			Response = ECR_Block;
		}
		else if (OverlapMask & ChannelBit)
		{
			Response = ECR_Overlap;
		}
		OutResponses.EnumArray[ChannelIndex] = Response;
	}
}

void FChaosVDDetailsCustomizationUtils::AddWidgetRowForCheckboxValue(TAttribute<ECheckBoxState>&& State, const FText& InValueName, IDetailGroup& DetailGroup)
{
	DetailGroup.AddWidgetRow()
	.IsEnabled(false)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(InValueName)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SBox)
		.WidthOverride(50.0f)
		.Content()
		[
			SNew(SCheckBox)
			.IsChecked(State)
		]
	];
}

FText FChaosVDDetailsCustomizationUtils::GetDefaultCollisionChannelsUseWarningMessage()
{
	return LOCTEXT("EngineDefaultsWarningMessageBox", "The following names are the Engine's default channel names. \n Some might be incorrect or missing");
}

#undef LOCTEXT_NAMESPACE
