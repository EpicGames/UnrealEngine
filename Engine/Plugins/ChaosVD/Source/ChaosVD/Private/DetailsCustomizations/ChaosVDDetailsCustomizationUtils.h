// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "PropertyHandle.h"
#include "Chaos/CollisionFilterData.h"
#include "Containers/Set.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Engine/EngineTypes.h"

class SChaosVDMainTab;
struct FCollisionFilterData;
enum ECollisionResponse : int;
enum class ECheckBoxState : uint8;
struct FChaosVDRecording;
class FChaosVDScene;
class IDetailsView;
struct FChaosVDCollisionChannelInfo;
struct FChaosVDCollisionChannelsInfoContainer;
class IDetailGroup;
class IPropertyHandle;
class IDetailLayoutBuilder;
class FName;

DECLARE_DELEGATE_RetVal_OneParam(ECollisionResponse, FChaosVDCollisionChannelStateGetter, int32 /* ChannelIndex*/);

template<typename PropertyType>
struct FChaosVDDetailsPropertyDataHandle
{
	explicit FChaosVDDetailsPropertyDataHandle(const TSharedRef<IPropertyHandle>& InPropertyHandle);

	PropertyType* GetDataInstance();
	
private:
	TSharedPtr<IPropertyHandle> PropertyHandle;
	PropertyType* DataInstance = nullptr;
};

template <typename PropertyType>
FChaosVDDetailsPropertyDataHandle<PropertyType>::FChaosVDDetailsPropertyDataHandle(const TSharedRef<IPropertyHandle>& InPropertyHandle)
{
	PropertyHandle = InPropertyHandle;
	
	FProperty* Property = PropertyHandle->GetProperty();
	if (!Property)
	{
		return;
	}

	const FStructProperty* StructProperty = CastField<FStructProperty>(Property);
	if (StructProperty && StructProperty->Struct && StructProperty->Struct->IsChildOf(PropertyType::StaticStruct()))
	{
		void* Data = nullptr;
		PropertyHandle->GetValueData(Data);
		if (Data)
		{
			DataInstance = static_cast<PropertyType*>(Data);
		}
	}
}

template <typename PropertyType>
PropertyType* FChaosVDDetailsPropertyDataHandle<PropertyType>::GetDataInstance()
{
	if (!PropertyHandle)
	{
		return nullptr;
	}

	return DataInstance;
}

/** Helper Class for CVD Custom Details view */
class FChaosVDDetailsCustomizationUtils
{
public:
	/**
	 * Hides all categories of this view, except the ones provided in the Allowed Categories set
	 * @param DetailBuilder Layout builder of the class we are customizing
	 * @param AllowedCategories Set of category names we do not want to hide 
	 */
	static void HideAllCategories(IDetailLayoutBuilder& DetailBuilder, const TSet<FName>& AllowedCategories = TSet<FName>());

	/**
	 * Marks any property of the provided handles array as hidden if they are not valid CVD properties (meaning they don't have serialized data loaded from a CVD recording)
	 * @param InPropertyHandles Handles of properties to evaluate and hide if needed
	 */
	static void HideInvalidCVDDataWrapperProperties(TConstArrayView<TSharedPtr<IPropertyHandle>> InPropertyHandles);

	/**
	 * Marks any property of the provided handles array as hidden if they are not valid CVD properties (meaning they don't have serialized data loaded from a CVD recording), using the provided details builder
	 * @param InPropertyHandles Handles of properties to evaluate and hide if needed
	 * @param DetailBuilder Details builder that will hide the property
	 */
	static void HideInvalidCVDDataWrapperProperties(TConstArrayView<TSharedRef<IPropertyHandle>> InPropertyHandles, IDetailLayoutBuilder& DetailBuilder);

	/**
	 * Marks any property of the provided handles array as hidden if they are not valid CVD properties (meaning they don't have serialized data loaded from a CVD recording), using the provided details builder
	 * @param InPropertyHandle Property Handle to evaluate if it is valid
	 * @param bOutIsCVDBaseDataStruct Set to true if the property handle provided is from a CVD Wrapper Data Base. All other types will be deemed valid
	 */
	static bool HasValidCVDWrapperData(const TSharedPtr<IPropertyHandle>& InPropertyHandle, bool& bOutIsCVDBaseDataStruct);

	static void CreateCollisionChannelsMatrixRow(int32 ChannelIndex, const FChaosVDCollisionChannelStateGetter& InChannelStateGetter, const FText& InChannelName, IDetailGroup& CollisionGroup, const float RowWidthCustomization);
	
	static TSharedPtr<FChaosVDCollisionChannelsInfoContainer> BuildDefaultCollisionChannelInfo();
	
	static void BuildCollisionChannelMatrix(const FChaosVDCollisionChannelStateGetter& InCollisionChannelStateGetter, TConstArrayView<FChaosVDCollisionChannelInfo> CollisionChannelsInfo, IDetailGroup& ParentCategoryGroup);

	static TSharedRef<SWidget> CreateCollisionResponseMatrixCheckbox(const FChaosVDCollisionChannelStateGetter& InStateGetter, int32 ChannelIndex, ECollisionResponse TargetResponse, float Width);

	static constexpr int32 GetMaxCollisionChannelIndex();

	UE_INTERNAL static void ExtractResponseContainer(const uint64 BlockMask, const uint64 OverlapMask, FCollisionResponseContainer& OutResponses);

	static void AddWidgetRowForCheckboxValue(TAttribute<ECheckBoxState>&& State, const FText& InValueName, IDetailGroup& DetailGroup);

	static FText GetDefaultCollisionChannelsUseWarningMessage();
};

constexpr int32 FChaosVDDetailsCustomizationUtils::GetMaxCollisionChannelIndex()
{
	// Skip the deprecated channel which is last
	return ECollisionChannel::ECC_MAX - 1;
}
