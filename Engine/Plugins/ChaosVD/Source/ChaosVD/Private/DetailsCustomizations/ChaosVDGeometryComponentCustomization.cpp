// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/ChaosVDGeometryComponentCustomization.h"

#include "DetailLayoutBuilder.h"
#include "DetailsCustomizations/ChaosVDDetailsCustomizationUtils.h"

FChaosVDGeometryComponentCustomization::FChaosVDGeometryComponentCustomization()
{
	AllowedCategories.Add(FName("StaticMesh"));
	AllowedCategories.Add(FName("Materials"));
	AllowedCategories.Add(FName("Geometry Data"));
}

TSharedRef<IDetailCustomization> FChaosVDGeometryComponentCustomization::MakeInstance()
{
	return MakeShareable( new FChaosVDGeometryComponentCustomization );
}

void FChaosVDGeometryComponentCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	FChaosVDDetailsCustomizationUtils::HideAllCategories(DetailBuilder, AllowedCategories);
}
