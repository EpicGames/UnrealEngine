// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IDetailCustomization.h"

/**
 * Customization view for CVD Geometry component classes 
 */
class FChaosVDGeometryComponentCustomization : public IDetailCustomization
{
public:
	FChaosVDGeometryComponentCustomization();

	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

private:
	TSet<FName> AllowedCategories;
};
