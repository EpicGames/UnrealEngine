// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/ChaosVDParticleDataWrapperCustomization.h"

#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "DetailsCustomizations/ChaosVDDetailsCustomizationUtils.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"


#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

TSharedRef<IPropertyTypeCustomization> FChaosVDParticleDataWrapperCustomization::MakeInstance()
{
	return MakeShareable(new FChaosVDParticleDataWrapperCustomization());
}

void FChaosVDParticleDataWrapperCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FChaosVDParticleDataWrapperCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	if (NumChildren == 0)
	{
		return;
	}

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> Handle = StructPropertyHandle->GetChildHandle(ChildIndex);

		if (!Handle)
		{
			continue;
		}

		bool bIsParticleDataStruct = false;
		if (FChaosVDDetailsCustomizationUtils::HasValidCVDWrapperData(Handle, bIsParticleDataStruct))
		{
			StructBuilder.AddProperty(Handle.ToSharedRef());
		}
		else if (!bIsParticleDataStruct) // If it is not a valid particle data because it is not a data struct, just add the property anyways
		{
			StructBuilder.AddProperty(Handle.ToSharedRef());
		}
	}
}

#undef LOCTEXT_NAMESPACE
