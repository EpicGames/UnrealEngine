// Copyright Epic Games, Inc. All Rights Reserved.
#include "ChaosVDParticleMetadataCustomization.h"

#include "ChaosVDDetailsCustomizationUtils.h"
#include "ChaosVDModule.h"
#include "DetailWidgetRow.h"
#include "Editor.h"
#include "IDetailChildrenBuilder.h"
#include "PropertyHandle.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Misc/MessageDialog.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

TSharedRef<IPropertyTypeCustomization> FChaosVDParticleMetadataCustomization::MakeInstance()
{
	return MakeShareable(new FChaosVDParticleMetadataCustomization());
}

void FChaosVDParticleMetadataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow
	.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];
}

void FChaosVDParticleMetadataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{

	FChaosVDDetailsPropertyDataHandle<FChaosVDParticleMetadata> ParticleMetadataHandle(StructPropertyHandle);
	FChaosVDParticleMetadata* ParticleMetadataInstance = ParticleMetadataHandle.GetDataInstance();

	if (!ParticleMetadataInstance)
	{
		return;
	}
	
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	if (NumChildren == 0)
	{
		return;
	}

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedPtr<IPropertyHandle> Handle = StructPropertyHandle->GetChildHandle(ChildIndex);

		if (!Handle)
		{
			continue;
		}

		StructBuilder.AddProperty(Handle.ToSharedRef());
	}

	StructBuilder.AddCustomRow(FText::GetEmpty()).WholeRowContent()
	[
		GenerateBrowseParticleOwnerButton(ParticleMetadataInstance->OwnerAssetPath)
	];
}

TSharedRef<SWidget> FChaosVDParticleMetadataCustomization::GenerateBrowseParticleOwnerButton(FTopLevelAssetPath AssetPath)
{
	TSharedRef<SWidget> BrowseButton = SNew(SHorizontalBox)
	+SHorizontalBox::Slot()
	.VAlign(VAlign_Center)
	.HAlign(HAlign_Center)
	.Padding(12.0f, 7.0f, 12.0f, 7.0f)
	.FillWidth(1.0f)
	[
		SNew(SButton)
		.ToolTip(SNew(SToolTip).Text(LOCTEXT("BrowseParticleOwnerDesc", "Click here to open attempt to find the asset owning this particle in the content browser. This only works if CVD is open in the editor of the project where the recording comes from.")))
		.ContentPadding(FMargin(0, 5.f, 0, 4.f))
		.OnClicked_Raw(this, &FChaosVDParticleMetadataCustomization::BrowseParticleOwner, AssetPath)
		.Content()
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.Padding(FMargin(3, 0, 0, 0))
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.TextStyle(FAppStyle::Get(), "SmallButtonText")
				.Text(LOCTEXT("BrowseParticleOwnerText","Attempt To Find in Content Browser"))
			]
		]
	];

	return BrowseButton;
}

FReply FChaosVDParticleMetadataCustomization::BrowseParticleOwner(FTopLevelAssetPath AssetPath)
{
	if (GEditor)
	{
		if (UObject* OwnerObject = FindObject<UObject>(AssetPath))
		{
			GEditor->SyncBrowserToObject(OwnerObject);
		}
		else
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToFindOwnerMessage", " Failed to find the asset from which this particle was created"), LOCTEXT("FailedToFindOwnerTitle", "CVD Asset Finder"));
		}
	}
	
	return FReply::Handled();
}
#undef LOCTEXT_NAMESPACE
