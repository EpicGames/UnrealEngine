// Copyright Epic Games, Inc. All Rights Reserved.

#include "DetailsCustomizations/ChaosVDQueryDataWrappersCustomizationDetails.h"

#include "ChaosVDEngine.h"
#include "ChaosVDModule.h"
#include "ChaosVDRecording.h"
#include "ChaosVDScene.h"
#include "ChaosVDSceneParticle.h"
#include "CollisionQueryParams.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#include "DetailsCustomizations/ChaosVDDetailsCustomizationUtils.h"
#include "DetailLayoutBuilder.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "PropertyCustomizationHelpers.h"
#include "PropertyHandle.h"
#include "Widgets/SChaosVDEnumFlagsMenu.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Widgets/SChaosVDWarningMessageBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

namespace Chaos::VD::Utils::Private
{
	TSharedPtr<FChaosVDSceneParticle> GetParticleSceneFromCVDToolkitHost(const TWeakPtr<SChaosVDMainTab>& InCVDToolkitHost, int32 SolverID, int32 ParticleID)
	{
		TSharedPtr<SChaosVDMainTab> CVDToolkitHost = InCVDToolkitHost.Pin();
		TSharedPtr<FChaosVDScene> CVDScene = CVDToolkitHost ? CVDToolkitHost->GetScene() : nullptr;
		return CVDScene ? CVDScene->GetParticleInstance(SolverID, ParticleID) : nullptr;
	}

	FDetailWidgetRow& PopulateWidgetRowWithParticleName(FDetailWidgetRow& InOutWidgetRow, const FText& InParticleName)
	{
		InOutWidgetRow.NameContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[ 
				SNew(STextBlock)
				.Text(LOCTEXT("OwningParticleName", "Owning Particle"))
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		]
		.ValueContent()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Right)
			[ 
				SNew(STextBlock)
				.Text(InParticleName)
				.ToolTipText(InParticleName)
				.Font(IDetailLayoutBuilder::GetDetailFont())
			]
		];

		return InOutWidgetRow;
	}
}

TSharedRef<IPropertyTypeCustomization> FChaosVDQueryDataWrappersCustomizationDetails::MakeInstance()
{
	return MakeShareable(new FChaosVDQueryDataWrappersCustomizationDetails());
}

void FChaosVDQueryDataWrappersCustomizationDetails::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	uint32 NumChildren;
	StructPropertyHandle->GetNumChildren(NumChildren);

	if (NumChildren == 0)
	{
		return;
	}

	TArray<TSharedPtr<IPropertyHandle>> Handles;
	Handles.Reserve(NumChildren);

	for (uint32 ChildIndex = 0; ChildIndex < NumChildren; ++ChildIndex)
	{
		TSharedRef<IPropertyHandle> Handle = StructPropertyHandle->GetChildHandle(ChildIndex).ToSharedRef();
		Handles.Add(Handle);
	}

	FChaosVDDetailsCustomizationUtils::HideInvalidCVDDataWrapperProperties(Handles);
}

FChaosVDQueryVisitDataCustomization::FChaosVDQueryVisitDataCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab) : MainTabWeakPtr(InMainTab)
{
}

TSharedRef<IDetailCustomization> FChaosVDQueryVisitDataCustomization::MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab)
{
	return MakeShareable( new FChaosVDQueryVisitDataCustomization(MainTab) );
}


void FChaosVDQueryVisitDataCustomization::ReplaceParticleIndexWithParticleName(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TSharedPtr<FStructOnScope>> StructsCustomized;	
	DetailBuilder.GetStructsBeingCustomized(StructsCustomized);

	if (StructsCustomized.IsEmpty())
	{
		return;
	}

	if (!ensure(StructsCustomized.Num() == 1))
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%s] Multiple struct selection is not supported yet. The first one will be customized", __func__);
		return;
	}
		
	const TSharedPtr<FStructOnScope>& StructScope = StructsCustomized[0];

	if (!StructScope || StructScope->GetStruct() != FChaosVDQueryVisitStep::StaticStruct())
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%s] attempted to customize an invalid struct ", __func__);
	}
		
	const FChaosVDQueryVisitStep* SQVisitData = reinterpret_cast<const FChaosVDQueryVisitStep*>(StructScope->GetStructMemory());

	using namespace Chaos::VD::Utils::Private;
	TSharedPtr<FChaosVDSceneParticle> SceneParticle = GetParticleSceneFromCVDToolkitHost(MainTabWeakPtr, SQVisitData->SolverID_Editor, SQVisitData->ParticleIndex);
	if (!SceneParticle)
	{
		return;
	}

	TSharedRef<IPropertyHandle> ParticleIndexPropertyHandle = DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FChaosVDQueryVisitStep, ParticleIndex));
		
	if (IDetailPropertyRow* ParticleIndexPropertyRow = DetailBuilder.EditDefaultProperty(ParticleIndexPropertyHandle))
	{
		const FText ParticleNameAsText = FText::FromString(SceneParticle->GetDisplayName());
		PopulateWidgetRowWithParticleName(ParticleIndexPropertyRow->CustomWidget(), ParticleNameAsText);
	}
}

void FChaosVDQueryVisitDataCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{

	ReplaceParticleIndexWithParticleName(DetailBuilder);

	TArray<TSharedRef<IPropertyHandle>> PotentialPropertiesToHide;
	
	constexpr int32 PropertiesToEvaluateNum = 2;
	PotentialPropertiesToHide.Reset(PropertiesToEvaluateNum);

	PotentialPropertiesToHide.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FChaosVDQueryVisitStep, QueryFastData)));
	PotentialPropertiesToHide.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FChaosVDQueryVisitStep, HitData)));

	FChaosVDDetailsCustomizationUtils::HideInvalidCVDDataWrapperProperties(PotentialPropertiesToHide, DetailBuilder);
}

TSharedRef<IPropertyTypeCustomization> FChaosVDQueryVisitDataPropertyCustomization::MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab)
{
	return MakeShareable( new FChaosVDQueryVisitDataPropertyCustomization(MainTab) );
}

void FChaosVDQueryVisitDataPropertyCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow,
	IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	HeaderRow.NameContent()
	[
		StructPropertyHandle->CreatePropertyNameWidget()
	];

	HeaderRow.ValueContent()
	[
		StructPropertyHandle->CreatePropertyValueWidget()
	];
}

void FChaosVDQueryVisitDataPropertyCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,
                                                                    IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FChaosVDDetailsPropertyDataHandle<FChaosVDQueryVisitStep> StructPropertyDataHandle(StructPropertyHandle);

	FChaosVDQueryVisitStep* SQVisitDataBeingCustomized = StructPropertyDataHandle.GetDataInstance();
	if (!SQVisitDataBeingCustomized)
	{
		return;
	}

	using namespace Chaos::VD::Utils::Private;
	TSharedPtr<FChaosVDSceneParticle> SceneParticle = GetParticleSceneFromCVDToolkitHost(MainTabWeakPtr, SQVisitDataBeingCustomized->SolverID_Editor, SQVisitDataBeingCustomized->ParticleIndex);
	if (!SceneParticle)
	{
		return;
	}

	uint32 NumChild = 0;
	StructPropertyHandle->GetNumChildren(NumChild);

	for (uint32 PropIndex = 0; PropIndex < NumChild; PropIndex++)
	{
		TSharedPtr<IPropertyHandle> InnerPropHandle = StructPropertyHandle->GetChildHandle(PropIndex);

		if (!InnerPropHandle)
		{
			continue;
		}

		if (InnerPropHandle->GetProperty()->GetFName() == GET_MEMBER_NAME_CHECKED(FChaosVDQueryVisitStep, ParticleIndex))
		{
			const FText ParticleNameAsText = FText::FromString(SceneParticle->GetDisplayName());
			const FText SearchText = FText::AsCultureInvariant(TEXT("Owning Particle"));
			PopulateWidgetRowWithParticleName(StructBuilder.AddCustomRow(SearchText), ParticleNameAsText);
		}
		else
		{
			StructBuilder.AddProperty(InnerPropHandle.ToSharedRef());
		}
	}
}

TSharedRef<IDetailCustomization> FChaosVDQueryDataWrapperCustomization::MakeInstance()
{
	return MakeShareable( new FChaosVDQueryDataWrapperCustomization );
}

void FChaosVDQueryDataWrapperCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TSharedRef<IPropertyHandle>> PotentialPropertiesToHide;

	constexpr int32 PropertiesToEvaluateNum = 3;
	PotentialPropertiesToHide.Reset(PropertiesToEvaluateNum);

	PotentialPropertiesToHide.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FChaosVDQueryDataWrapper, CollisionQueryParams)));
	PotentialPropertiesToHide.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FChaosVDQueryDataWrapper, CollisionResponseParams)));
	PotentialPropertiesToHide.Add(DetailBuilder.GetProperty(GET_MEMBER_NAME_CHECKED(FChaosVDQueryDataWrapper, CollisionObjectQueryParams)));

	FChaosVDDetailsCustomizationUtils::HideInvalidCVDDataWrapperProperties(PotentialPropertiesToHide, DetailBuilder);
}

FChaosVDCollisionChannelsCustomizationBase::FChaosVDCollisionChannelsCustomizationBase(const TWeakPtr<SChaosVDMainTab>& InMainTab)
{
	MainTabWeakPtr = InMainTab;
	// Fill with the Engine defaults. When a CVD file is loaded we will update it with any new data is available
	UpdateCollisionChannelsInfoCache({});
}

void FChaosVDCollisionChannelsCustomizationBase::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle,IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	TSharedPtr<SChaosVDMainTab> MainTabPtr =  MainTabWeakPtr.Pin(); 
	TSharedPtr<FChaosVDScene> Scene = MainTabPtr ? MainTabPtr->GetChaosVDEngineInstance()->GetCurrentScene() : nullptr;

	if (TSharedPtr<FChaosVDRecording> CVDRecording = Scene ? Scene->GetLoadedRecording() : nullptr)
	{
		UpdateCollisionChannelsInfoCache(CVDRecording->GetCollisionChannelsInfoContainer());
	}
	else
	{
		UpdateCollisionChannelsInfoCache(nullptr);
	}
}

void FChaosVDCollisionChannelsCustomizationBase::UpdateCollisionChannelsInfoCache(TSharedPtr<FChaosVDCollisionChannelsInfoContainer> NewCollisionChannelsInfo)
{
	if (NewCollisionChannelsInfo)
	{
		CachedCollisionChannelInfos = NewCollisionChannelsInfo;
		bChannelInfoBuiltFromDefaults = false;
	}
	else
	{
		// Fallback to engine channels name using the enum metadata
		CachedCollisionChannelInfos = FChaosVDDetailsCustomizationUtils::BuildDefaultCollisionChannelInfo();
		bChannelInfoBuiltFromDefaults = true;
	}
}

ECollisionResponse FChaosVDCollisionResponseParamsCustomization::GetCurrentCollisionResponseForChannel(int32 ChannelIndex) const
{
	constexpr int32 MaxChannels = FChaosVDDetailsCustomizationUtils::GetMaxCollisionChannelIndex();

	if (ChannelIndex >= MaxChannels)
	{
		return ECollisionResponse::ECR_MAX;
	}

	if (FChaosVDCollisionResponseParams* Data = CurrentPropertyDataHandle ? CurrentPropertyDataHandle->GetDataInstance() : nullptr)
	{
		if (ensure(ChannelIndex < Data->FlagsPerChannel.Num()))
		{
			return static_cast<ECollisionResponse>(Data->FlagsPerChannel[ChannelIndex]);
		}
	}

	return ECollisionResponse::ECR_MAX;
}

TSharedRef<IPropertyTypeCustomization> FChaosVDCollisionResponseParamsCustomization::MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab)
{
	return MakeShared<FChaosVDCollisionResponseParamsCustomization>(MainTab);
}

void FChaosVDCollisionResponseParamsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FChaosVDCollisionChannelsCustomizationBase::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);

	if (!CachedCollisionChannelInfos)
	{
		return;
	}
	
	CurrentPropertyDataHandle = MakeShared<FChaosVDDetailsPropertyDataHandle<FChaosVDCollisionResponseParams>>(StructPropertyHandle);
	FChaosVDCollisionResponseParams* CollisionResponseParams = CurrentPropertyDataHandle->GetDataInstance();

	if (!CollisionResponseParams)
	{
		CurrentPropertyDataHandle = nullptr;
		return;
	}

	IDetailGroup& CollisionGroup = StructBuilder.AddGroup(TEXT("CollisionResponseParams"), LOCTEXT("CollisionResponseQueryParamsLabel", "Collision Response Query Params"));
	CollisionGroup.EnableReset(false);

	CollisionGroup.HeaderRow()
	.NameContent()
	[
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Right)
		[ 
			SNew(STextBlock)
			.Text(LOCTEXT("CollisionResponsesLabel", "Collision Response Query params"))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
	];

	if (bChannelInfoBuiltFromDefaults)
	{
		CollisionGroup.AddWidgetRow()
		.WholeRowContent()
		[
			SNew(SChaosVDWarningMessageBox).WarningText(FChaosVDDetailsCustomizationUtils::GetDefaultCollisionChannelsUseWarningMessage())
		];
	}
	
	FChaosVDCollisionChannelStateGetter CollisionChannelStateGetter;
	CollisionChannelStateGetter.BindSP(this, &FChaosVDCollisionResponseParamsCustomization::GetCurrentCollisionResponseForChannel);
	
	FChaosVDDetailsCustomizationUtils::BuildCollisionChannelMatrix(CollisionChannelStateGetter,
																	CachedCollisionChannelInfos->CollisionChannelInfos,
																	CollisionGroup);
}

TSharedRef<IPropertyTypeCustomization> FChaosVDCollisionObjectParamsCustomization::MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab)
{
	return MakeShared<FChaosVDCollisionObjectParamsCustomization>(MainTab);
}

void FChaosVDCollisionObjectParamsCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	FChaosVDCollisionChannelsCustomizationBase::CustomizeChildren(StructPropertyHandle, StructBuilder, StructCustomizationUtils);
	
	if (!CachedCollisionChannelInfos)
	{
		return;
	}
	
	CurrentPropertyDataHandle = MakeShared<FChaosVDDetailsPropertyDataHandle<FChaosVDCollisionObjectQueryParams>>(StructPropertyHandle);
	FChaosVDCollisionObjectQueryParams* CollisionResponseParams = CurrentPropertyDataHandle->GetDataInstance();

	if (!CollisionResponseParams)
	{
		CurrentPropertyDataHandle = nullptr;
		return;
	}

	IDetailGroup& CollisionGroup = StructBuilder.AddGroup(TEXT("CollisionObjectResponseParams"), LOCTEXT("CollisionObjectResponseQueryParamsLabel", "Collision Response Query Params") );
	CollisionGroup.EnableReset(false);

	for (int32 ChannelIndex = 0 ; ChannelIndex < CachedCollisionChannelInfos->CollisionChannelInfos.Num(); ChannelIndex++)
	{
		FChaosVDCollisionChannelInfo& ChannelInfo = CachedCollisionChannelInfos->CollisionChannelInfos[ChannelIndex];

		if (ChannelInfo.bIsTraceType)
		{
			continue;
		}

		// Currently, all details panel in CVD are read only
		constexpr bool bIsEditable = false;

		CollisionGroup.AddWidgetRow()
		.NameContent()
		[
			SNew(STextBlock)
			.Text(FText::FromString(ChannelInfo.DisplayName))
			.Font(IDetailLayoutBuilder::GetDetailFont())
		]
		.ValueContent()
		[
			SNew(SBox)
			.IsEnabled(bIsEditable)
			.WidthOverride(50.0f)
			.Content()
			[
				SNew(SCheckBox)
				.IsChecked_Raw(this, &FChaosVDCollisionObjectParamsCustomization::GetCurrentObjectFlag, ChannelIndex)
			]
		];
	}
}

ECheckBoxState FChaosVDCollisionObjectParamsCustomization::GetCurrentObjectFlag(int32 ChannelIndex) const
{
	FChaosVDCollisionObjectQueryParams* CollisionObjectResponseParams = CurrentPropertyDataHandle ? CurrentPropertyDataHandle->GetDataInstance() : nullptr;
	if (!CollisionObjectResponseParams)
	{
		return ECheckBoxState::Undetermined;
	}

	return (CollisionObjectResponseParams->ObjectTypesToQuery64 & ECC_TO_BITFIELD(ChannelIndex)) != 0 ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
}

#undef LOCTEXT_NAMESPACE
