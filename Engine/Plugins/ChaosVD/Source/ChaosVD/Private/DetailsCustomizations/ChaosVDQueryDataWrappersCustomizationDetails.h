// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDDetailsCustomizationUtils.h"
#include "IDetailCustomization.h"
#include "IDetailGroup.h"
#include "IPropertyTypeCustomization.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"

class SChaosVDMainTab;
enum ECollisionResponse : int;
struct FChaosVDCollisionChannelInfo;

/** Custom property layout for the ChaosVD SQ Data wrapper struct */
class FChaosVDQueryDataWrappersCustomizationDetails : public IPropertyTypeCustomization
{
	static TSharedRef<IPropertyTypeCustomization> MakeInstance();

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
};

/** Custom details panel for the ChaosVD SQ Visit Data struct */
class FChaosVDQueryVisitDataCustomization : public IDetailCustomization
{
public:
	FChaosVDQueryVisitDataCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab);

	virtual ~FChaosVDQueryVisitDataCustomization() override = default;

	static TSharedRef<IDetailCustomization> MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab);
	void ReplaceParticleIndexWithParticleName(IDetailLayoutBuilder& DetailBuilder);

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;

protected:
	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr = nullptr;
};

/** Custom property details for the ChaosVD SQ Visit Data struct */
class FChaosVDQueryVisitDataPropertyCustomization : public IPropertyTypeCustomization
{
public:
	FChaosVDQueryVisitDataPropertyCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab) : MainTabWeakPtr(InMainTab)
	{
	}

	virtual ~FChaosVDQueryVisitDataPropertyCustomization() override = default;

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab);

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr = nullptr;
};

/** Custom details panel for the ChaosVD SQ Data Wrapper struct */
class FChaosVDQueryDataWrapperCustomization : public IDetailCustomization
{
public:
	FChaosVDQueryDataWrapperCustomization(){};
	virtual ~FChaosVDQueryDataWrapperCustomization() override= default;

	static TSharedRef<IDetailCustomization> MakeInstance();

	virtual void CustomizeDetails(IDetailLayoutBuilder& DetailBuilder) override;
};

/** Custom details panel for the ChaosVD SQ Data Collision Response View */
class FChaosVDCollisionChannelsCustomizationBase : public IPropertyTypeCustomization
{
public:

	FChaosVDCollisionChannelsCustomizationBase(const TWeakPtr<SChaosVDMainTab>& InMainTab);

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

protected:
	void UpdateCollisionChannelsInfoCache(TSharedPtr<FChaosVDCollisionChannelsInfoContainer> NewCollisionChannelsInfo);
	
	TSharedPtr<FChaosVDCollisionChannelsInfoContainer> CachedCollisionChannelInfos;
	bool bChannelInfoBuiltFromDefaults = true;

	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr = nullptr;
};

/** Custom details panel for the ChaosVD SQ Data Collision Response View */
class FChaosVDCollisionResponseParamsCustomization : public FChaosVDCollisionChannelsCustomizationBase
{
public:
	FChaosVDCollisionResponseParamsCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab) : FChaosVDCollisionChannelsCustomizationBase(InMainTab)
	{
	}

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab);

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	
	ECollisionResponse GetCurrentCollisionResponseForChannel(int32 ChannelIndex) const;

	TSharedPtr<FChaosVDDetailsPropertyDataHandle<FChaosVDCollisionResponseParams>> CurrentPropertyDataHandle = nullptr;
};

/** Custom details panel for the ChaosVD SQ Data Collision Object Response View */
class FChaosVDCollisionObjectParamsCustomization : public FChaosVDCollisionChannelsCustomizationBase
{
public:
	FChaosVDCollisionObjectParamsCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab) : FChaosVDCollisionChannelsCustomizationBase(InMainTab)
	{
	}

	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab);

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override {}
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	
	ECheckBoxState GetCurrentObjectFlag(int32 ChannelIndex) const;

	TSharedPtr<FChaosVDDetailsPropertyDataHandle<FChaosVDCollisionObjectQueryParams>> CurrentPropertyDataHandle = nullptr;
};
