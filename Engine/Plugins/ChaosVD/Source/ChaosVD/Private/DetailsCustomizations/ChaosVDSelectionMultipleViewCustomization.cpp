// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSelectionMultipleViewCustomization.h"

#include "ChaosVDModule.h"
#include "ChaosVDSolverDataSelection.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "PropertyHandle.h"


TSharedRef<IDetailCustomization> FChaosVDSelectionMultipleViewCustomization::MakeInstance()
{
	return MakeShareable( new FChaosVDSelectionMultipleViewCustomization );
}

void FChaosVDSelectionMultipleViewCustomization::CustomizeDetails(IDetailLayoutBuilder& DetailBuilder)
{
	TArray<TSharedPtr<FStructOnScope>> StructsBeingCustomized;
	DetailBuilder.GetStructsBeingCustomized(StructsBeingCustomized);

	if (!ensure(StructsBeingCustomized.Num() == 1))
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] [%d] objects were selectioned but this customization panel only support single object selection.", ANSI_TO_TCHAR(__FUNCTION__), StructsBeingCustomized.Num())
	}

	if (StructsBeingCustomized.IsEmpty())
	{
		return;
	}
	TSharedPtr<FStructOnScope> SelectionViewStructOnScope = StructsBeingCustomized.IsEmpty() ? nullptr : StructsBeingCustomized[0];

	if (!SelectionViewStructOnScope)
	{
		return;
	}

	FChaosVDSelectionMultipleView* StructView = reinterpret_cast<FChaosVDSelectionMultipleView*>(SelectionViewStructOnScope->GetStructMemory());

	static const FName DefaultCategoryName("Recorded Data");

	for (const FChaosVDSelectionMultipleView::FDataEntry& Entry : StructView->DataInstances)
	{
		if (!Entry.StructOnScope || !Entry.StructOnScope->IsValid())
		{
			continue;
		}

		const FName CategoryName = Entry.GroupName.IsNone() ? DefaultCategoryName : Entry.GroupName;
		IDetailCategoryBuilder& CategoryBuilder = DetailBuilder.EditCategory(CategoryName);

		if (IDetailPropertyRow* CreatedRow = CategoryBuilder.AddExternalStructure(Entry.StructOnScope))
		{
			CreatedRow->DisplayName(Entry.StructOnScope->GetStructPtr()->GetDisplayNameText());
			CreatedRow->ShouldAutoExpand(true);
		}
	}
}
