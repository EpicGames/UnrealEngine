// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDShapeDataCustomization.h"

#include "ChaosVDDetailsCustomizationUtils.h"
#include "ChaosVDEngine.h"
#include "ChaosVDRecording.h"
#include "ChaosVDScene.h"
#include "DetailCategoryBuilder.h"
#include "DetailLayoutBuilder.h"
#include "DetailWidgetRow.h"
#include "IDetailChildrenBuilder.h"
#include "IDetailGroup.h"
#include "PropertyHandle.h"
#include "SEnumCombo.h"
#include "Engine/EngineTypes.h"
#include "Physics/PhysicsFiltering.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Widgets/SChaosVDWarningMessageBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SSpinBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

TSharedRef<IPropertyTypeCustomization> FChaosVDShapeDataCustomization::MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab)
{
	return MakeShared<FChaosVDShapeDataCustomization>(MainTab);
}

FChaosVDShapeDataCustomization::FChaosVDShapeDataCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab)
{
	MainTabWeakPtr = InMainTab;
}

FChaosVDShapeDataCustomization::~FChaosVDShapeDataCustomization()
{
	RegisterCVDScene(nullptr);
}

void FChaosVDShapeDataCustomization::CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	constexpr bool bPropagateToChildren = true;
	HeaderRow.OverrideResetToDefault(FResetToDefaultOverride::Hide(bPropagateToChildren));
}

void FChaosVDShapeDataCustomization::CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils)
{
	CurrentShapeDataHandle = nullptr;

	TSharedPtr<SChaosVDMainTab> MainTabPtr =  MainTabWeakPtr.Pin(); 
	TSharedPtr<FChaosVDScene> Scene = MainTabPtr ? MainTabPtr->GetChaosVDEngineInstance()->GetCurrentScene() : nullptr;

	RegisterCVDScene(Scene);

	if (!Scene)
	{
		return;
	}

	if (TSharedPtr<FChaosVDRecording> CVDRecording = Scene->GetLoadedRecording())
	{
		UpdateCollisionChannelsInfoCache(CVDRecording->GetCollisionChannelsInfoContainer());
	}
	else
	{
		UpdateCollisionChannelsInfoCache(nullptr);
	}

	if (!CachedCollisionChannelInfos)
	{
		return;
	}

	IDetailLayoutBuilder& ParentLayoutBuilder = StructBuilder.GetParentCategory().GetParentLayout();
	IDetailCategoryBuilder& CollisionCategoryBuilder = ParentLayoutBuilder.EditCategory(TEXT("ShapeCollisionSettings"), LOCTEXT("ShapeDataCollisionSettingsLabel", "Shape Collision Settings"));

	CurrentShapeDataHandle = MakeShared<FChaosVDDetailsPropertyDataHandle<FChaosVDShapeCollisionData>>(StructPropertyHandle);

	FChaosVDShapeCollisionData* ShapeData = CurrentShapeDataHandle->GetDataInstance();

	if (!ShapeData || !ShapeData->bIsValid)
	{
		StructBuilder
		.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		[
			SNew(SChaosVDWarningMessageBox).WarningText(LOCTEXT("InvalidShapeDataWarningMessageBox", "Warning : Failed to load shape data for the selected shape! "))
		];

		return;
	}

	CacheCollisionFilterData();
	
	TSharedPtr<IPropertyHandle> CollisionTraceTypeHandle = StructPropertyHandle->GetChildHandle(GET_MEMBER_NAME_CHECKED(FChaosVDShapeCollisionData, CollisionTraceType));
	CollisionCategoryBuilder.AddProperty(CollisionTraceTypeHandle.ToSharedRef());
	
	constexpr bool bIsEditable = false;

	const FText& CollisionEnabledRowLabel = LOCTEXT("CollisionEnabledLabel", "Collision Enabled");
	CollisionCategoryBuilder.AddCustomRow(CollisionEnabledRowLabel)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(CollisionEnabledRowLabel)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SBox)
		[ 
			SNew(STextBlock)
			.Text_Raw(this, &FChaosVDShapeDataCustomization::GetCurrentCollisionEnabledAsText) 
			.IsEnabled(bIsEditable)
			.Font(IDetailLayoutBuilder::GetDetailFontBold())
		]
	];

	const FText& ComplexCollisionRowLabel = LOCTEXT("ComplexCollisionLabel", "Is Complex");
	CollisionCategoryBuilder.AddCustomRow(ComplexCollisionRowLabel)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(ComplexCollisionRowLabel)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SBox)
		.IsEnabled(bIsEditable)
		.WidthOverride(50.0f)
		.VAlign(VAlign_Center)
		.Content()
		[
			SNew(SCheckBox)
			.IsChecked_Raw(this, &FChaosVDShapeDataCustomization::GetCurrentIsComplexCollision)
		]
	];

	if (!bFiltersRequireSplitView)
	{
		BuildCollisionFilerDataLayout(ECollisionFilterDataType::Query, LOCTEXT("FilterDataLabel", "Filter Data"), ParentLayoutBuilder, ECollisionFilterLayoutFlags::StartExpanded);
	}
	else
	{
		BuildCollisionFilerDataLayout(ECollisionFilterDataType::Query, LOCTEXT("QueryFilterDataRowLabel", "Query Filter Data"), ParentLayoutBuilder, ECollisionFilterLayoutFlags::StartExpanded);
		BuildCollisionFilerDataLayout(ECollisionFilterDataType::Sim, LOCTEXT("SimFilterDataRowLabel", "Sim Filter Data"), ParentLayoutBuilder, ECollisionFilterLayoutFlags::StartExpanded);
	}
}

void FChaosVDShapeDataCustomization::BuildCollisionFilerDataLayout(ECollisionFilterDataType FilterDataType, const FText& InDetailsGroupLabel, IDetailLayoutBuilder& ParentLayoutBuilder, ECollisionFilterLayoutFlags LayoutFlags)
{
	const bool bStartExpanded = EnumHasAnyFlags(LayoutFlags, ECollisionFilterLayoutFlags::StartExpanded);
	const bool bIsEditable = EnumHasAnyFlags(LayoutFlags, ECollisionFilterLayoutFlags::IsEditable);
	const bool bIsAdvanced = EnumHasAnyFlags(LayoutFlags, ECollisionFilterLayoutFlags::IsAdvanced);

	IDetailCategoryBuilder& FilterDataCategory = ParentLayoutBuilder.EditCategory(FName(InDetailsGroupLabel.ToString()), InDetailsGroupLabel);

	if (bChannelInfoBuiltFromDefaults)
	{
		FilterDataCategory.AddCustomRow(FText::GetEmpty())
		.WholeRowContent()
		[
			SNew(SChaosVDWarningMessageBox).WarningText(FChaosVDDetailsCustomizationUtils::GetDefaultCollisionChannelsUseWarningMessage())
		];
	}

	const FText CollisionChannelRowLabel = LOCTEXT("FilterDataCollisionChannelLabel", "Collision Channel");
	FilterDataCategory.AddCustomRow(CollisionChannelRowLabel)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(CollisionChannelRowLabel)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(STextBlock)
		.Text_Raw(this, &FChaosVDShapeDataCustomization::GetCurrentCollisionChannelAsText, FilterDataType)
		.IsEnabled(bIsEditable)
		.Font(IDetailLayoutBuilder::GetDetailFontBold())
	];

	const FText ExtraFilterRowLabel = LOCTEXT("FilterDataExtraFilterLabel", "Extra Filter");
	FilterDataCategory.AddCustomRow(ExtraFilterRowLabel)
	.NameContent()
	[
		SNew(STextBlock)
		.Text(ExtraFilterRowLabel)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	]
	.ValueContent()
	[
		SNew(SSpinBox<uint8>)
		.Value_Raw(this, &FChaosVDShapeDataCustomization::GetCurrentExtraFilter, FilterDataType)
		.IsEnabled(bIsEditable)
		.Font(IDetailLayoutBuilder::GetDetailFont())
	];
	
	AddExtraCollisionFilterFlags(FilterDataType, LOCTEXT("FilterDataExtraCollisionFilterTags", "Chaos Collision Filter Flags"), FilterDataCategory);

	IDetailGroup& CollisionResponseGroup = FilterDataCategory.AddGroup(TEXT("FilterDataCollisionResponseGroup"), LOCTEXT("FilterDataCollisionResponseGroupLabel", "Collision Response Flags"), bIsAdvanced, bStartExpanded);
	CollisionResponseGroup.EnableReset(false);

	FChaosVDCollisionChannelStateGetter CollisionSimChannelStateGetter;
	CollisionSimChannelStateGetter.BindSPLambda(this, [this, FilterDataType](int32 ChannelIndex)
	{
		return GetCurrentCollisionResponseForChannel(ChannelIndex, FilterDataType);
	});

	FChaosVDDetailsCustomizationUtils::BuildCollisionChannelMatrix(CollisionSimChannelStateGetter,
	                                                               CachedCollisionChannelInfos->CollisionChannelInfos,
	                                                               CollisionResponseGroup);
}

void FChaosVDShapeDataCustomization::UpdateCollisionChannelsInfoCache(const TSharedPtr<FChaosVDCollisionChannelsInfoContainer>& NewCollisionChannelsInfo)
{
	if (NewCollisionChannelsInfo)
	{
		CachedCollisionChannelInfos = NewCollisionChannelsInfo;
		bChannelInfoBuiltFromDefaults = false;
	}
	else
	{
		// Fallback to engine channels name using the enum metadata
		CachedCollisionChannelInfos = FChaosVDDetailsCustomizationUtils::BuildDefaultCollisionChannelInfo();
		bChannelInfoBuiltFromDefaults = true;
	}
}
FText FChaosVDShapeDataCustomization::GetCurrentCollisionEnabledAsText() const
{
	if (!CurrentShapeDataHandle)
	{
		return FText::GetEmpty();
	}

	FChaosVDShapeCollisionData* Data = CurrentShapeDataHandle->GetDataInstance();

	return Data ? UEnum::GetDisplayValueAsText(CollisionEnabledFromFlags(Data->bQueryCollision, Data->bSimCollision, Data->bIsProbe)) : FText::GetEmpty();
}

ECheckBoxState FChaosVDShapeDataCustomization::GetCurrentIsComplexCollision() const
{
	if (FChaosVDShapeCollisionData* Data = CurrentShapeDataHandle ? CurrentShapeDataHandle->GetDataInstance() : nullptr)
	{
		return Data->bIsComplex ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}

	return ECheckBoxState::Undetermined;
}

FText FChaosVDShapeDataCustomization::GetCurrentCollisionChannelAsText(ECollisionFilterDataType FilterDataType) const
{
	if (FChaosVDShapeCollisionData* Data = CurrentShapeDataHandle ? CurrentShapeDataHandle->GetDataInstance() : nullptr)
	{
		const FCachedFilterData& FilterData = CachedFilterData[FilterDataType];
		return UEnum::GetDisplayValueAsText((ECollisionChannel)FilterData.ChannelIndex);
	}

	return FText::GetEmpty();
}

uint8 FChaosVDShapeDataCustomization::GetCurrentExtraFilter(ECollisionFilterDataType FilterDataType) const
{
	if (FChaosVDShapeCollisionData* Data = CurrentShapeDataHandle ? CurrentShapeDataHandle->GetDataInstance() : nullptr)
	{
		const FCachedFilterData& FilterData = CachedFilterData[FilterDataType];
		return FilterData.MaskFilter;
	}

	return 0;
}

ECollisionResponse FChaosVDShapeDataCustomization::GetCurrentCollisionResponseForChannel(int32 ChannelIndex, ECollisionFilterDataType Type) const
{
	if (CurrentShapeDataHandle == nullptr || CurrentShapeDataHandle->GetDataInstance() == nullptr)
	{
		return ECollisionResponse::ECR_MAX;
	}

	const FCachedFilterData& FilterData = CachedFilterData[Type];
	if (ChannelIndex < 0 || ChannelIndex >= UE_ARRAY_COUNT(FilterData.Responses.EnumArray))
	{
		return ECollisionResponse::ECR_MAX;
	}
	return static_cast<ECollisionResponse>(FilterData.Responses.EnumArray[ChannelIndex]);
}

ECheckBoxState FChaosVDShapeDataCustomization::GetCurrentStateForFilteringFlag(ECollisionFilterDataType Type, Chaos::EFilterFlags FilteringFlag) const
{
	if (FChaosVDShapeCollisionData* Data = CurrentShapeDataHandle ? CurrentShapeDataHandle->GetDataInstance() : nullptr)
	{
		const FCachedFilterData& FilterData = CachedFilterData[Type];
		return EnumHasAnyFlags(FilterData.Flags, FilteringFlag) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	}
	
	return ECheckBoxState::Undetermined;
}
void FChaosVDShapeDataCustomization::AddExtraCollisionFilterFlags(ECollisionFilterDataType Type, const FText& IntSectionLabel, IDetailCategoryBuilder& CategoryBuilder)
{
	IDetailGroup& ExtraFilterFlagsGroup = CategoryBuilder.AddGroup(FName(IntSectionLabel.ToString()), IntSectionLabel);
	ExtraFilterFlagsGroup.EnableReset(false);

	AddCollisionFilterFlagRow(Type, Chaos::EFilterFlags::SimpleCollision, ExtraFilterFlagsGroup);
	AddCollisionFilterFlagRow(Type, Chaos::EFilterFlags::ComplexCollision, ExtraFilterFlagsGroup);
	AddCollisionFilterFlagRow(Type, Chaos::EFilterFlags::ContactNotify, ExtraFilterFlagsGroup);
	AddCollisionFilterFlagRow(Type, Chaos::EFilterFlags::CCD, ExtraFilterFlagsGroup);
	AddCollisionFilterFlagRow(Type, Chaos::EFilterFlags::ModifyContacts, ExtraFilterFlagsGroup);
	AddCollisionFilterFlagRow(Type, Chaos::EFilterFlags::StaticShape, ExtraFilterFlagsGroup);
	AddCollisionFilterFlagRow(Type, Chaos::EFilterFlags::KinematicKinematicPairs, ExtraFilterFlagsGroup);
}

void FChaosVDShapeDataCustomization::AddCollisionFilterFlagRow(ECollisionFilterDataType Type, Chaos::EFilterFlags Flag, IDetailGroup& DetailGroup) const
{
	TAttribute<ECheckBoxState> CheckBoxStateAttribute;
	CheckBoxStateAttribute.BindSPLambda(this, [this, Flag, Type]()
	{
		return GetCurrentStateForFilteringFlag(Type, Flag);
	});

	FChaosVDDetailsCustomizationUtils::AddWidgetRowForCheckboxValue(MoveTemp(CheckBoxStateAttribute), FText::FromString(Chaos::LexToString(Flag)), DetailGroup);
}

void FChaosVDShapeDataCustomization::CacheCollisionFilterData()
{
	const FChaosVDShapeCollisionData* Data = CurrentShapeDataHandle ? CurrentShapeDataHandle->GetDataInstance() : nullptr;
	if (Data == nullptr)
	{
		return;
	}

	const FChaosVDCollisionFilterDataV2& FilterData = Data->FilterData;
	constexpr uint32 ChannelCount = ECollisionChannel::ECC_OverlapAll_Deprecated;
	static_assert(ChannelCount == 64);
	static_assert(UE_ARRAY_COUNT(FCollisionResponseContainer::EnumArray) == ChannelCount);

	FCachedFilterData& QueryFilterData = CachedFilterData[ECollisionFilterDataType::Query];
	FCachedFilterData& SimFilterData = CachedFilterData[ECollisionFilterDataType::Sim];
	QueryFilterData = FCachedFilterData();
	SimFilterData = FCachedFilterData();
	bFiltersRequireSplitView = false;

	// Convert back to channel index for now
	if (FilterData.CollisionChannelMask != 0)
	{
		QueryFilterData.ChannelIndex = (uint8)FMath::CountTrailingZeros64(FilterData.CollisionChannelMask);
	}
	QueryFilterData.Flags = (Chaos::EFilterFlags)FilterData.GetFlags();
	QueryFilterData.MaskFilter = FilterData.GetMaskFilter();
	FChaosVDDetailsCustomizationUtils::ExtractResponseContainer(FilterData.BlockMask, FilterData.OverlapMask, QueryFilterData.Responses);

	if (FilterData.bLoadedFromLegacy)
	{
		uint32 SimFlags = 0;
		FilterData.ExtractWord3(FilterData.LegacySimWord3, SimFilterData.MaskFilter, SimFilterData.ChannelIndex, SimFlags);
		SimFilterData.Flags = (Chaos::EFilterFlags)SimFlags;
		FChaosVDDetailsCustomizationUtils::ExtractResponseContainer(FilterData.LegacySimBlockMask, 0, SimFilterData.Responses);
		bFiltersRequireSplitView = SimFilterData.Flags != QueryFilterData.Flags
			|| SimFilterData.ChannelIndex != QueryFilterData.ChannelIndex
			|| SimFilterData.MaskFilter != QueryFilterData.MaskFilter
			|| FilterData.BlockMask != FilterData.LegacySimBlockMask;
	}
	else
	{
		SimFilterData = QueryFilterData;
	}
}

void FChaosVDShapeDataCustomization::RegisterCVDScene(const TSharedPtr<FChaosVDScene>& InScene)
{
	TSharedPtr<FChaosVDScene> CurrentScene = SceneWeakPtr.Pin();
	if (InScene != CurrentScene)
	{
		if (CurrentScene)
		{
			CurrentScene->OnSceneUpdated().RemoveAll(this);
		}

		if (InScene)
		{
			InScene->OnSceneUpdated().AddSP(this, &FChaosVDShapeDataCustomization::HandleSceneUpdated);
		}

		SceneWeakPtr = InScene;
	}
}

void FChaosVDShapeDataCustomization::HandleSceneUpdated()
{
	// To avoid reconstructing the data channel flags every time a row in the collision channel response matrix is updated, we process it once and cache it
	// but we need to update it each time the CVD scene is updated as the source data of the currents elected particle might have changed.
	CacheCollisionFilterData();

	// Note: Intentionally not calling UpdateCollisionChannelsInfoCache, that is only updated once when a recording is loaded, therefore we don't nee to do it again
}

#undef LOCTEXT_NAMESPACE
