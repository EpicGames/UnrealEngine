// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDDetailsCustomizationUtils.h"
#include "IPropertyTypeCustomization.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "Engine/EngineTypes.h"

class IDetailCategoryBuilder;
class IDetailGroup;
class SChaosVDMainTab;

enum class ECheckBoxState : uint8;

struct FChaosVDCollisionChannelsInfoContainer;

/** Custom details panel for the ChaosVD SQ Data Collision Response View */
class FChaosVDShapeDataCustomization : public IPropertyTypeCustomization
{
public:
	static TSharedRef<IPropertyTypeCustomization> MakeInstance(TWeakPtr<SChaosVDMainTab> MainTab);
	
	FChaosVDShapeDataCustomization(const TWeakPtr<SChaosVDMainTab>& InMainTab);
	virtual ~FChaosVDShapeDataCustomization() override;

	virtual void CustomizeHeader(TSharedRef<IPropertyHandle> StructPropertyHandle, FDetailWidgetRow& HeaderRow, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;
	virtual void CustomizeChildren(TSharedRef<IPropertyHandle> StructPropertyHandle, IDetailChildrenBuilder& StructBuilder, IPropertyTypeCustomizationUtils& StructCustomizationUtils) override;

private:
	void UpdateCollisionChannelsInfoCache(const TSharedPtr<FChaosVDCollisionChannelsInfoContainer>& NewCollisionChannelsInfo);

	FText GetCurrentCollisionEnabledAsText() const;

	ECheckBoxState GetCurrentIsComplexCollision() const;

	enum ECollisionFilterDataType
	{
		Query,
		Sim,
		Count,
	};

	enum class ECollisionFilterLayoutFlags : uint8
	{
		None = 0,
		StartExpanded = 1 << 0,
		IsEditable = 1 << 1,
		IsAdvanced = 1 << 2
	};
	FRIEND_ENUM_CLASS_FLAGS(ECollisionFilterLayoutFlags)

	struct FCachedFilterData
	{
		uint8 ChannelIndex = 0;
		uint8 MaskFilter = 0;
		Chaos::EFilterFlags Flags = Chaos::EFilterFlags::None;
		FCollisionResponseContainer Responses;
	};

	void BuildCollisionFilerDataLayout(ECollisionFilterDataType FilterDataType, const FText& InDetailsGroupLabel, IDetailLayoutBuilder& ParentLayoutBuilder, ECollisionFilterLayoutFlags LayoutFlags);

	FText GetCurrentCollisionChannelAsText(ECollisionFilterDataType FilterDataType) const;
	uint8 GetCurrentExtraFilter(ECollisionFilterDataType FilterDataType) const;

	ECollisionResponse GetCurrentCollisionResponseForChannel(int32 ChannelIndex, ECollisionFilterDataType Type) const;

	void AddCollisionFilterFlagRow(ECollisionFilterDataType Type, Chaos::EFilterFlags Flag, IDetailGroup& DetailGroup) const;
	ECheckBoxState GetCurrentStateForFilteringFlag(ECollisionFilterDataType Type, Chaos::EFilterFlags FilteringFlag) const;

	void AddExtraCollisionFilterFlags(ECollisionFilterDataType Type, const FText& IntSectionLabel, IDetailCategoryBuilder& CategoryBuilder);

	void CacheCollisionFilterData();

	void RegisterCVDScene(const TSharedPtr<FChaosVDScene>& InScene);

	void HandleSceneUpdated();

	TSharedPtr<FChaosVDCollisionChannelsInfoContainer> CachedCollisionChannelInfos;
	bool bChannelInfoBuiltFromDefaults = true;
	
	TStaticArray<FCachedFilterData, ECollisionFilterDataType::Count> CachedFilterData;
	bool bFiltersRequireSplitView = false;

	TSharedPtr<FChaosVDDetailsPropertyDataHandle<FChaosVDShapeCollisionData>> CurrentShapeDataHandle = nullptr;

	TWeakPtr<FChaosVDScene> SceneWeakPtr = nullptr;

	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr = nullptr;
};
