// Copyright Epic Games, Inc. All Rights Reserved.

#include "ExtensionsSystem/ChaosVDExtensionsManager.h"

#include "ChaosVDModule.h"
#include "Features/IModularFeatures.h"
#include "Misc/LazySingleton.h"

FChaosVDExtensionsManager::FChaosVDExtensionsManager()
{
}

FChaosVDExtensionsManager::~FChaosVDExtensionsManager()
{
	IModularFeatures::Get().OnModularFeatureRegistered().RemoveAll(this);
	IModularFeatures::Get().OnModularFeatureUnregistered().RemoveAll(this);
	
	// We can't use EnumerateExtensions here as it iterates on AvailableExtensions
	// which is being modified by UnRegisterExtension, causing a crash
	TArray<TSharedRef<FChaosVDExtension>> AvailableExtensionsCopy;
	AvailableExtensions.GenerateValueArray(AvailableExtensionsCopy);
	for (TSharedRef<FChaosVDExtension> AvailableExtension : AvailableExtensionsCopy)
	{
		UnRegisterExtension(AvailableExtension);
	}
}

FChaosVDExtensionsManager& FChaosVDExtensionsManager::Get()
{
	return TLazySingleton<FChaosVDExtensionsManager>::Get();
}

void FChaosVDExtensionsManager::TearDown()
{
	TLazySingleton<FChaosVDExtensionsManager>::TearDown();
}

void FChaosVDExtensionsManager::RegisterExtension(const TSharedRef<FChaosVDExtension>& InExtension)
{
	FName ExtensionType = InExtension->GetExtensionType();
	if (ensure(!AvailableExtensions.Contains(ExtensionType)))
	{
		AvailableExtensions.Add(ExtensionType, InExtension);

		ExtensionRegisteredEvent.Broadcast(InExtension);

		UE_LOGF(LogChaosVDEditor, Log, "[%ls] Registering CVD Extension [%ls] ...", ANSI_TO_TCHAR(__FUNCTION__), *ExtensionType.ToString());
	}
	else
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] CVD Extension [%ls] already registered (or another extension is using the same id). Skipping ...", ANSI_TO_TCHAR(__FUNCTION__), *ExtensionType.ToString());
	}
}

void FChaosVDExtensionsManager::UnRegisterExtension(const TSharedRef<FChaosVDExtension>& InExtension)
{
	FName ExtensionType = InExtension->GetExtensionType();
	AvailableExtensions.Remove(ExtensionType);
	
	ExtensionRegisteredEvent.Broadcast(InExtension);

	UE_LOGF(LogChaosVDEditor, Log, "[%ls] UnRegistering CVD Extension [%ls] ...", ANSI_TO_TCHAR(__FUNCTION__), *ExtensionType.ToString());
}
