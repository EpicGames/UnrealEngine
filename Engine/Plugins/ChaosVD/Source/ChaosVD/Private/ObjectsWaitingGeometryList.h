// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "AsyncCompilationHelpers.h"
#include "Containers/Map.h"
#include "Containers/Array.h"
#include "Templates/Function.h"
#include "Templates/SharedPointer.h"

class FTextFormat;

/** Object that handles objects waiting for geometry to be ready to perform a desired operation
 * The operation has to run on the Game Thread, and there is a limit on how many operations will be processed per tick
 */
template<typename ObjectType>
class FObjectsWaitingGeometryList
{

public:
	 FObjectsWaitingGeometryList(const TFunction<bool(uint32, ObjectType)>& InObjectProcessorCallback, const FTextFormat& InProgressNotificationNameFormat, const TFunction<bool(uint32)>& InShouldProcessObjectsOverride) :
		  ObjectProcessorCallback(InObjectProcessorCallback)
		, ShouldProcessObjectsForKeyOverride(InShouldProcessObjectsOverride)
		, AsyncProgressNotification(InProgressNotificationNameFormat)
	{
	}

	~FObjectsWaitingGeometryList()
	 {
	 	// Clean up the notification progress bar
	 	WaitingObjectsByGeometryKey.Empty();
	 	AsyncProgressNotification.Update(0);
	 }

	/** Adds an object to process when the provided geometry key is ready */
	void AddObject(uint32 GeometryKey, ObjectType ObjectToProcess);

	/** Removed an object to process */
	void RemoveObject(uint32 GeometryKey, ObjectType& OutToProcess);

	/** Evaluates the current geometry available, and process any objects on the waiting list for it */
	bool ProcessWaitingObjects(float TimeBudgetSeconds);

private:

	bool ShouldProcessObjectsForKey(uint32 GeometryKey) const
	{
		if (ShouldProcessObjectsForKeyOverride)
		{
			return ShouldProcessObjectsForKeyOverride(GeometryKey);
		}
		return true;
	}

	TFunction<bool(uint32, ObjectType)> ObjectProcessorCallback;

	TFunction<bool(uint32)> ShouldProcessObjectsForKeyOverride;

	TMap<uint32, TArray<ObjectType>> WaitingObjectsByGeometryKey;

	FAsyncCompilationNotification AsyncProgressNotification;

	int32 QueuedObjectsToProcessNum = 0;
};

template <typename ObjectType>
void FObjectsWaitingGeometryList<ObjectType>::AddObject(uint32 GeometryKey, ObjectType ObjectToProcess)
{
	check(IsInGameThread());

	if (TArray<ObjectType>* ObjectsInQueueForKey = WaitingObjectsByGeometryKey.Find(GeometryKey))
	{
		ObjectsInQueueForKey->Add(ObjectToProcess);
	}
	else
	{
		WaitingObjectsByGeometryKey.Add(GeometryKey, {ObjectToProcess});
	}

	++QueuedObjectsToProcessNum;
}

template <typename ObjectType>
void FObjectsWaitingGeometryList<ObjectType>::RemoveObject(uint32 GeometryKey, ObjectType& OutToProcess)
{
	check(IsInGameThread());

	if (TArray<ObjectType>* QueueObjectsForKey = WaitingObjectsByGeometryKey.Find(GeometryKey))
	{
		TArray<ObjectType>& QueueObjectsForKeyRef = *QueueObjectsForKey;
		for(typename TArray<ObjectType>::TIterator ObjectToRemoveIterator = QueueObjectsForKeyRef.CreateIterator();  ObjectToRemoveIterator; ++ObjectToRemoveIterator)
		{
			ObjectType& ObjectToRemove = *ObjectToRemoveIterator;

			if (ObjectToRemove == OutToProcess)
			{
				ObjectToRemoveIterator.RemoveCurrent();

				--QueuedObjectsToProcessNum;
			}
		}		
	}
}

template <typename ObjectType>
bool FObjectsWaitingGeometryList<ObjectType>::ProcessWaitingObjects(float TimeBudgetSeconds)
{
	AsyncProgressNotification.Update(QueuedObjectsToProcessNum);

	const double StartTimeSeconds = FPlatformTime::Seconds();
	double CurrentTimeSpentSeconds = 0.0;
	int32 CurrentObjectsProcessedNum = 0;

	bool bCanContinueProcessing = true;
	for (typename TMap<uint32, TArray<ObjectType>>::TIterator QueuedObjectRemoveIterator = WaitingObjectsByGeometryKey.CreateIterator(); QueuedObjectRemoveIterator; ++QueuedObjectRemoveIterator)
	{
		const uint32 GeometryKey = QueuedObjectRemoveIterator.Key();

		if (ShouldProcessObjectsForKey(GeometryKey))
		{
			for (typename TArray<ObjectType>::TIterator ObjectRemoveIterator = QueuedObjectRemoveIterator.Value().CreateIterator(); ObjectRemoveIterator; ++ObjectRemoveIterator)
			{
				if (ObjectProcessorCallback && ObjectProcessorCallback(GeometryKey, *ObjectRemoveIterator))
				{
					CurrentObjectsProcessedNum++;
					--QueuedObjectsToProcessNum;
					bCanContinueProcessing = CurrentTimeSpentSeconds < TimeBudgetSeconds;
					
					AsyncProgressNotification.Update(QueuedObjectsToProcessNum);

					// Only check the budget every 5 tasks as Getting the current time is a syscall and it is not free
					if (CurrentObjectsProcessedNum % 5 == 0)
					{
						CurrentTimeSpentSeconds += FPlatformTime::Seconds() - StartTimeSeconds;
					}
	
					ObjectRemoveIterator.RemoveCurrent();
				}
			}
		}

		if (!bCanContinueProcessing)
		{
			break;
		}	

		if (QueuedObjectRemoveIterator.Value().IsEmpty())
		{
			QueuedObjectRemoveIterator.RemoveCurrent();
		}

		if (!bCanContinueProcessing)
		{
			break;
		}
	}

	return bCanContinueProcessing;
}

/** Object that allows adding objects that need processing to a queue, and provide a processor callback.
 * When there are pending objects in the queue, a progress notification will be shown in the editor
 */
template<typename ObjectType>
class FObjectsWaitingProcessingQueue
{

public:
	FObjectsWaitingProcessingQueue(const TFunction<bool(ObjectType)>& InObjectProcessorCallback, const FTextFormat& InProgressNotificationNameFormat) :
		 ObjectProcessorCallback(InObjectProcessorCallback)
	   , AsyncProgressNotification(InProgressNotificationNameFormat)
	{
	}

	~FObjectsWaitingProcessingQueue();

	void EnqueueObject(ObjectType ObjectToProcess);

	void ProcessWaitingTasks(float TimeBudgetSeconds);

private:

	TFunction<bool(ObjectType)> ObjectProcessorCallback;

	TQueue<ObjectType> WaitingObjectQueue;

	FAsyncCompilationNotification AsyncProgressNotification;

	int32 QueuedObjectsToProcessNum = 0;
};


template <typename ObjectType>
FObjectsWaitingProcessingQueue<ObjectType>::~FObjectsWaitingProcessingQueue()
{
	WaitingObjectQueue.Empty();
	AsyncProgressNotification.Update(0);
}

template <typename ObjectType>
void FObjectsWaitingProcessingQueue<ObjectType>::EnqueueObject(ObjectType ObjectToProcess)
{
	WaitingObjectQueue.Enqueue(ObjectToProcess);
	QueuedObjectsToProcessNum++;
}

template <typename ObjectType>
void FObjectsWaitingProcessingQueue<ObjectType>::ProcessWaitingTasks(float TimeBudgetSeconds)
{
	AsyncProgressNotification.Update(QueuedObjectsToProcessNum);

	const double StartTimeSeconds = FPlatformTime::Seconds();
	double CurrentTimeSpentSeconds = 0.0;
	int32 CurrentTasksProcessedNum = 0;

	ObjectType QueuedTask;
	while (TimeBudgetSeconds > CurrentTimeSpentSeconds && WaitingObjectQueue.Dequeue(QueuedTask))
	{
		// Only check the budget every 5 tasks as Getting the current time is a syscall and it is not free
		if (CurrentTasksProcessedNum % 5 == 0)
		{
			CurrentTimeSpentSeconds += FPlatformTime::Seconds() - StartTimeSeconds;
		}

		if (ObjectProcessorCallback(QueuedTask))
		{
			CurrentTasksProcessedNum++;
			--QueuedObjectsToProcessNum;
		}

		AsyncProgressNotification.Update(QueuedObjectsToProcessNum);
	}
}
