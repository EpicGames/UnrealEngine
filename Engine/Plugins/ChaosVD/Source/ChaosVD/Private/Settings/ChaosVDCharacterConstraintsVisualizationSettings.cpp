// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCharacterConstraintsVisualizationSettings.h"

#include "ChaosVDSettingsManager.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCharacterConstraintsVisualizationSettings)


void UChaosVDCharacterConstraintsVisualizationSettings::SetDataVisualizationFlags(EChaosVDCharacterGroundConstraintDataVisualizationFlags NewFlags)
{
	if (UChaosVDCharacterConstraintsVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCharacterConstraintsVisualizationSettings>())
	{
		Settings->GlobalCharacterGroundConstraintDataVisualizationFlags = static_cast<uint32>(NewFlags);
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDCharacterGroundConstraintDataVisualizationFlags UChaosVDCharacterConstraintsVisualizationSettings::GetDataVisualizationFlags()
{
	if (UChaosVDCharacterConstraintsVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCharacterConstraintsVisualizationSettings>())
	{
		return static_cast<EChaosVDCharacterGroundConstraintDataVisualizationFlags>(Settings->GlobalCharacterGroundConstraintDataVisualizationFlags);
	}

	return EChaosVDCharacterGroundConstraintDataVisualizationFlags::None;
}

bool UChaosVDCharacterConstraintsVisualizationSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, GlobalCharacterGroundConstraintDataVisualizationFlags, EChaosVDCharacterGroundConstraintDataVisualizationFlags::EnableDraw);
}
