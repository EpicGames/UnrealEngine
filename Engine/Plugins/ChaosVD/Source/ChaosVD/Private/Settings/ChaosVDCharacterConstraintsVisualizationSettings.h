// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "UObject/Object.h"
#include "ChaosVDCharacterConstraintsVisualizationSettings.generated.h"

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDCharacterGroundConstraintDataVisualizationFlags : uint32
{
	None					= 0 UMETA(Hidden),
	/** Draw the target movement vector */
	TargetDeltaPosition		= 1 << 0,
	/** Draw the target orientation facing vector */
	TargetDeltaFacing		= 1 << 1,
	/** Draw the ground query distance based on the constraint's data */
	GroundQueryDistance		= 1 << 2,
	/** Draw the ground query normal based on the constraint's data */
	GroundQueryNormal		= 1 << 3,
	/** Draw the applied force vector */
	AppliedRadialForce		= 1 << 4,
	/** Draw the applied force vector */
	AppliedNormalForce		= 1 << 5,
	/** Draw the applied force vector */
	AppliedTorque			= 1 << 6,
	/** Draw the constraint even if it is disabled */
	DrawDisabled			= 1 << 7,
	/** Only debugs draw data for a selected constraint */
	OnlyDrawSelected		= 1 << 8,
	/** Enables debug draw for constraint data from any solver */
	EnableDraw				= 1 << 9,
};
ENUM_CLASS_FLAGS(EChaosVDCharacterGroundConstraintDataVisualizationFlags);

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDCharacterConstraintsVisualizationSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()

public:

	/** If true, any debug draw text available will be drawn */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	bool bShowDebugText = false;
	
	/** Color used for torque vector */
	UPROPERTY(config, EditAnywhere, Category = DetailsPanel)
	bool bAutoSelectConstraintFromSelectedParticle = false;

	/** The depth priority used for while drawing data. Can be World or Foreground (with this one the shapes will be drawn on top of the geometry and be always visible) */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground;
	/** Scale to apply to the force vector before draw it. */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float ForceScale = 0.01f;
	/** Scale to apply to the torque vector before draw it. */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float TorqueScale = 0.01f;
	/** Scale to apply to anything that does not have a dedicated scale setting before draw it. */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float GeneralScale = 1.0f;
	/** Line thickness to use as a base to calculate the different line thickness values used to debug draw the data. */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float BaseLineThickness = 2.0f;
	/** Color used for normal force vector */
	UPROPERTY(config, EditAnywhere, Category = DebugDraw)
	FColor NormalForceColor = FColor(255, 0, 0);
	/** Color used for radial force vector */
	UPROPERTY(config, EditAnywhere, Category = DebugDraw)
	FColor RadialForceColor = FColor(255, 255, 0);
	/** Color used for torque vector */
	UPROPERTY(config, EditAnywhere, Category = DebugDraw)
	FColor TorqueColor = FColor(255, 0, 255);

	static void SetDataVisualizationFlags(EChaosVDCharacterGroundConstraintDataVisualizationFlags NewFlags);
	static EChaosVDCharacterGroundConstraintDataVisualizationFlags GetDataVisualizationFlags();

	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;

private:
	/** Set of flags to enable/disable visualization of specific character ground constraint data as debug draw */
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVD.EChaosVDCharacterGroundConstraintDataVisualizationFlags"))
	uint32 GlobalCharacterGroundConstraintDataVisualizationFlags = static_cast<uint32>(EChaosVDCharacterGroundConstraintDataVisualizationFlags::GroundQueryDistance | EChaosVDCharacterGroundConstraintDataVisualizationFlags::GroundQueryNormal | EChaosVDCharacterGroundConstraintDataVisualizationFlags::TargetDeltaPosition);
};
