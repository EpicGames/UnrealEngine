// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCollisionVisualizationSettings.h"

#include "ChaosVDSettingsManager.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCollisionVisualizationSettings)


void UChaosVDCollisionDataVisualizationSettings::SetDataVisualizationFlags(EChaosVDCollisionVisualizationFlags NewFlags)
{
	if (UChaosVDCollisionDataVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCollisionDataVisualizationSettings>())
	{
		Settings->CollisionDataVisualizationFlags = static_cast<uint32>(NewFlags);
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDCollisionVisualizationFlags UChaosVDCollisionDataVisualizationSettings::GetDataVisualizationFlags()
{
	if (UChaosVDCollisionDataVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCollisionDataVisualizationSettings>())
	{
		return static_cast<EChaosVDCollisionVisualizationFlags>(Settings->CollisionDataVisualizationFlags) ;
	}

	return EChaosVDCollisionVisualizationFlags::None;
}

bool UChaosVDCollisionDataVisualizationSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, CollisionDataVisualizationFlags, EChaosVDCollisionVisualizationFlags::EnableDraw);
}

