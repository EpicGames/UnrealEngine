// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"

#include "ChaosVDCollisionVisualizationSettings.generated.h"

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDCollisionVisualizationFlags: uint32
{
	None							= 0 UMETA(Hidden),
	ContactPoints					= 1 << 0,
	ContactInfo						= 1 << 1,
	NetPushOut						= 1 << 2,
	NetImpulse						= 1 << 3,
	ContactNormal					= 1 << 4,
	AccumulatedImpulse				= 1 << 5,
	DrawInactiveContacts			= 1 << 6,
	DrawDataOnlyForSelectedParticle	= 1 << 7,
	EnableDraw						= 1 << 8,
};
ENUM_CLASS_FLAGS(EChaosVDCollisionVisualizationFlags);

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDCollisionDataVisualizationSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()
public:

	/** If true, any available debug text will be drawn */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	bool bShowDebugText = false;

	/** The depth priority used for while drawing contact data. Can be World or Foreground (with this one the shapes will be drawn on top of the geometry and be always visible) */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_World;

	/** The radius of the debug draw circle used to represent a contact point */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float ContactCircleRadius = 6.0f;

	/** The scale value to be applied to the normal vector of a contact used to change its size to make it easier to see */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float ContactNormalScale = 30.0f;

	static void SetDataVisualizationFlags(EChaosVDCollisionVisualizationFlags NewFlags);
	static EChaosVDCollisionVisualizationFlags GetDataVisualizationFlags();

	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;

private:
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVD.EChaosVDCollisionVisualizationFlags"))
	/** Set of flags to enable/disable visualization of specific collision data as debug draw */
	uint32 CollisionDataVisualizationFlags = static_cast<uint32>(EChaosVDCollisionVisualizationFlags::ContactInfo | EChaosVDCollisionVisualizationFlags::ContactPoints | EChaosVDCollisionVisualizationFlags::ContactNormal);
};

