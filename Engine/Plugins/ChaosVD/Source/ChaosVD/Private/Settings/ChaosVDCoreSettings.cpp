// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/ChaosVDCoreSettings.h"

#include "Misc/ConfigContext.h"
#include "Widgets/SChaosVDPlaybackViewport.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCoreSettings)


UChaosVDSettingsObjectBase::UChaosVDSettingsObjectBase()
{
	
}

void UChaosVDSettingsObjectBase::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	UObject::PostEditChangeProperty(PropertyChangedEvent);

	BroadcastSettingsChanged();
}

void UChaosVDSettingsObjectBase::PostEditUndo()
{
	UObject::PostEditUndo();
	BroadcastSettingsChanged();
}

void UChaosVDSettingsObjectBase::OverridePerObjectConfigSection(FString& SectionName)
{
	if (OverrideConfigSectionName.IsEmpty())
	{
		OverrideConfigSectionName = GetClass()->GetPathName() + TEXT(" Instance");
	}

	SectionName = OverrideConfigSectionName;
}

void UChaosVDSettingsObjectBase::BroadcastSettingsChanged()
{
	SettingsChangedDelegate.Broadcast(this);

	constexpr bool bAllowCopyToDefaultObject = false;
	SaveConfig(CPF_Config,nullptr, GConfig, bAllowCopyToDefaultObject);
}

void UChaosVDVisualizationSettingsObjectBase::BroadcastSettingsChanged()
{
	Super::BroadcastSettingsChanged();

	// All geometry related operations are queued and de-duplicated at the end of the frame before being applied
	// Therefore we need to wait one frame to invalidate the viewport
	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([](float DeltaTime)
	{
		SChaosVDPlaybackViewport::ExecuteExternalViewportInvalidateRequest();
		return false;
	}));
}

bool UChaosVDVisualizationSettingsObjectBase::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return true;
}
