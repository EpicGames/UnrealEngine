// Copyright Epic Games, Inc. All Rights Reserved.

#include "Settings/ChaosVDGeneralSettings.h"

#include "ChaosVDModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDGeneralSettings)

UE::TraceBasedDebuggers::ETraceTransportMode UChaosVDGeneralSettings::GetTransportModeForTargetType(EBuildTargetType TargetType) const
{
	switch (TargetType)
	{
	case EBuildTargetType::Server:
		return GameServerDataTransportMode;
	case EBuildTargetType::Game:
	case EBuildTargetType::Client:
		return GameClientDataTransportMode;
	case EBuildTargetType::Editor:
	case EBuildTargetType::Unknown:
	case EBuildTargetType::Program:
	default:
		return UE::TraceBasedDebuggers::ETraceTransportMode::Direct;
	}
}

void UChaosVDGeneralSettings::SetTransportModeForTargetType(EBuildTargetType TargetType, UE::TraceBasedDebuggers::ETraceTransportMode NewTransportMode)
{
	switch (TargetType)
	{
	case EBuildTargetType::Server:
		{
			GameServerDataTransportMode = NewTransportMode;
			break;
		}
	case EBuildTargetType::Game:
	case EBuildTargetType::Client:
		{
			GameClientDataTransportMode = NewTransportMode;
			break;
		}
	case EBuildTargetType::Editor:
	case EBuildTargetType::Unknown:
	case EBuildTargetType::Program:
	default:
		{
			const FString ErrorMessage = FString::Printf(TEXT("[%hs Changing transport mode in the selected target type is not supported. Target type [%s]]"), __func__, LexToString(TargetType));

			ensureMsgf(false, TEXT("[%s]"), *ErrorMessage);
			UE_LOGF(LogChaosVDEditor, Error, "[%ls]",*ErrorMessage);
			return;
		}
	}
}

PRAGMA_DISABLE_DEPRECATION_WARNINGS
EChaosVDTransportMode UChaosVDGeneralSettings::GetTransportModeForTarget(EBuildTargetType TargetType) const
{
	return static_cast<EChaosVDTransportMode>(GetTransportModeForTargetType(TargetType));
}

void UChaosVDGeneralSettings::SetTransportModeForTarget(EBuildTargetType TargetType, EChaosVDTransportMode NewTransportMode)
{
	SetTransportModeForTargetType(TargetType, static_cast<UE::TraceBasedDebuggers::ETraceTransportMode>(NewTransportMode));
}
PRAGMA_ENABLE_DEPRECATION_WARNINGS
