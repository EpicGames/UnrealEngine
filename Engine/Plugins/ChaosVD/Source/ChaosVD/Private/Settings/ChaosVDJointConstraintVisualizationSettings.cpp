// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDJointConstraintVisualizationSettings.h"

#include "ChaosVDSettingsManager.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDJointConstraintVisualizationSettings)


void UChaosVDJointConstraintsVisualizationSettings::SetDataVisualizationFlags(EChaosVDJointsDataVisualizationFlags NewFlags)
{
	if (UChaosVDJointConstraintsVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDJointConstraintsVisualizationSettings>())
	{
		Settings->GlobalJointsDataVisualizationFlags = static_cast<uint32>(NewFlags);
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDJointsDataVisualizationFlags UChaosVDJointConstraintsVisualizationSettings::GetDataVisualizationFlags()
{
	if (UChaosVDJointConstraintsVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDJointConstraintsVisualizationSettings>())
	{
		return static_cast<EChaosVDJointsDataVisualizationFlags>(Settings->GlobalJointsDataVisualizationFlags);
	}

	return EChaosVDJointsDataVisualizationFlags::None;
}

bool UChaosVDJointConstraintsVisualizationSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, GlobalJointsDataVisualizationFlags, EChaosVDJointsDataVisualizationFlags::EnableDraw);
}
