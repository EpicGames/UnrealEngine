// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"

#include "ChaosVDJointConstraintVisualizationSettings.generated.h"

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDJointsDataVisualizationFlags : uint32
{
	None					= 0 UMETA(Hidden),
	/** Draw the PushOut vector based on the constraint's data */
	PushOut					= 1 << 0,
	/** Draw the Angular Impulse vector based on the constraint's data */
	AngularImpulse			= 1 << 1 UMETA(Hidden),
	ActorConnector			= 1 << 2,
	CenterOfMassConnector	= 1 << 3,
	Stretch					= 1 << 4,
	Axes					= 1 << 5,
	/** Draw the joint even if one of the particles or both are kinematic */
	DrawKinematic			= 1 << 6,
	/** Draw the joint even if it is disabled */
	DrawDisabled			= 1 << 7,
	/** Only debugs draw data for a selected joint constraint */
	OnlyDrawSelected		= 1 << 8,
	/** Enables Debug draw for Joint Constraint data from any solver that is visible */
	EnableDraw				= 1 << 9,
};
ENUM_CLASS_FLAGS(EChaosVDJointsDataVisualizationFlags);

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDJointConstraintsVisualizationSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()

public:

	/** If true, any debug draw text available will be drawn */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	bool bShowDebugText = false;

	/** The depth priority used for while drawing data. Can be World or Foreground (with this one the shapes will be drawn on top of the geometry and be always visible) */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground;

	/** Scale to apply to the Linear Impulse vector before draw it. */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float LinearImpulseScale = 0.001f;

	/** Scale to apply to the Angular Impulse vector before draw it. */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float AngularImpulseScale = 0.1f;

	/** Scale to apply to anything that does not have a dedicated scale setting before draw it. */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float GeneralScale = 1.0f;

	/** Line thickness to use as a base to calculate the different line thickness values used to debug draw the data. */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float BaseLineThickness = 2.0f;

	/** Size of the debug drawn Center Of Mass. */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float CenterOfMassSize = 1.0f;

	/** Size of the debug drawn if the Constraint Axis */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float ConstraintAxisLength = 10.0f;

	static void SetDataVisualizationFlags(EChaosVDJointsDataVisualizationFlags NewFlags);
	static EChaosVDJointsDataVisualizationFlags GetDataVisualizationFlags();

	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;

private:
	/** Set of flags to enable/disable visualization of specific joint constraints data as debug draw */
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVD.EChaosVDJointsDataVisualizationFlags"))
	uint32 GlobalJointsDataVisualizationFlags = static_cast<uint32>(EChaosVDJointsDataVisualizationFlags::ActorConnector | EChaosVDJointsDataVisualizationFlags::DrawKinematic);
};