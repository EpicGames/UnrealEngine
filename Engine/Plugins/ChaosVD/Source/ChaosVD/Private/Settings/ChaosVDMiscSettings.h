// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "ChaosVDMiscSettings.generated.h"

/** Structure with the details about a recently open CVD file */
USTRUCT()
struct FChaosVDRecentFile
{
	GENERATED_BODY()

	/** Path to the CVD File */
	UPROPERTY(config)
	FString FileName;

	/** Timestamp of the last time the CVD opened this file */
	UPROPERTY(config)
	FDateTime LastOpenTime;

	FChaosVDRecentFile()
	{}

	FChaosVDRecentFile(const FString& InProjectName, FDateTime InLastOpenTime)
		: FileName(InProjectName)
		, LastOpenTime(InLastOpenTime)
	{}

	struct FRecentFilesSortPredicate
	{
		bool operator()( const FChaosVDRecentFile& A, const FChaosVDRecentFile& B ) const
		{
			return B.LastOpenTime < A.LastOpenTime;
		}
	};

	bool operator<(const FChaosVDRecentFile& Other) const
	{
		return LastOpenTime < Other.LastOpenTime;
	}

	bool operator==(const FChaosVDRecentFile& Other) const
	{
		return FileName == Other.FileName;
	}

	bool operator==(const FString& OtherProjectName) const
	{
		return FileName == OtherProjectName;
	}
};

/**
 * General non-core CVD settings
 */
UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDMiscSettings : public UChaosVDSettingsObjectBase
{
	GENERATED_BODY()

public:
	/** List of recently opened files */
	UPROPERTY(config)
	TArray<FChaosVDRecentFile> RecentFiles;

	/** Maximum number of recent files we can keep track of*/
	UPROPERTY(config)
	int32 MaxRecentFilesNum = 10;

	/** Saved data channel enabled state */
	UPROPERTY(config)
	TMap<FString, bool> DataChannelEnabledState;
};
