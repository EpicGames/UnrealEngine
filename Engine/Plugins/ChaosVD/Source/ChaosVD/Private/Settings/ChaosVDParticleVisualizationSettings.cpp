// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDParticleVisualizationSettings.h"

#include "ChaosVDSettingsManager.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDParticleVisualizationSettings)


FColor FChaosDebugDrawColorsByState::GetColorFromState(EChaosVDObjectStateType State) const
{
	switch (State)
	{
		case EChaosVDObjectStateType::Sleeping:
			return SleepingColor;
		case EChaosVDObjectStateType::Kinematic:
			return KinematicColor;
		case EChaosVDObjectStateType::Static:
			return StaticColor;
		case EChaosVDObjectStateType::Dynamic:
			return DynamicColor;
		default:
			return FColor::Purple;
	}
}

FColor FChaosParticleDataDebugDrawColors::GetColorForDataID(EChaosVDParticleDataVisualizationFlags DataID, bool bIsSelected) const
{
	constexpr float DefaultIntensityFactor = 0.6f;
	constexpr float SelectedIntensityFactor = 1.0f;
	const float IntensityFactor = bIsSelected ? SelectedIntensityFactor : DefaultIntensityFactor;

	return (GetLinearColorForDataID(DataID) * IntensityFactor).ToFColorSRGB();
}

const FLinearColor& FChaosParticleDataDebugDrawColors::GetLinearColorForDataID(EChaosVDParticleDataVisualizationFlags DataID) const
{
	static FLinearColor InvalidColor = FColor::Purple;
	switch (DataID)
	{
	case EChaosVDParticleDataVisualizationFlags::Acceleration:
		return AccelerationColor;
	case EChaosVDParticleDataVisualizationFlags::Velocity:
		return VelocityColor;
	case EChaosVDParticleDataVisualizationFlags::AngularVelocity:
		return AngularVelocityColor;
	case EChaosVDParticleDataVisualizationFlags::AngularAcceleration:
		return AngularAccelerationColor;
	case EChaosVDParticleDataVisualizationFlags::LinearImpulse:
		return LinearImpulseColor;
	case EChaosVDParticleDataVisualizationFlags::AngularImpulse:
		return AngularImpulseColor;
	case EChaosVDParticleDataVisualizationFlags::ClusterConnectivityEdge:
		return ConnectivityDataColor;
	case EChaosVDParticleDataVisualizationFlags::CenterOfMass:
		return CenterOfMassColor;
	case EChaosVDParticleDataVisualizationFlags::None:
	case EChaosVDParticleDataVisualizationFlags::DrawDataOnlyForSelectedParticle:
	default:
		return InvalidColor;
	}
}

float UChaosVDParticleVisualizationDebugDrawSettings::GetScaleFortDataID(EChaosVDParticleDataVisualizationFlags DataID) const
{
	switch (DataID)
    {
    	case EChaosVDParticleDataVisualizationFlags::Acceleration:
    		return AccelerationScale;
    	case EChaosVDParticleDataVisualizationFlags::Velocity:
    		return VelocityScale;
    	case EChaosVDParticleDataVisualizationFlags::AngularVelocity:
    		return AngularVelocityScale;
    	case EChaosVDParticleDataVisualizationFlags::AngularAcceleration:
    		return AngularAccelerationScale;
    	case EChaosVDParticleDataVisualizationFlags::LinearImpulse:
    		return LinearImpulseScale;
    	case EChaosVDParticleDataVisualizationFlags::AngularImpulse:
    		return AngularImpulseScale;
    	case EChaosVDParticleDataVisualizationFlags::ClusterConnectivityEdge:
    	case EChaosVDParticleDataVisualizationFlags::CenterOfMass:
    	case EChaosVDParticleDataVisualizationFlags::None:
    	case EChaosVDParticleDataVisualizationFlags::DrawDataOnlyForSelectedParticle:
    	default:
    		return 1.0f;
    }
}

void UChaosVDParticleVisualizationDebugDrawSettings::SetDataDebugDrawVisualizationFlags(EChaosVDParticleDataVisualizationFlags Flags)
{
	if (UChaosVDParticleVisualizationDebugDrawSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationDebugDrawSettings>())
	{
		Settings->ParticleDataVisualizationFlags = static_cast<uint32>(Flags);
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDParticleDataVisualizationFlags UChaosVDParticleVisualizationDebugDrawSettings::GetDataDebugDrawVisualizationFlags()
{
	if (UChaosVDParticleVisualizationDebugDrawSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationDebugDrawSettings>())
	{
		return static_cast<EChaosVDParticleDataVisualizationFlags>(Settings->ParticleDataVisualizationFlags);
	}

	return EChaosVDParticleDataVisualizationFlags::None;
}

bool UChaosVDParticleVisualizationDebugDrawSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, ParticleDataVisualizationFlags, EChaosVDParticleDataVisualizationFlags::EnableDraw);
}

void UChaosVDParticleVisualizationSettings::SetGeometryVisualizationFlags(EChaosVDGeometryVisibilityFlags Flags)
{
	if (UChaosVDParticleVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationSettings>())
	{
		Settings->GeometryVisibilityFlags = static_cast<uint32>(Flags);
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDGeometryVisibilityFlags UChaosVDParticleVisualizationSettings::GetGeometryVisualizationFlags()
{
	if (UChaosVDParticleVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationSettings>())
	{
		return static_cast<EChaosVDGeometryVisibilityFlags>(Settings->GeometryVisibilityFlags);
	}

	return EChaosVDGeometryVisibilityFlags::None;
}

FColor FChaosDebugDrawColorsByShapeType::GetColorFromShapeType(Chaos::EImplicitObjectType ShapeType) const
{
	switch(ShapeType)
	{
		case Chaos::ImplicitObjectType::Sphere:
			return SimpleTypeColor;
		case Chaos::ImplicitObjectType::Box:
			return SimpleTypeColor;
		case Chaos::ImplicitObjectType::Plane:
			return SimpleTypeColor;
		case Chaos::ImplicitObjectType::Capsule:
			return SimpleTypeColor;
		case Chaos::ImplicitObjectType::TaperedCylinder:
			return SimpleTypeColor;
		case Chaos::ImplicitObjectType::Cylinder:
			return SimpleTypeColor;
		case Chaos::ImplicitObjectType::Convex:
			return ConvexColor;
		case Chaos::ImplicitObjectType::HeightField:
			return HeightFieldColor;
		case Chaos::ImplicitObjectType::TriangleMesh:
			return TriangleMeshColor;
		case Chaos::ImplicitObjectType::LevelSet:
			return LevelSetColor;			
		default:
			return FColor::Purple; 
	}
}

FColor FChaosDebugDrawColorsByClientServer::GetColorFromState(bool bIsServer, EChaosVDObjectStateType State) const
{
	if (State == EChaosVDObjectStateType::Uninitialized)
	{
		return FColor::Purple;
	}

	constexpr float IntensityFactor = 1.0f / static_cast<float>(EChaosVDObjectStateType::Count);

	// Make sure static is always darker than sleeping
	if (State == EChaosVDObjectStateType::Static)
	{

		constexpr float StaticStateIntensity = IntensityFactor * (static_cast<float>(EChaosVDObjectStateType::Sleeping) * 0.6f);
		return GetColorAtIntensity(bIsServer ? ServerBaseColor : ClientBaseColor, StaticStateIntensity);
	}

	const float Intensity = IntensityFactor * static_cast<float>(State);
	return GetColorAtIntensity(bIsServer ? ServerBaseColor : ClientBaseColor, Intensity);
}

FColor FChaosDebugDrawColorsByClientServer::GetColorAtIntensity(const FColor& InColor, float Intensity) const
{
	return (FLinearColor(InColor) * Intensity).ToFColor(true);
}
