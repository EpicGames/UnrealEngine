// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "Chaos/ImplicitObject.h"
#include "UObject/Object.h"
#include "ChaosVDParticleVisualizationSettings.generated.h"

enum class EChaosVDObjectStateType: int8;

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDGeometryVisibilityFlags : uint8
{
	None				= 0 UMETA(Hidden),
	/** Draws all geometry that is for Query Only */
	Query = 1 << 1,
	/** Draws all geometry that is for [Physics Collision] or [Physics Collision and Query only] */
	Simulated = 1 << 2,
	/** Draws all simple geometry */
	Simple = 1 << 3,
	/** Draws all complex geometry */
	Complex = 1 << 4,
	/** Draws heightfields even if complex is not selected */
	ShowHeightfields = 1 << 5,
	/** Draws all particles that are in a disabled state */
	ShowDisabledParticles = 1 << 6,
};
ENUM_CLASS_FLAGS(EChaosVDGeometryVisibilityFlags)

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDParticleDataVisualizationFlags : uint32
{
	None				= 0 UMETA(Hidden),
	Velocity			= 1 << 0,
	AngularVelocity		= 1 << 1,
	Acceleration		= 1 << 2,
	AngularAcceleration = 1 << 3,
	LinearImpulse		= 1 << 4,
	AngularImpulse		= 1 << 5,
	ClusterConnectivityEdge	= 1 << 6,
	CenterOfMass	= 1 << 7,
	DrawDataOnlyForSelectedParticle	= 1 << 8,
	
	Bounds = 1 << 10,
	InflatedBounds = 1 << 11,

	EnableDraw	= 1 << 9
};
ENUM_CLASS_FLAGS(EChaosVDParticleDataVisualizationFlags);


/** Structure holding the settings using to debug draw Particles shape based on their shape type on the Chaos Visual Debugger */
USTRUCT()
struct FChaosDebugDrawColorsByShapeType
{
	GENERATED_BODY()

	/** Color used for Sphere, Plane, Cube, Capsule, Cylinder, tapered shapes */
	UPROPERTY(EditAnywhere, Category=DebugDraw)
	FColor SimpleTypeColor = FColor(0, 158, 115); 

	/** Color used for convex shapes */
	UPROPERTY(EditAnywhere, Category=DebugDraw)
	FColor ConvexColor = FColor(240, 228, 66);

	/** Color used for heightfield */
	UPROPERTY(EditAnywhere, Category=DebugDraw)
	FColor HeightFieldColor = FColor(86, 180, 233);
	
	/** Color used for triangle meshes */
	UPROPERTY(EditAnywhere, Category=DebugDraw)
	FColor TriangleMeshColor = FColor(213, 94, 0);

	/** Color used for triangle LevelSets */
	UPROPERTY(EditAnywhere, Category=DebugDraw)
	FColor LevelSetColor = FColor(204, 121, 167);

	FColor GetColorFromShapeType(Chaos::EImplicitObjectType ShapeType) const;
};

/** Structure holding the settings using to debug draw Particles shape based on whether they are client or server objects (in PIE) Chaos Visual Debugger */
USTRUCT()
struct FChaosDebugDrawColorsByClientServer
{
	GENERATED_BODY()
	
	/** Color used for server shapes that are not awake or sleeping dynamic */
    UPROPERTY(config, EditAnywhere, Category=DebugDraw)
    FColor ServerBaseColor = FColor(231, 92, 80);

	/** Color used for server shapes that are not awake or sleeping dynamic */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FColor ClientBaseColor = FColor(0, 114, 178); 

	FColor GetColorFromState(bool bIsServer, EChaosVDObjectStateType State) const;
	FColor GetColorAtIntensity(const FColor& InColor, float Intensity) const;
};

UENUM()
enum class EChaosVDParticleDebugColorMode
{
	/** Draw particles with the default gray color */
	None,
	/** Draw particles with a specific color based on the recorded particle state */
	State,
	/** Draw particles with a specific color based on their shape type */
	ShapeType,
	/** Draw particles with a specific color based on if they are a Server Particle or Client particle */
	ClientServer,
};

/** Structure holding the settings using to debug draw Particles shape based on their state on the Chaos Visual Debugger */
USTRUCT()
struct FChaosDebugDrawColorsByState
{
	GENERATED_BODY()

	/** Color used for dynamic particles */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FColor DynamicColor = FColor(253, 246, 98);
	
	/** Color used for sleeping particles */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FColor SleepingColor = FColor(231, 92, 80);

	/** Color used for kinematic particles */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FColor KinematicColor = FColor(0, 114, 178);

	/** Color used for static particles */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FColor StaticColor = FColor(150, 159, 156);

	FColor GetColorFromState(EChaosVDObjectStateType State) const;
};

USTRUCT()
struct FChaosParticleDataDebugDrawColors
{
	GENERATED_BODY()

	/** Color to apply to the Velocity vector when drawing it */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FLinearColor VelocityColor = FColor::Green;

	/** Color to apply to the Angular Velocity vector when drawing it */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FLinearColor AngularVelocityColor = FColor::Blue;

	/** Color to apply to the Acceleration vector when drawing it */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FLinearColor AccelerationColor = FColor::Orange;

	/** Color to apply to the Angular Acceleration vector when drawing it */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FLinearColor AngularAccelerationColor = FColor::Silver;

	/** Color to apply to the Linear Impulse when drawing it */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FLinearColor LinearImpulseColor = FColor::Turquoise;

	/** Color to apply to the Angular Impulse vector when drawing it */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FLinearColor AngularImpulseColor = FColor::Emerald;

	/** Color to apply the debug drawn sphere representing the center of mass location */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FLinearColor CenterOfMassColor = FColor::Red;

	/** Color to apply to when drawing the connectivity data */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FLinearColor ConnectivityDataColor = FColor::Yellow;
	
	FColor GetColorForDataID(EChaosVDParticleDataVisualizationFlags DataID, bool bIsSelected = false) const;
	const FLinearColor& GetLinearColorForDataID(EChaosVDParticleDataVisualizationFlags DataID) const;
};

namespace Chaos::VisualDebugger::ParticleDataUnitsStrings
{
	static FString Velocity = TEXT("cm/s");
	static FString AngularVelocity = TEXT("rad/s");
	static FString Acceleration = TEXT("cm/s2");
	static FString AngularAcceleration = TEXT("rad/s2");
	static FString LinearImpulse = TEXT("g.m/s");
	static FString AngularImpulse = TEXT("g.m2/s");

	static FString GetUnitByID(EChaosVDParticleDataVisualizationFlags DataID)
	{
		switch (DataID)
		{
		case EChaosVDParticleDataVisualizationFlags::Velocity:
			return Velocity;
		case EChaosVDParticleDataVisualizationFlags::AngularVelocity:
			return AngularVelocity;
		case EChaosVDParticleDataVisualizationFlags::Acceleration:
			return Acceleration;
		case EChaosVDParticleDataVisualizationFlags::AngularAcceleration:
			return AngularAcceleration;
		case EChaosVDParticleDataVisualizationFlags::LinearImpulse:
			return LinearImpulse;
		case EChaosVDParticleDataVisualizationFlags::AngularImpulse:
			return AngularImpulse;

		case EChaosVDParticleDataVisualizationFlags::None:
		case EChaosVDParticleDataVisualizationFlags::ClusterConnectivityEdge:
		case EChaosVDParticleDataVisualizationFlags::CenterOfMass:
		case EChaosVDParticleDataVisualizationFlags::DrawDataOnlyForSelectedParticle:
		default:
			return TEXT("");
		}
	}
}

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDParticleVisualizationColorSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()
public:

	UPROPERTY(config, EditAnywhere, Category = "Colors Mode")
	EChaosVDParticleDebugColorMode ParticleColorMode = EChaosVDParticleDebugColorMode::State;
	
	UPROPERTY(config, EditAnywhere, Category = "Colors By Shape", meta=(EditCondition = "ParticleColorMode == EChaosVDParticleDebugColorMode::ShapeType", EditConditionHides))
	FChaosDebugDrawColorsByShapeType ColorsByShapeType;
	
	UPROPERTY(config, EditAnywhere, Category = "Colors By State", meta=(EditCondition = "ParticleColorMode == EChaosVDParticleDebugColorMode::State", EditConditionHides))
	FChaosDebugDrawColorsByState ColorsByParticleState;
	
	UPROPERTY(config, EditAnywhere, Category = "Colors By Solver", meta=(EditCondition = "ParticleColorMode == EChaosVDParticleDebugColorMode::ClientServer", EditConditionHides))
	FChaosDebugDrawColorsByClientServer ColorsByClientServer;
};

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDParticleVisualizationDebugDrawSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()
public:

	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	bool bShowDebugText = false;

	/** The depth priority used for while drawing contact data. Can be World or Foreground (with this one the shapes will be drawn on top of the geometry and be always visible) */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_World;

	/** Scale to apply to the Velocity vector before draw it. Unit is cm/s */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float VelocityScale = 0.5f;

	/** Scale to apply to the Angular Velocity vector before draw it. Unit is rad/s */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float AngularVelocityScale = 50.0f;

	/** Scale to apply to the Acceleration vector before draw it. Unit is cm/s2 */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float AccelerationScale = 0.005f;

	/** Scale to apply to the Angular Acceleration vector before draw it. Unit is rad/s2 */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float AngularAccelerationScale = 0.5f;

	/** Scale to apply to the Linear Impulse vector before draw it. Unit is g.m/s */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float LinearImpulseScale = 0.001f;

	/** Scale to apply to the Angular Impulse vector before draw it. Unit is g.m2/s */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float AngularImpulseScale = 0.1f;

	/** Radius to use when creating the sphere that will represent the center of mass location */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float CenterOfMassRadius = 10.0f;

	/** Should a triangle mesh's BVH draw. */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	bool bDrawTriMeshBVH = false;

	/** What depth of a triangle mesh's BVH to draw. Level '-1' means draw everything. */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw, meta = (EditCondition = "bDrawTriMeshBVH", EditConditionHides))
	int TriMeshBVHDrawLevel = 0;

	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	FChaosParticleDataDebugDrawColors ColorSettings;

	float GetScaleFortDataID(EChaosVDParticleDataVisualizationFlags DataID) const;
	
	static void SetDataDebugDrawVisualizationFlags(EChaosVDParticleDataVisualizationFlags Flags);

	static EChaosVDParticleDataVisualizationFlags GetDataDebugDrawVisualizationFlags();

	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;

private:
	/** Set of flags to enable/disable visualization of specific particle data as debug draw */
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVD.EChaosVDParticleDataVisualizationFlags"))
	uint32 ParticleDataVisualizationFlags = static_cast<uint32>(EChaosVDParticleDataVisualizationFlags::Velocity | EChaosVDParticleDataVisualizationFlags::AngularVelocity);
};

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDParticleVisualizationSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()
public:

	static void SetGeometryVisualizationFlags(EChaosVDGeometryVisibilityFlags Flags);
	static EChaosVDGeometryVisibilityFlags GetGeometryVisualizationFlags();

	/** Set of flags to enable/disable visibility of specific types of geometry/particles */
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVD.EChaosVDGeometryVisibilityFlags"))
	uint32 GeometryVisibilityFlags = static_cast<uint32>(EChaosVDGeometryVisibilityFlags::Simulated | EChaosVDGeometryVisibilityFlags::Simple | EChaosVDGeometryVisibilityFlags::ShowHeightfields | EChaosVDGeometryVisibilityFlags::Complex | EChaosVDGeometryVisibilityFlags::Query);
};