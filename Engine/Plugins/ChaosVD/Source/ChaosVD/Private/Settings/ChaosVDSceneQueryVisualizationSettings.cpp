// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSceneQueryVisualizationSettings.h"

#include "ChaosVDSettingsManager.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSceneQueryVisualizationSettings)


void UChaosVDSceneQueriesVisualizationSettings::SetDataVisualizationFlags(EChaosVDSceneQueryVisualizationFlags NewFlags)
{
	if (UChaosVDSceneQueriesVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSceneQueriesVisualizationSettings>())
	{
		Settings->GlobalSceneQueriesVisualizationFlags = static_cast<uint32>(NewFlags);
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDSceneQueryVisualizationFlags UChaosVDSceneQueriesVisualizationSettings::GetDataVisualizationFlags()
{
	if (UChaosVDSceneQueriesVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSceneQueriesVisualizationSettings>())
	{
		return static_cast<EChaosVDSceneQueryVisualizationFlags>(Settings->GlobalSceneQueriesVisualizationFlags);
	}

	return EChaosVDSceneQueryVisualizationFlags::None;
}

bool UChaosVDSceneQueriesVisualizationSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, GlobalSceneQueriesVisualizationFlags, EChaosVDSceneQueryVisualizationFlags::EnableDraw);
}
