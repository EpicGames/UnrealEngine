// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"

#include "ChaosVDSceneQueryVisualizationSettings.generated.h"

/** Set of visualization flags options for Scene Queries */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDSceneQueryVisualizationFlags: uint32
{
	None					= 0 UMETA(Hidden),
	EnableDraw				= 1 << 0,
	DrawLineTraceQueries	= 1 << 1,
	DrawSweepQueries		= 1 << 2,
	DrawOverlapQueries		= 1 << 3,
	DrawHits				= 1 << 4,
	OnlyDrawSelectedQuery	= 1 << 5,
	HideEmptyQueries		= 1 << 6,
	HideSubQueries			= 1 << 7,
};
ENUM_CLASS_FLAGS(EChaosVDSceneQueryVisualizationFlags);

/** Available scene query Visualization Modes */
UENUM()
enum class EChaosVDSQFrameVisualizationMode : uint8
{
	/** All the recorded scene queries for the current frame, that passes the visualization flag filter
	 * will be shown */
	AllEnabledQueries,
	/** Scene queries will be shown one at the time, in the order they were recorded (per solver), controlled by the Scene Query browser timeline */
	PerSolverRecordingOrder
};

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDSceneQueriesVisualizationSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()

public:

	static void SetDataVisualizationFlags(EChaosVDSceneQueryVisualizationFlags NewFlags);
	static EChaosVDSceneQueryVisualizationFlags GetDataVisualizationFlags();

	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;

	/** If true, any debug draw text available will be drawn */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	bool bShowText = false;

	/** The depth priority used for while drawing. Can be World or Foreground (with this one the shapes representing the query
	 * will be drawn on top of the geometry and be always visible) */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_World;

	UPROPERTY()
	EChaosVDSQFrameVisualizationMode CurrentVisualizationMode = EChaosVDSQFrameVisualizationMode::AllEnabledQueries;

private:
	/** Set of flags to enable/disable visualization of specific scene queries data as debug draw */
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVD.EChaosVDSceneQueryVisualizationFlags"))
	uint32 GlobalSceneQueriesVisualizationFlags = static_cast<uint32>(EChaosVDSceneQueryVisualizationFlags::DrawHits | EChaosVDSceneQueryVisualizationFlags::DrawLineTraceQueries);
};
