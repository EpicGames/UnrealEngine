// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDSolverTrackSettings.generated.h"

/**
 * Settings object for Solver Tracks configuration
 */
UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDSolverTrackSettings : public UChaosVDSettingsObjectBase
{
	GENERATED_BODY()

public:
	/** Sync mode used to keep all timelines in sync during playback.
	 * Not: Not all solver tracks support all modes. When an unsupported mode is selected, the default mode, RecordedTimestamp, will be used
	 */
	UPROPERTY(config, EditAnywhere, Category="Timeline Sync")
	EChaosVDSyncTimelinesMode SyncMode = EChaosVDSyncTimelinesMode::RecordedTimestamp;
};
