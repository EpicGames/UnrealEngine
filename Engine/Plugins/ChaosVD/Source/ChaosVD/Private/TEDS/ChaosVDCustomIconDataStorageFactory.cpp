// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCustomIconDataStorageFactory.h"

#include "ChaosVDParticleEditorDataFactory.h"
#include "ChaosVDSceneParticle.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementIconOverrideColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCustomIconDataStorageFactory)

void UChaosVDCustomIconDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::RegisterQueries(DataStorage);

	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Add icon override column to Particle"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo)
			{
				if (RawObject.Object && TypeInfo.TypeInfo->IsChildOf(FChaosVDBaseSceneObject::StaticStruct()))
				{
					FChaosVDBaseSceneObject* SceneObject = static_cast<FChaosVDBaseSceneObject*>(RawObject.Object);

					Context.AddColumn(Row, FTypedElementIconOverrideColumn{ .IconName = SceneObject->GetIconName() });
				}
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag, FChaosVDObjectDataTag>()
			.None<FTypedElementIconOverrideColumn>()
		.Compile());
}
