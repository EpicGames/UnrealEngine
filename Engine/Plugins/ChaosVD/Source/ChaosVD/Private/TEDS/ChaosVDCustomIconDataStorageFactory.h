// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "ChaosVDCustomIconDataStorageFactory.generated.h"

/**
 * 
 */
UCLASS()
class UChaosVDCustomIconDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()
	
	virtual ~UChaosVDCustomIconDataStorageFactory() override = default;

	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};
