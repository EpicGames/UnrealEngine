// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDDataStorageVisibilityQueries.h"

#include "ChaosVDParticleEditorDataFactory.h"
#include "ChaosVDSceneParticle.h"
#include "DataStorage/Queries/Types.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDDataStorageVisibilityQueries)


struct FTypedElementScriptStructTypeInfoColumn;


void UChaosVDDataStorageVisibilityQueries::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::RegisterQueries(DataStorage);
	
	RegisterParticleAddVisibilityColumn(DataStorage);
	RegisterParticleVisibilityToColumnQuery(DataStorage);
	RegisterVisibilityColumnToParticleQuery(DataStorage);
}

void UChaosVDDataStorageVisibilityQueries::RegisterParticleAddVisibilityColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Particle Visibility Object to New Column"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo)
			{
				if (RawObject.Object && TypeInfo.TypeInfo == FChaosVDSceneParticle::StaticStruct())
				{
					FChaosVDSceneParticle* Particle = static_cast<FChaosVDSceneParticle*>(RawObject.Object);
					
					Context.AddColumn(Row, FVisibleInEditorColumn{ .bIsVisibleInEditor = Particle->IsVisible() });
				}
			})
		.Where()
			.All<FChaosVDObjectDataTag, FTypedElementSyncFromWorldTag>()
			.None<FVisibleInEditorColumn>()
		.Compile()
	);
}

void UChaosVDDataStorageVisibilityQueries::RegisterParticleVisibilityToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Particle Visibility Object to Column"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo, FVisibleInEditorColumn& VisibilityColumn)
			{
				if (RawObject.Object && TypeInfo.TypeInfo == FChaosVDSceneParticle::StaticStruct())
				{
					FChaosVDSceneParticle* Particle = static_cast<FChaosVDSceneParticle*>(RawObject.Object);
					
					VisibilityColumn.bIsVisibleInEditor = Particle->IsVisible();
				}
			})
		.Where()
			.All<FChaosVDObjectDataTag, FTypedElementSyncFromWorldTag>()
		.Compile()
	);
}

void UChaosVDDataStorageVisibilityQueries::RegisterVisibilityColumnToParticleQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	struct FSetParticleVisibilityCommand
	{
		void operator()()
		{
			if (bIsVisible)
			{
				Particle->RemoveHiddenFlag(EChaosVDHideParticleFlags::HiddenBySceneOutliner);
			}
			else
			{
				Particle->AddHiddenFlag(EChaosVDHideParticleFlags::HiddenBySceneOutliner);
			}

			Particle->UpdateGeometryComponentsVisibility(EChaosVDParticleVisibilityUpdateFlags::DirtyScene);	
		}
		FChaosVDSceneParticle* Particle;
		bool bIsVisible;
	};

	DataStorage.RegisterQuery(
		Select(
			TEXT("Particle Visibility Column to Object"),
			FProcessor(EQueryTickPhase::FrameEnd, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncDataStorageToExternal)).SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo, const FVisibleInEditorColumn& VisibilityColumn)
			{
				if (RawObject.Object && TypeInfo.TypeInfo == FChaosVDSceneParticle::StaticStruct())
				{
					FChaosVDSceneParticle* Particle = static_cast<FChaosVDSceneParticle*>(RawObject.Object);

					Context.PushCommand(FSetParticleVisibilityCommand
												{
													.Particle = Particle,
													.bIsVisible = VisibilityColumn.bIsVisibleInEditor
												});

					
				}
			})
		.Where()
			.All<FChaosVDObjectDataTag, FVisibilityChangedTag>()
		.Compile()
	);
}
