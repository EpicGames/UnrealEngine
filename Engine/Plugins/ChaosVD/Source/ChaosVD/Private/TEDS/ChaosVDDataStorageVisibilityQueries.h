// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "ChaosVDDataStorageVisibilityQueries.generated.h"

UCLASS()
class UChaosVDDataStorageVisibilityQueries : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	~UChaosVDDataStorageVisibilityQueries() override = default;

	void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:

	/**
	 * Adds the Visibility columns to new actors that do not have one already.
	 */
	void RegisterParticleAddVisibilityColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	/**
	 * Takes the visibility set on an actor and copies it to the Data Storage if they differ.
	 */
	void RegisterParticleVisibilityToColumnQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Takes the visibility stored in the Data Storage and copies it to the actor's visibility if the FTypedElementSyncBackToWorldTag
	 * has been set and the visibility differ.
	 */
	void RegisterVisibilityColumnToParticleQuery(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
};
