// Copyright Epic Games, Inc. All Rights Reserved.


#include "ChaosVDParentDataStorageFactory.h"

#include "ChaosVDModule.h"
#include "ChaosVDParticleEditorDataFactory.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDScene.h"
#include "ChaosVDTedsUtils.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHiearchyColumns.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDParentDataStorageFactory)

namespace Chaos::VD::ParentDataStorageUtils::Private
{
	using namespace UE::Editor::DataStorage::Queries;

	void UpdateParentData(ICoreProvider& DataStorage, IQueryContext& Context, FChaosVDTableRowParentColumn& RowParentColumn, RowHandle ItemRowHandle, RowHandle NewParentRowHandle)
	{
		if (DataStorage.IsRowAvailable(NewParentRowHandle))
		{
			if (RowParentColumn.ParentObject != NewParentRowHandle)
			{
				if (FChaosVDTableRowParentColumn* OldParentData = DataStorage.GetColumn<FChaosVDTableRowParentColumn>(RowParentColumn.ParentObject))
				{
					OldParentData->ChildrenSet.Remove(ItemRowHandle);
					Context.AddColumns<FTypedElementSyncBackToWorldTag>(RowParentColumn.ParentObject);
				}

				RowParentColumn.ParentObject = NewParentRowHandle;

				if (FChaosVDTableRowParentColumn* ParentData = DataStorage.GetColumn<FChaosVDTableRowParentColumn>(NewParentRowHandle))
				{
					ParentData->ChildrenSet.Emplace(NewParentRowHandle);
					Context.AddColumns<FTypedElementSyncBackToWorldTag>(NewParentRowHandle);
				}
			}
		}
	};
}

void UChaosVDParentDataStorageFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::RegisterQueries(DataStorage);
	
	DefaultRootObjectForCVDActors = MakeShared<FChaosVDBaseSceneObject>();

	DefaultRootObjectForCVDActors->SetDisplayName(TEXT("Chaos Visual Debugger"));
	DefaultRootObjectForCVDActors->SetIconName(TEXT("ChaosVisualDebugger"));

	DefaultRootObjectForCVDActors->SetTedsRowHandle(Chaos::VD::TedsUtils::AddObjectToDataStorage(DefaultRootObjectForCVDActors.Get()));
	
	Chaos::VD::TedsUtils::AddColumnToObject<FTypedElementFromCVDWorldTag>(DefaultRootObjectForCVDActors.Get());
	Chaos::VD::TedsUtils::AddColumnToObject<FChaosVDObjectDataTag>(DefaultRootObjectForCVDActors.Get());
	Chaos::VD::TedsUtils::AddColumnToObject<FChaosVDTableRowParentColumn>(DefaultRootObjectForCVDActors.Get());

	RegisterAddParentColumn(DataStorage);
	RegisterUpdateOrRemoveParentColumn(DataStorage);
}

void UChaosVDParentDataStorageFactory::RegisterAddParentColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;

	DataStorage.RegisterQuery(
		Select(
			TEXT("Add parent column to actor"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[&DataStorage, DefaultRoot = DefaultRootObjectForCVDActors](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Actor)
			{
				if (AActor* CVDActor = Cast<AActor>(Actor.Object))
				{
					if (DefaultRoot)
					{
						if (FChaosVDTableRowParentColumn* ParentData = DataStorage.GetColumn<FChaosVDTableRowParentColumn>(DefaultRoot->GetTedsRowHandle()))
						{
							FChaosVDTableRowParentColumn ParentColumn;
							ParentColumn.ParentObject = DefaultRoot->GetTedsRowHandle();

							FMapKeyView IdKey = FMapKeyView(CVDActor);
							RowHandle ActorRow = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, IdKey);
							ParentData->ChildrenSet.Emplace(ActorRow);
							Context.AddColumn(Row, MoveTemp(ParentColumn));
						}
					}
				}
			})
		.Where()
			.All<FTypedElementSyncFromWorldTag, FTypedElementActorTag, FTypedElementFromCVDWorldTag>()
			.None<FChaosVDTableRowParentColumn>()
		.Compile());
}

void UChaosVDParentDataStorageFactory::RegisterUpdateOrRemoveParentColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const
{
	using namespace UE::Editor::DataStorage::Queries;
	using namespace Chaos::VD::ParentDataStorageUtils::Private;
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync Particle's parent to column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[&DataStorage](IQueryContext& Context, RowHandle Row, const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo, FChaosVDTableRowParentColumn& ParentColumn)
			{
				if (RawObject.Object && TypeInfo.TypeInfo->IsChildOf(FChaosVDBaseSceneObject::StaticStruct()))
				{
					FChaosVDBaseSceneObject* Particle = static_cast<FChaosVDBaseSceneObject*>(RawObject.Object);
					
					RowHandle NewParentRowHandle = InvalidRowHandle;
					if (TSharedPtr<FChaosVDBaseSceneObject> NewParentSceneObject = Particle->GetParent().Pin())
					{
						NewParentRowHandle = NewParentSceneObject->GetTedsRowHandle();
					}
					else if (AChaosVDSolverInfoActor* NewParentActor = Cast<AChaosVDSolverInfoActor>(Particle->GetParentActor()))
					{
						FMapKeyView IdKey = FMapKeyView(NewParentActor);
						NewParentRowHandle = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, IdKey);
					}

					UpdateParentData(DataStorage, Context, ParentColumn, Row, NewParentRowHandle);
				}
			})
		.Where()
			.All<FTypedElementFromCVDWorldTag, FTypedElementSyncFromWorldTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync actor's parent to column"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[&DataStorage](IQueryContext& Context, RowHandle Row, const FTypedElementUObjectColumn& Actor, FChaosVDTableRowParentColumn& Parent)
			{
				if (AActor* ActorPtr = Cast<AActor>(Actor.Object))
				{
					RowHandle NewParentRowHandle = InvalidRowHandle;
	
					if (AActor* ParentActor = ActorPtr->GetAttachParentActor())
					{
						FMapKeyView IdKey = FMapKeyView(ParentActor);
						NewParentRowHandle = Context.LookupMappedRow(ICompatibilityProvider::ObjectMappingDomain, IdKey);
					}
					
					UpdateParentData(DataStorage, Context, Parent, Row, NewParentRowHandle);
				}
			})
		.Where()
			.All<FTypedElementActorTag, FTypedElementSyncFromWorldTag>()
		.Compile());

	DataStorage.RegisterQuery(
		Select(
				TEXT("Remove Row From Parent Data"),
				FObserver::OnRemove<FTypedElementLabelColumn>().SetExecutionMode(EExecutionMode::GameThread),
				[&DataStorage](IQueryContext& Context, RowHandle Row)
				{
					if (FChaosVDTableRowParentColumn* ParentData = DataStorage.GetColumn<FChaosVDTableRowParentColumn>(Row))
					{
						if (FChaosVDTableRowParentColumn* OldParentData = DataStorage.GetColumn<FChaosVDTableRowParentColumn>(ParentData->ParentObject))
						{
							OldParentData->ChildrenSet.Remove(Row);
							Context.AddColumns<FTypedElementSyncBackToWorldTag>(ParentData->ParentObject);
							Context.AddColumns<FTypedElementSyncBackToWorldTag>(Row);
						}
					}
				})
			.Where()
				.All<FTypedElementFromCVDWorldTag>()
			.Compile());
}
