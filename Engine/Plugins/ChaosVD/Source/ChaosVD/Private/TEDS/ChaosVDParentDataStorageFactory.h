// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "ChaosVDParentDataStorageFactory.generated.h"

USTRUCT(meta = (DisplayName = "Parent"))
struct FChaosVDTableRowParentColumn final : public FEditorDataStorageColumn
{
	GENERATED_BODY()

	UE::Editor::DataStorage::RowHandle ParentObject = UE::Editor::DataStorage::InvalidRowHandle;

	TArray<UE::Editor::DataStorage::RowHandle> Children;
	TSet<UE::Editor::DataStorage::RowHandle> ChildrenSet;
};

struct FChaosVDBaseSceneObject;

/**
 * 
 */
UCLASS()
class UChaosVDParentDataStorageFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UChaosVDParentDataStorageFactory() override = default;

	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;

private:
	/**
	 * Checks rows with actors that don't have a parent column yet if one needs to be added whenever
	 * the row is marked for updates.
	 */
	void RegisterAddParentColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;
	/**
	 * Updates the parent column with the parent from the actor or removes it if there's no parent associated
	 * with the actor anymore.
	 */
	void RegisterUpdateOrRemoveParentColumn(UE::Editor::DataStorage::ICoreProvider& DataStorage) const;

	TSharedPtr<FChaosVDBaseSceneObject> DefaultRootObjectForCVDActors;
};
