// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDParticleEditorDataFactory.h"

#include "ChaosVDParentDataStorageFactory.h"
#include "ChaosVDSceneParticle.h"
#include "TEDS/ChaosVDSelectionInterface.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "DataStorage/Handles.h"
#include "Elements/Columns/TypedElementCompatibilityColumns.h"
#include "Elements/Columns/TypedElementHandleColumn.h"
#include "Elements/Columns/TypedElementLabelColumns.h"
#include "Elements/Columns/TypedElementMiscColumns.h"
#include "Elements/Columns/TypedElementTypeInfoColumns.h"
#include "Elements/Columns/TypedElementVisibilityColumns.h"
#include "Elements/Framework/TypedElementQueryBuilder.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"
#include "Elements/Interfaces/TypedElementDataStorageInterface.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDParticleEditorDataFactory)

void UChaosVDParticleEditorDataFactory::RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::ICompatibilityProvider& DataStorageCompatibility)
{
	Super::RegisterTables(DataStorage, DataStorageCompatibility);

	UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
	if (!ensure(Registry))
	{
		return;
	}

	using namespace Chaos::VD::TypedElementDataUtil;

	Registry->RegisterElementType<FStructTypedElementData, true>(NAME_CVD_StructDataElement);
	Registry->RegisterElementInterface<ITypedElementSelectionInterface>(NAME_CVD_StructDataElement, NewObject<UChaosVDSelectionInterface>());
	Registry->RegisterElementInterface<ITedsTypedElementBridgeInterface>(NAME_CVD_StructDataElement, NewObject<UChaosVDTedsTypedElementBridgeInterface>());

	using namespace UE::Editor::DataStorage;

	TableHandle ObjectTable = DataStorage.RegisterTable
		<
			FChaosVDObjectDataTag, FTypedElementFromCVDWorldTag, FTypedElementLabelColumn, FTypedElementLabelHashColumn, 
			FVisibleInEditorColumn, FChaosVDTableRowParentColumn, FChaosVDActiveObjectTag
		>(
			FName("CVD_SceneObjectDataTable"),
			FTableRegistrationOptions{ .SourceTable = DataStorage.FindTable(FName("Editor_StandardExternalObjectTable")) });

	TableHandle ActorTable = DataStorage.RegisterTable
		<FChaosVDObjectDataTag, FTypedElementFromCVDWorldTag, FVisibleInEditorColumn, FChaosVDTableRowParentColumn, FChaosVDActiveObjectTag>(
		FName("CVD_ActorDataTable"),
		FTableRegistrationOptions{ .SourceTable = DataStorage.FindTable(FName("Editor_StandardActorTable")) });

	DataStorageCompatibility.RegisterTypeTableAssociation(FChaosVDBaseSceneObject::StaticStruct(), ObjectTable);
	DataStorageCompatibility.RegisterTypeTableAssociation(AChaosVDSolverInfoActor::StaticClass(), ActorTable);
}

void UChaosVDParticleEditorDataFactory::RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage)
{
	Super::RegisterQueries(DataStorage);

	using namespace UE::Editor::DataStorage::Queries;
	using namespace UE::Editor::DataStorage;
		
		DataStorage.RegisterQuery(
		Select(
			TEXT("Generate TypedElement Handle for ChaosVDDataObject"),
			FObserver::OnAdd<FChaosVDObjectDataTag>(),
			[](IQueryContext& Context, RowHandle Row, const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo)
			{
				if (RawObject.Object)
				{
					// Note: Templated AcquireTypedElementHandleForStruct<T> call to CreateTypedElementDataForStructData<T>
					// implicitly determines the type from T
					// Therefore it is necessary to explicitly downcast to FChaosVDSceneParticle so that the StructDataType::StaticStruct()
					// resolves the correct type info.
					if (TypeInfo.TypeInfo->IsChildOf(FChaosVDSceneParticle::StaticStruct()))
					{
						FChaosVDSceneParticle* Particle = static_cast<FChaosVDSceneParticle*>(RawObject.Object);
						const bool bAllowCreate = true;
						FTypedElementHandle Handle = Chaos::VD::TypedElementDataUtil::AcquireTypedElementHandleForStruct(Particle, bAllowCreate);
						Context.AddColumn(Row, Compatibility::FTypedElementColumn
						{
							.Handle = MoveTemp(Handle)
						});
					}
				}
			})
			.Compile());
	
	DataStorage.RegisterQuery(
		Select(
			TEXT("Sync particle name to label"),
			FProcessor(EQueryTickPhase::PrePhysics, DataStorage.GetQueryTickGroupName(EQueryTickGroups::SyncExternalToDataStorage))
				.SetExecutionMode(EExecutionMode::GameThread),
			[](IQueryContext& Context, RowHandle Row, 
				const FTypedElementExternalObjectColumn& RawObject, const FTypedElementScriptStructTypeInfoColumn& TypeInfo, FTypedElementLabelColumn& Label, FTypedElementLabelHashColumn& LabelHash)
			{
				if (RawObject.Object && TypeInfo.TypeInfo->IsChildOf(FChaosVDBaseSceneObject::StaticStruct()))
				{
					FChaosVDBaseSceneObject* SceneObject = static_cast<FChaosVDBaseSceneObject*>(RawObject.Object);
					const FString& LabelRef = SceneObject->GetDisplayNameRef();

					uint64 ObjectLabelHash = CityHash64(reinterpret_cast<const char*>(*LabelRef), LabelRef.Len() * sizeof(**LabelRef));
					if (LabelHash.LabelHash != ObjectLabelHash)
					{
						FString Name = SceneObject->GetDisplayName();
						LabelHash.LabelHash = ObjectLabelHash;
						Label.Label = MoveTemp(Name);
						Context.AddColumns<FTypedElementSyncBackToWorldTag>(Row);
					}
				}
			})
		.Where()
			.All<FChaosVDObjectDataTag, FTypedElementSyncFromWorldTag>()
		.Compile());
}
