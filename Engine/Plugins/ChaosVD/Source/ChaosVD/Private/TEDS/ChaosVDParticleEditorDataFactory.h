// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#pragma once

#include "DataStorage/CommonTypes.h"
#include "Elements/Interfaces/TypedElementDataStorageFactory.h"
#include "UObject/ObjectMacros.h"

#include "ChaosVDParticleEditorDataFactory.generated.h"

/**
 * TEDS tag added to any object that belongs to a CVD
 */
USTRUCT(meta = (DisplayName = "CVD Object Data"))
struct FChaosVDObjectDataTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * TEDS tag added to any object that belongs to a CVD World
 */
USTRUCT(meta = (DisplayName = "From CVD World"))
struct FTypedElementFromCVDWorldTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

/**
 * TEDS tag added to any object that belongs to a CVD World, and is active (visible in the scene outliner, and with valid data)
 */
USTRUCT(meta = (DisplayName = "CVD Active Object"))
struct FChaosVDActiveObjectTag final : public FEditorDataStorageTag
{
	GENERATED_BODY()
};

UCLASS()
class UChaosVDParticleEditorDataFactory : public UEditorDataStorageFactory
{
	GENERATED_BODY()

public:
	virtual ~UChaosVDParticleEditorDataFactory() override = default;

	virtual void RegisterTables(UE::Editor::DataStorage::ICoreProvider& DataStorage, UE::Editor::DataStorage::ICompatibilityProvider& DataStorageCompatibility) override;
	virtual void RegisterQueries(UE::Editor::DataStorage::ICoreProvider& DataStorage) override;
};
