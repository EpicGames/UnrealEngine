// Copyright Epic Games, Inc. All Rights Reserved.

#include "TEDS/ChaosVDSelectionInterface.h"

#include "ChaosVDSceneParticle.h"
#include "TEDS/ChaosVDStructTypedElementData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSelectionInterface)

FTedsRowHandle UChaosVDTedsTypedElementBridgeInterface::GetRowHandle(const FTypedElementHandle& InElementHandle) const
{
	const FChaosVDSceneParticle* Particle = Chaos::VD::TypedElementDataUtil::GetStructDataFromTypedElementHandle<FChaosVDSceneParticle>(InElementHandle);
	if (Particle)
	{
		return FTedsRowHandle(Particle->GetTedsRowHandle());
	}
	
	return FTedsRowHandle();
}
