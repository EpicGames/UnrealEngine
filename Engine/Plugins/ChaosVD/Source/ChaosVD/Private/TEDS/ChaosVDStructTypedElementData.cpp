// Copyright Epic Games, Inc. All Rights Reserved.

#include "TEDS/ChaosVDStructTypedElementData.h"

#include "Elements/Framework/TypedElementOwnerStore.h"
#include "Elements/Framework/TypedElementRegistry.h"
#include "UObject/StructOnScope.h"

namespace Chaos::VD::TypedElementDataUtil
{
	UE_DEFINE_TYPED_ELEMENT_DATA_RTTI(FStructTypedElementData);

	TTypedElementOwnerStore<FStructTypedElementData, void*> GCVDTypedStructDataElementOwnerStore = TTypedElementOwnerStore<FStructTypedElementData, void*>();

	TSharedPtr<FStructOnScope> FStructTypedElementData::GetDataAsStructScope() const
	{
		if (TypeInfo && RawData)
		{
			return MakeShared<FStructOnScope>(TypeInfo, static_cast<uint8*>(RawData));
		}

		return nullptr;
	}

	void DestroyTypedElementHandleForStruct(void* InElementData)
	{
		TTypedElementOwner<FStructTypedElementData> EditorElement = GCVDTypedStructDataElementOwnerStore.UnregisterElementOwner(InElementData);
		if (!EditorElement)
		{
			return;
		}

		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();
		if (!Registry)
		{
			return;
		}

		Registry->DestroyElement(EditorElement);
	}

	TSharedPtr<FStructOnScope> GetStructOnScopeDataFromTypedElementHandle(const FTypedElementHandle& InHandle, const bool bSilent)
	{
		const FStructTypedElementData* StructElement = InHandle.GetData<FStructTypedElementData>(bSilent);
		return StructElement ? StructElement->GetDataAsStructScope() : nullptr;
	}

	void CleanUpTypedElementStore()
	{
		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

		auto DestroyElement = [Registry](TTypedElementOwner<FStructTypedElementData>&& EditorElement)
		{
			if (Registry)
			{
				Registry->DestroyElement(EditorElement);
			}
			else
			{
				EditorElement.Private_DestroyNoRef();
			}
		};
	
		GCVDTypedStructDataElementOwnerStore.UnregisterElementOwners([](const TTypedElementOwner<FStructTypedElementData>& EditorElementDestroyElement)
		{
			return true;
		}, DestroyElement);
	}
}
