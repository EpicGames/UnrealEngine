// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "DataStorage/Features.h"
#include "Elements/Interfaces/TypedElementDataStorageCompatibilityInterface.h"

/** Set of utility methods to interact with TEDS using CVD types */
namespace Chaos::VD::TedsUtils
{
	template<typename ColumnType, typename ObjectType>
	static void AddColumnToObject(ObjectType* Object)
	{
		using namespace UE::Editor::DataStorage;

		if (const ICompatibilityProvider* Compatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
		{
			RowHandle Row = Compatibility->FindRowWithCompatibleObject(Object);
			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				if (DataStorage->IsRowAvailable(Row))
				{
					DataStorage->AddColumn<ColumnType>(Row);
				}
			}
		}
	}

	template<typename ColumnType>
	static void AddColumnToHandle(UE::Editor::DataStorage::RowHandle Handle)
	{
		using namespace UE::Editor::DataStorage;

		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (DataStorage->IsRowAvailable(Handle))
			{
				DataStorage->AddColumn<ColumnType>(Handle);
			}
		}
	}

	template<typename ColumnType>
	static void BatchAddOrRemoveColumn(TConstArrayView<UE::Editor::DataStorage::RowHandle> Rows)
	{
		using namespace UE::Editor::DataStorage;

		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			DataStorage->BatchAddRemoveColumns(Rows, { ColumnType::StaticStruct(), 1}, {});
		}
	}

	template<typename ColumnType>
	static void RemoveColumnToHandle(UE::Editor::DataStorage::RowHandle Handle)
	{
		using namespace UE::Editor::DataStorage;

		if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
		{
			if (DataStorage->IsRowAvailable(Handle))
			{
				DataStorage->RemoveColumn<ColumnType>(Handle);
			}
		}
	}

	template<typename ColumnType, typename ObjectType>
	static void RemoveColumnFromObject(ObjectType* Object)
	{
		using namespace UE::Editor::DataStorage;

		if (const ICompatibilityProvider* Compatibility = GetDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName))
		{
			RowHandle Row = Compatibility->FindRowWithCompatibleObject(Object);

			if (ICoreProvider* DataStorage = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName))
			{
				if (DataStorage->IsRowAvailable(Row))
				{
					DataStorage->RemoveColumn<ColumnType>(Row);
				}
			}
		}
	}

	template<typename ObjectType>
	static UE::Editor::DataStorage::RowHandle AddObjectToDataStorage(ObjectType* Object,
		UE::Editor::DataStorage::ICoreProvider* DataStorageInterface,
		UE::Editor::DataStorage::ICompatibilityProvider* DataStorageCompatibility)
	{
		using namespace UE::Editor::DataStorage;
		if (DataStorageCompatibility != nullptr && DataStorageInterface != nullptr)
		{
			return DataStorageCompatibility->AddCompatibleObject(Object);
		}

		return InvalidRowHandle;
	}

	template<typename ObjectType>
	static UE::Editor::DataStorage::RowHandle AddObjectToDataStorage(ObjectType* Object)
	{
		using namespace UE::Editor::DataStorage;
		ICoreProvider* DataStorageInterface = GetMutableDataStorageFeature<ICoreProvider>(StorageFeatureName);
		ICompatibilityProvider* DataStorageCompatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);

		return AddObjectToDataStorage(Object, DataStorageInterface, DataStorageCompatibility);	
	}

	template<typename ObjectType>
	static void RemoveObjectToFromDataStorage(ObjectType* Object, UE::Editor::DataStorage::ICompatibilityProvider* DataStorageCompatibility)
	{
		if (DataStorageCompatibility != nullptr)
		{
			DataStorageCompatibility->RemoveCompatibleObject(Object);
		}
	}

	template<typename ObjectType>
	static void RemoveObjectToFromDataStorage(ObjectType* Object)
	{
		using namespace UE::Editor::DataStorage;
		ICompatibilityProvider* DataStorageCompatibility = GetMutableDataStorageFeature<ICompatibilityProvider>(CompatibilityFeatureName);
		RemoveObjectToFromDataStorage(Object, DataStorageCompatibility);
	}
}
