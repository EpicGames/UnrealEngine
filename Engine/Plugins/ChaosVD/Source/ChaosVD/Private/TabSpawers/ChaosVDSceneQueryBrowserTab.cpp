// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSceneQueryBrowserTab.h"

#include "ChaosVDStyle.h"
#include "EditorModeManager.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Widgets/SChaosVDSceneQueryBrowser.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDSceneQueryBrowserTab::~FChaosVDSceneQueryBrowserTab()
{
}

TSharedRef<SDockTab> FChaosVDSceneQueryBrowserTab::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> SceneQueryBrowserTab =
	SNew(SDockTab)
	.TabRole(ETabRole::PanelTab)
	.Label(LOCTEXT("SceneQueryBrowserTab", "Scene Query Browser"))
	.ToolTipText(LOCTEXT("SceneQueryBrowserTabTabTip", "Shows all recorded scene queries for the current frame, and allows you to select them or play then in order"));

	if (const TSharedPtr<SChaosVDMainTab> MainTabPtr = OwningTabWidget.Pin())
	{
		SceneQueryBrowserTab->SetContent
		(
			SAssignNew(SceneQueryBrowser, SChaosVDSceneQueryBrowser, GetChaosVDScene(), MainTabPtr->GetEditorModeManager().AsWeak())
		);
	}
	else
	{
		SceneQueryBrowserTab->SetContent(GenerateErrorWidget());
	}

	SceneQueryBrowserTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("SceneQueriesInspectorIcon"));

	HandleTabSpawned(SceneQueryBrowserTab);

	return SceneQueryBrowserTab;
}

void FChaosVDSceneQueryBrowserTab::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	FChaosVDTabSpawnerBase::HandleTabClosed(InTabClosed);
}

#undef LOCTEXT_NAMESPACE
