// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDTabSpawnerBase.h"
#include "Templates/SharedPointer.h"

class SChaosVDSceneQueryBrowser;

/** Spawns and handles and instance for the visual debugger Scene Query browser tab */
class FChaosVDSceneQueryBrowserTab : public FChaosVDTabSpawnerBase
{
public:
	FChaosVDSceneQueryBrowserTab(const FName& InTabID, const TSharedPtr<FTabManager>& InTabManager, const TWeakPtr<SChaosVDMainTab>& InOwningTabWidget)
		: FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{
	}

	virtual ~FChaosVDSceneQueryBrowserTab() override;
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

	TWeakPtr<SChaosVDSceneQueryBrowser> GetSceneQueryDataInspectorInstance() const { return SceneQueryBrowser; }

protected:
	TSharedPtr<SChaosVDSceneQueryBrowser> SceneQueryBrowser;
};
