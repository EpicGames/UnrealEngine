// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCombinedTraceFile.h"

#include "ChaosVDModule.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "HAL/FileManager.h"
#include "HAL/FileManagerGeneric.h"
#include "Internationalization/Internationalization.h"
#include "Misc/ScopedSlowTask.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"

namespace Chaos::VisualDebugger::CombinedTraceFile
{
	int64 FInnerFileHandle::Tell()
	{
		return ContainerFileHandle->Tell() - DataOffset;
	}

	bool FInnerFileHandle::Seek(int64 NewPosition)
	{
		if (NewPosition >= DataOffset && NewPosition <= (DataOffset + DataSize))
		{
			return ensure(ContainerFileHandle->Seek(DataOffset + NewPosition));
		}
	
		return ensure(false);
	}

	bool FInnerFileHandle::SeekFromEnd(int64 NewPositionRelativeToEnd)
	{
		check(NewPositionRelativeToEnd <= 0);
		const int64 EndOfFilePos = DataOffset + DataSize;

		const int64 NewPosition = EndOfFilePos + NewPositionRelativeToEnd;
		if (NewPosition >= DataOffset && NewPosition <= (DataOffset + DataSize))
		{
			return ensure(ContainerFileHandle->Seek(NewPosition));
		}

		return ensure(false);
	}

	bool FInnerFileHandle::Read(uint8* Destination, int64 BytesToRead)
	{
		return ensure(ContainerFileHandle->Read(Destination, BytesToRead));
	}

	bool FInnerFileHandle::ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset)
	{
		return ensure(ContainerFileHandle->ReadAt(Destination, BytesToRead, Offset + DataOffset));
	}

	bool FInnerFileHandle::Write(const uint8* Source, int64 BytesToWrite)
	{
		return ensureMsgf(false, TEXT("Not Supported"));
	}

	bool FInnerFileHandle::Flush(const bool bFullFlush)
	{
		return ensureMsgf(false, TEXT("Not Supported"));
	}

	bool FInnerFileHandle::Truncate(int64 NewSize)
	{
		return ensureMsgf(false, TEXT("Not Supported"));
	}

	int64 FInnerFileHandle::Size()
	{
		return DataSize;
	}

	FInnerFileHandle::FInnerFileHandle(const FString& InContainerFilePath, int64 DataOffset, int64 DataSize)
	: ContainerFileHandle(nullptr)
	, DataOffset(DataOffset)
	, DataSize(DataSize)
	{
		IPlatformFile& FileSystem = IPlatformFile::GetPlatformPhysical();
		ContainerFileHandle = TUniquePtr<IFileHandle>(FileSystem.OpenRead(*InContainerFilePath));
		if (ContainerFileHandle)
		{
			ContainerFileHandle->Seek(DataOffset);
		}
	}

	bool CombineFiles(const TArray<TUniquePtr<IFileHandle>>& InFileHandlesToCombine, const FString& InCombinedFilePathName)
	{
		TUniquePtr<IFileHandle> CombinedFileHandle = nullptr;

		if (InFileHandlesToCombine.Num() == 0)
		{
			UE_LOGF(LogChaosVDEditor, Error, "Failed to combine files because an empty file handles array was provided.");
			return false;
		}

		IPlatformFile& FileSystem = IPlatformFile::GetPlatformPhysical();
		
		if (FileSystem.FileExists(*InCombinedFilePathName))
		{
			UE_LOGF(LogChaosVDEditor, Error, "Failed to combine files because an the target file name already exists.");
			return false;
		}

		IFileManager& FileManager = IFileManager::Get();

		FArchive* FileWriter = FileManager.CreateFileWriter(*InCombinedFilePathName);

		if (!FileWriter)
		{
			UE_LOGF(LogChaosVDEditor, Error, "Failed to combine files because there was an error while creating the target file.")
			return false;
		}
		
		constexpr float AmountOfWork = 1.0f;
		const float CombiningFilesPercentagePerElement = 1.0f / static_cast<float>(InFileHandlesToCombine.Num());

		FScopedSlowTask CombiningOpenRecordingsTask(AmountOfWork, NSLOCTEXT("ChaosVisualDebugger", "CombiningFilesMessage", "Combining Open Recordings ..."));
		CombiningOpenRecordingsTask.MakeDialog();

		FChaosVDArchiveHeader::Current().Serialize(*FileWriter);

		int64 FileTableHeaderPos = FileWriter->Tell();
		{
			FFileTableHeader EmptyHeaderPlaceholder;
			*FileWriter << EmptyHeaderPlaceholder;
		}

		FFileTable FileTable;
		for (const TUniquePtr<IFileHandle>& FileHandle : InFileHandlesToCombine)
		{
			FScopedSlowTask ProcessingFileForCombiningTask(AmountOfWork, NSLOCTEXT("ChaosVisualDebugger", "ProcessingFileForCombiningTaskMessage", "Processing file ..."));
			CombiningOpenRecordingsTask.MakeDialog();

			FFileEntry FileEntry;
			FileEntry.Size = FileHandle->Size();
			FileEntry.StartPos = FileWriter->Tell();

			FileTable.Files.Emplace(FileEntry);

			constexpr int32 ReportProgressAfterReadSize = PLATFORM_FILE_WRITER_BUFFER_SIZE * 10;
			constexpr float ProcessingFileMinProgressAmount = 1.0f / static_cast<float>(ReportProgressAfterReadSize);

			constexpr int32 MaxChunkSize = PLATFORM_FILE_WRITER_BUFFER_SIZE;
			TArray<uint8, TInlineAllocator<MaxChunkSize>> ChunkToCopy;
			int64 DataRemaining = FileEntry.Size;

			FileHandle->Seek(0);

			while (DataRemaining > 0)
			{
				int32 DataSizeToRead = MaxChunkSize;
				if (MaxChunkSize > DataRemaining)
				{
					DataSizeToRead = static_cast<int32>(DataRemaining);
				}

				ChunkToCopy.SetNum(DataSizeToRead);

				FileHandle->Read(ChunkToCopy.GetData(), DataSizeToRead);
				FileWriter->Serialize(ChunkToCopy.GetData(), DataSizeToRead);

				DataRemaining -= DataSizeToRead;
				
				const int64 DataProcessedSoFar = FileEntry.Size - DataRemaining;
				if (DataProcessedSoFar % ReportProgressAfterReadSize == 0)
				{
					ProcessingFileForCombiningTask.EnterProgressFrame(ProcessingFileMinProgressAmount);
				}
			}
			
			CombiningOpenRecordingsTask.EnterProgressFrame(CombiningFilesPercentagePerElement);
		}

		FFileTableHeader FileHeader;
		FileHeader.FileTablePos = FileWriter->Tell();

		*FileWriter << FileTable;

		// Now that we have the final file table location, we can go back and re-serialize
		// our header with the correct position data
		FileWriter->Seek(FileTableHeaderPos);
		*FileWriter << FileHeader;

		FileWriter->Close();
		
		return true;
	}

	TArray<TUniquePtr<IFileHandle>> GetInnerFileHandles(const FString& InContainerFilePath)
	{
		TArray<TUniquePtr<IFileHandle>> FileHandles;
		IPlatformFile& FileSystem = IPlatformFile::GetPlatformPhysical();
		
		IFileHandle* ContainerFileHandle = FileSystem.OpenRead(*InContainerFilePath);
		if (!ensure(ContainerFileHandle))
		{
			UE_LOGF(LogChaosVDEditor, Error, "Failed to read combined trace file. There was an error trying to open the file for read | File path [%ls].", *InContainerFilePath);
			return FileHandles;
		}

		FArchiveFileReaderGeneric FileReader(ContainerFileHandle, *InContainerFilePath, ContainerFileHandle->Size());

		FChaosVDArchiveHeader::Current().Serialize(FileReader);

		FFileTableHeader FileTableHeader;
		FileReader << FileTableHeader;

		if (FileTableHeader.FileTablePos == 0)
		{
			UE_LOGF(LogChaosVDEditor, Error, "Failed to read combined trace file. The serialized file table header is not valid | File path [%ls].", *InContainerFilePath);
			return FileHandles;
		}

		FileReader.Seek(FileTableHeader.FileTablePos);

		FFileTable FileTable;
		FileReader << FileTable;

		for (const FFileEntry& FileEntry : FileTable.Files)
		{
			TUniquePtr<FInnerFileHandle> PackedDataFileHandle = MakeUnique<FInnerFileHandle>(InContainerFilePath, FileEntry.StartPos, FileEntry.Size);
			if (ensure(PackedDataFileHandle->IsValidHandle()))
			{
				FileHandles.Emplace(MoveTemp(PackedDataFileHandle));
			}
			else
			{
				UE_LOGF(LogChaosVDEditor, Error, "Failed to create a inner file handle for. there was an error trying to open the file for read | Combined file path [%ls].", *InContainerFilePath);
			}
		}

		/** Each InnerFileHandle opens its own file table to the container file, so it is fine to close this handle */
		FileReader.Close();
		
		return MoveTemp(FileHandles);
	}
}
