// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "HAL/Platform.h"
#include "Containers/UnrealString.h"
#include "GenericPlatform/GenericPlatformFile.h"
#include "Templates/SharedPointer.h"

namespace Chaos::VisualDebugger::CombinedTraceFile
{
	/** File handle that allows other systems, like UE Trace, to read data from a combined file, as it was a separated file */
	class FInnerFileHandle : public IFileHandle
	{
	public:
		virtual int64 Tell() override;
		virtual bool Seek(int64 NewPosition) override;
		virtual bool SeekFromEnd(int64 NewPositionRelativeToEnd = 0) override;
		virtual bool Read(uint8* Destination, int64 BytesToRead) override;
		virtual bool ReadAt(uint8* Destination, int64 BytesToRead, int64 Offset) override;
		virtual bool Write(const uint8* Source, int64 BytesToWrite) override;
		virtual bool Flush(const bool bFullFlush = false) override;
		virtual bool Truncate(int64 NewSize) override;
		virtual int64 Size() override;

		FInnerFileHandle(const FString& InContainerFilePath, int64 DataOffset, int64 DataSize);

		bool IsValidHandle() const
		{
			return ContainerFileHandle.IsValid();
		}

	private:
		TUniquePtr<IFileHandle> ContainerFileHandle;
		int64 DataOffset;
		int64 DataSize;
	};

	struct FFileEntry
	{
		int64 StartPos = 0;
		int64 Size = 0;
		
		void Serialize(FArchive& Ar)
		{
			Ar << StartPos; 
			Ar << Size; 
		}
	};

	struct FFileTable
	{
		TArray<FFileEntry> Files;
		
		void Serialize(FArchive& Ar)
		{
			Ar << Files; 
		}
	};

	struct FFileTableHeader
	{
		int64 FileTablePos = 0;

		void Serialize(FArchive& Ar)
		{
			Ar << FileTablePos;
		}		
	};

	inline FArchive& operator<<(FArchive& Ar, FFileTableHeader& Data)
	{
		Data.Serialize(Ar);
		return Ar;
	}

	inline FArchive& operator<<(FArchive& Ar, FFileTable& Data)
	{
		Data.Serialize(Ar);
		return Ar;
	}

	inline FArchive& operator<<(FArchive& Ar, FFileEntry& Data)
	{
		Data.Serialize(Ar);
		return Ar;
	}

	/**
	 * Takes an array of file handles and creates a new file that contains all their data, but that can be accessive as indivitual files later on
	 * @param InFileHandlesToCombine File handles to get the data to combine from
	 * @param InCombinedFilePathName Desired file path name for the generated file
	 * @return true if successful
	 */
	bool CombineFiles(const TArray<TUniquePtr<IFileHandle>>& InFileHandlesToCombine, const FString& InCombinedFilePathName);

	/**
	 * Take a file path to a combined CVD recording file, and returns file handles for the individual files inside it
	 * @param InContainerFilePath File path name to the combined CVD file
	 * @return Unique file handles for each one of the files that were combined
	 */
	TArray<TUniquePtr<IFileHandle>> GetInnerFileHandles(const FString& InContainerFilePath);

}


