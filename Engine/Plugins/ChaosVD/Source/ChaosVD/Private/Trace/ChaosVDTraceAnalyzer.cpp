// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/ChaosVDTraceAnalyzer.h"

#include "ChaosVDModule.h"
#include "Containers/Ticker.h"
#include "Misc/MessageDialog.h"
#include "Trace/ChaosVDTraceProvider.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Frames.h"

void FChaosVDTraceAnalyzer::OnAnalysisBegin(const FOnAnalysisContext& Context)
{
	FInterfaceBuilder& Builder = Context.InterfaceBuilder;
	Builder.RouteEvent(RouteId_ChaosVDParticleDestroyed, "ChaosVDLogger", "ChaosVDParticleDestroyed");
	
	Builder.RouteEvent(RouteId_ChaosVDSolverFrameStart, "ChaosVDLogger", "ChaosVDSolverFrameStart");
	Builder.RouteEvent(RouteId_ChaosVDSolverFrameEnd, "ChaosVDLogger", "ChaosVDSolverFrameEnd");
	
	Builder.RouteEvent(RouteId_ChaosVDSolverStepStart, "ChaosVDLogger", "ChaosVDSolverStepStart");
	Builder.RouteEvent(RouteId_ChaosVDSolverStepEnd, "ChaosVDLogger", "ChaosVDSolverStepEnd");
	
	Builder.RouteEvent(RouteId_ChaosVDBinaryDataStart, "ChaosVDLogger", "ChaosVDBinaryDataStart");
	Builder.RouteEvent(RouteId_ChaosVDBinaryDataContent, "ChaosVDLogger", "ChaosVDBinaryDataContent");
	Builder.RouteEvent(RouteId_ChaosVDBinaryDataEnd, "ChaosVDLogger", "ChaosVDBinaryDataEnd");
	Builder.RouteEvent(RouteId_ChaosVDSolverSimulationSpace, "ChaosVDLogger", "ChaosVDSolverSimulationSpace");

	Builder.RouteEvent(RouteId_ChaosVDNetworkTickOffset, "ChaosVDLogger", "ChaosVDNetworkTickOffset");
	Builder.RouteEvent(RouteId_ChaosVDRolledBackDataID, "ChaosVDLogger", "ChaosVDRolledBackDataID");
	Builder.RouteEvent(RouteId_ChaosVDUsesAutoRTFM, "ChaosVDLogger", "ChaosVDUsesAutoRTFM");
	
	Builder.RouteEvent(RouteId_ChaosVDMiscHeaderData, "ChaosVDLogger", "ChaosVDMiscHeaderData");

	Builder.RouteEvent(RouteId_BeginFrame, "Misc", "BeginFrame");
	Builder.RouteEvent(RouteId_EndFrame, "Misc", "EndFrame");

	TraceServices::FAnalysisSessionEditScope _(Session);
	ChaosVDTraceProvider->CreateRecordingInstanceForSession(Session.GetName());
}

void FChaosVDTraceAnalyzer::OnAnalysisEnd()
{
	ChaosVDTraceProvider->HandleAnalysisComplete();

	OnAnalysisComplete().Broadcast();
}

void FChaosVDTraceAnalyzer::PushSimulatedSolverTrackForGTData(const UE::Trace::IAnalyzer::FOnEventContext& Context, const UE::Trace::IAnalyzer::FEventData& EventData)
{
	FChaosVDSolverFrameData NewSimulatedSolverFrameData;

	int32 CurrentGTTrackID = ChaosVDTraceProvider->GetCurrentGameThreadTrackID();
	if (CurrentGTTrackID == INDEX_NONE)
	{
		CurrentGTTrackID = ChaosVDTraceProvider->RemapSolverID(0);
		ChaosVDTraceProvider->SetCurrentGameThreadTrackID(CurrentGTTrackID);
	}

	NewSimulatedSolverFrameData.SolverID = CurrentGTTrackID;
	NewSimulatedSolverFrameData.FrameCycle = EventData.GetValue<uint64>("Cycle");
	NewSimulatedSolverFrameData.bIsKeyFrame = true;
	NewSimulatedSolverFrameData.bIsResimulated = false;
	NewSimulatedSolverFrameData.StartTime = Context.EventTime.AsSeconds(NewSimulatedSolverFrameData.FrameCycle);

	static FString GeneratedStageName = TEXT("Stage 0");
	NewSimulatedSolverFrameData.SolverSteps.AddDefaulted_GetRef().StepName = GeneratedStageName;

	TSharedPtr<FChaosVDGameFrameData> NewGameFrameData = MakeShared<FChaosVDGameFrameData>();
	NewSimulatedSolverFrameData.GetCustomData().GetOrAddDefaultData<FChaosVDGameFrameDataWrapper>()->FrameData = NewGameFrameData;

	if (TSharedPtr<FChaosVDGameFrameDataWrapperContext> FrameDataContext = NewGameFrameData->GetCustomDataHandler().GetOrAddDefaultData<FChaosVDGameFrameDataWrapperContext>())
	{
		for (const TPair<int32, int32>& RemappedID : ChaosVDTraceProvider->RemappedSolversIDs)
		{
			FrameDataContext->SupportedSolverIDs.Emplace(RemappedID.Value);
		}
	}

	NewSimulatedSolverFrameData.DebugFName = FName(TEXT("Additional Game Frame Data Track"));
	
	NewSimulatedSolverFrameData.AddAttributes(EChaosVDSolverFrameAttributes::HasGTDataToReRoute);

	{
		if (FChaosVDSolverFrameData* PrevFrameData = ChaosVDTraceProvider->GetCurrentSolverFrame(CurrentGTTrackID))
		{
			PrevFrameData->EndTime = NewSimulatedSolverFrameData.StartTime;
		}
	}
					
	ChaosVDTraceProvider->StartSolverFrame(NewSimulatedSolverFrameData.SolverID, MoveTemp(NewSimulatedSolverFrameData));
}

bool FChaosVDTraceAnalyzer::OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context)
{
	LLM_SCOPE_BYNAME(TEXT("Insights/FChaosVDTraceAnalyzer"));

	TraceServices::FAnalysisSessionEditScope _(Session);

	const FEventData& EventData = Context.EventData;
	
	switch (RouteId)
	{
	case RouteId_BeginFrame:
		{
			uint8 FrameType = EventData.GetValue<uint8>("FrameType");
			if (static_cast<ETraceFrameType>(FrameType) == TraceFrameType_Game)
			{
				// TODO: Currently, CVD does not support multiple GT tracks -
				// The proper long term solution is remove the concept of GT tracks altogether, and just have 
				// a dedicated solver data track, that happens to represent GT data.
				// Until that is done, we generate fake solver track to hold the data in multi recording/session mode

				if (ChaosVDTraceProvider->DoesOwnRecordingInstance())
				{
					TSharedPtr<FChaosVDGameFrameData> FrameData = MakeShared<FChaosVDGameFrameData>();
					FrameData->FirstCycle = EventData.GetValue<uint64>("Cycle");
					FrameData->StartTime = Context.EventTime.AsSeconds(FrameData->FirstCycle);

					ChaosVDTraceProvider->StartGameFrame(FrameData);
				}
				else
				{
					PushSimulatedSolverTrackForGTData(Context, EventData);
				}
			}

			break;
		}

	case RouteId_EndFrame:
		{
			uint8 FrameType = EventData.GetValue<uint8>("FrameType");
			if (static_cast<ETraceFrameType>(FrameType) == TraceFrameType_Game)
			{
				if (TSharedPtr<FChaosVDGameFrameData> CurrentFrameData = ChaosVDTraceProvider->GetCurrentGameFrame().Pin())
				{
					CurrentFrameData->LastCycle = EventData.GetValue<uint64>("Cycle");
					CurrentFrameData->EndTime = Context.EventTime.AsSeconds(CurrentFrameData->LastCycle);	
				}
			}
			break;
		}
	case RouteId_ChaosVDSolverFrameStart:
		{
			FChaosVDSolverFrameData NewFrameData;

			NewFrameData.SolverID = EventData.GetValue<int32>("SolverID");
			NewFrameData.FrameCycle = EventData.GetValue<uint64>("Cycle");
			NewFrameData.InternalFrameNumber = EventData.GetValue<int32>("CurrentFrameNumber", INDEX_NONE);
			NewFrameData.bIsKeyFrame = EventData.GetValue<bool>("IsKeyFrame");
			NewFrameData.bIsResimulated = EventData.GetValue<bool>("IsReSimulated");
			NewFrameData.StartTime = Context.EventTime.AsSeconds(NewFrameData.FrameCycle);

			int32 RemappedExistingSolverID = ChaosVDTraceProvider->GetRemappedSolverID(NewFrameData.SolverID);

			if (int32* TickOffsetPtr = ChaosVDTraceProvider->GetCurrentTickOffsetsBySolverID().Find(RemappedExistingSolverID))
			{
				NewFrameData.NetworkTickOffset = *TickOffsetPtr;
			}

			FWideStringView DebugNameView;
			EventData.GetString("DebugName", DebugNameView);
			NewFrameData.DebugFName = FName(DebugNameView);

			// Currently not all solvers have an end frame event, so lets just set the end frame time of the previous frame, with the start of this new one.
			if (RemappedExistingSolverID != INDEX_NONE)
			{
				if (FChaosVDSolverFrameData* PrevFrameData = ChaosVDTraceProvider->GetCurrentSolverFrame(RemappedExistingSolverID))
				{
					PrevFrameData->EndTime = NewFrameData.StartTime;
				}
			}

			if (RemappedExistingSolverID == INDEX_NONE)
			{
				RemappedExistingSolverID = ChaosVDTraceProvider->RemapSolverID(NewFrameData.SolverID);	
			}

			NewFrameData.SolverID = RemappedExistingSolverID;

			// Add an empty frame. It will be filled out by the solver trace events
			ChaosVDTraceProvider->StartSolverFrame(NewFrameData.SolverID, MoveTemp(NewFrameData));
			break;
		}
	case RouteId_ChaosVDSolverFrameEnd:
		{
			break;
		}
	case RouteId_ChaosVDSolverStepStart:
		{
			const int32 SolverID = ChaosVDTraceProvider->GetRemappedSolverID(EventData.GetValue<int32>("SolverID"));

			// This can be null if the recording started Mid-Frame. In this case we just discard the data for now
			if (FChaosVDSolverFrameData* FrameData = ChaosVDTraceProvider->GetCurrentSolverFrame(SolverID))
			{
				if (FrameData->SolverSteps.Num() > 0)
				{
					FChaosVDFrameStageData& LastSolverStage = FrameData->SolverSteps.Last();
					if (EnumHasAnyFlags(LastSolverStage.StageFlags, EChaosVDSolverStageFlags::Open) && ensure(!EnumHasAnyFlags(LastSolverStage.StageFlags, EChaosVDSolverStageFlags::ExplicitStage)))
					{
						// If the current Solver stage was implicitly generated, we need to close it before starting a new one.
						// This should not happen with an explicitly recorded stage
						EnumRemoveFlags(LastSolverStage.StageFlags, EChaosVDSolverStageFlags::Open);
					}
				}

				// Add an empty step. It will be filled out by the particle (and later on other objects/elements) events
				FChaosVDFrameStageData& NewSolverStageData = FrameData->SolverSteps.AddDefaulted_GetRef();

				FWideStringView DebugNameView;
				EventData.GetString("StepName", DebugNameView);
				NewSolverStageData.StepName = DebugNameView;
				EnumAddFlags(NewSolverStageData.StageFlags, EChaosVDSolverStageFlags::Open);
				EnumAddFlags(NewSolverStageData.StageFlags, EChaosVDSolverStageFlags::ExplicitStage);
			}
	
			break;
		}
	case RouteId_ChaosVDSolverStepEnd:
		{
			const int32 SolverID = ChaosVDTraceProvider->GetRemappedSolverID(EventData.GetValue<int32>("SolverID"));
			if (FChaosVDSolverFrameData* FrameData = ChaosVDTraceProvider->GetCurrentSolverFrame(SolverID))
			{
				if (FrameData->SolverSteps.Num() > 0)
				{
					EnumRemoveFlags(FrameData->SolverSteps.Last().StageFlags, EChaosVDSolverStageFlags::Open);
				}
			}
			break;
		}
	case RouteId_ChaosVDParticleDestroyed:
		{
			const int32 SolverID = ChaosVDTraceProvider->GetRemappedSolverID(EventData.GetValue<int32>("SolverID"));

			if (FChaosVDSolverFrameData* FrameData = ChaosVDTraceProvider->GetCurrentSolverFrame(SolverID))
			{
				int32 ParticleDestroyedID = EventData.GetValue<int32>("ParticleID");

				// We need to add all particles that were destroyed in any step of this frame to the Frame data structure
				// So we can properly process these events when not all the steps are played back
				// Either because of the lock sub-step feature or because we are manually scrubbing from frame to frame
				FrameData->ParticlesDestroyedIDs.Add(ParticleDestroyedID);
				
				if (FrameData->SolverSteps.Num() > 0)
				{
					FrameData->SolverSteps.Last().ParticlesDestroyedIDs.Add(ParticleDestroyedID);
				}
			}

			break;
		}
	case RouteId_ChaosVDBinaryDataStart:
		{
			const int32 DataID = EventData.GetValue<int32>("DataID");
			
			FChaosVDTraceProvider::FBinaryDataContainer& DataContainer = ChaosVDTraceProvider->FindOrAddUnprocessedData(DataID);
			DataContainer.bIsCompressed = EventData.GetValue<bool>("IsCompressed");
			DataContainer.UncompressedSize = EventData.GetValue<uint32>("OriginalSize");
			DataContainer.DataID = EventData.GetValue<int32>("DataID");

			EventData.GetString("TypeName", DataContainer.TypeName);

			const uint32 DataSize = EventData.GetValue<uint32>("DataSize");
			DataContainer.RawData.Reserve(DataSize);

			break;
		}
	case RouteId_ChaosVDBinaryDataContent:
		{
			const int32 DataID = EventData.GetValue<int32>("DataID");	

			FChaosVDTraceProvider::FBinaryDataContainer& DataContainer = ChaosVDTraceProvider->FindOrAddUnprocessedData(DataID);

			const TArrayView<const uint8> SerializedDataChunk = EventData.GetArrayView<uint8>("RawData");
			DataContainer.RawData.Append(SerializedDataChunk.GetData(), SerializedDataChunk.Num());

			break;
		}
	case RouteId_ChaosVDBinaryDataEnd:
		{
			const int32 DataID = EventData.GetValue<int32>("DataID");

			if (!ChaosVDTraceProvider->ProcessBinaryData(DataID))
			{
				// This can happen during live debugging as we miss some of the events at the beginning.
				// Loading a trace file that was recorded as part of a live session, will have the same issue.
				UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] FailedToProcess Binary Data with ID [%d]", ANSI_TO_TCHAR(__FUNCTION__), DataID);
			}

			break;
		}
	case RouteId_ChaosVDSolverSimulationSpace:
		{
			const int32 SolverID = ChaosVDTraceProvider->GetRemappedSolverID(EventData.GetValue<int32>("SolverID"));

			FVector Position;
			CVD_READ_TRACE_VECTOR(Position, Position, float, EventData);

			FQuat Rotation;
			CVD_READ_TRACE_QUAT(Rotation, Rotation, float, EventData);

			// This can be null if the recording started Mid-Frame. In this case we just discard the data for now
			if (FChaosVDSolverFrameData* FrameData = ChaosVDTraceProvider->GetCurrentSolverFrame(SolverID))
			{
				FrameData->SimulationTransform.SetLocation(Position);
				FrameData->SimulationTransform.SetRotation(Rotation);
			}
			break;
		}
	case RouteId_ChaosVDNetworkTickOffset:
		{
			FChaosVDTrackedTransform TrackedTransform;

			const int32 TickOffset = EventData.GetValue<int32>("Offset");
			const int32 SolverID = ChaosVDTraceProvider->GetRemappedSolverID(EventData.GetValue<int32>("SolverID"));

			if (SolverID != INDEX_NONE)
			{
				ChaosVDTraceProvider->GetCurrentTickOffsetsBySolverID().FindOrAdd(SolverID) = TickOffset;
			}

			break;
		}
	case RouteId_ChaosVDRolledBackDataID:
		{
			const int32 DataID = EventData.GetValue<int32>("DataID");
			ChaosVDTraceProvider->RemoveUnprocessedData(DataID);

			break;
		}	
	case RouteId_ChaosVDUsesAutoRTFM:
		{
			if (EventData.GetValue<bool>("bUsingAutoRTFM"))
			{
				UE_LOGF(LogChaosVDEditor, Log, "Processing recording was made with AutoRTFM enabled. ");
			}

			break;
		}
	case RouteId_ChaosVDMiscHeaderData:
		{
			if (EventData.GetValue<bool>("bUsesSharedGeometryContextArchive"))
			{
				ChaosVDTraceProvider->CreateGeometryArchiveContext();
			}
			break;
		}
	default:
		break;
	}

	return true;
}
