// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDRecording.h"
#include "Trace/Analyzer.h"
#include "Templates/SharedPointer.h"
#include "ChaosVisualDebugger/ChaosVDOptionalDataChannel.h"

struct FChaosVDFrameStageData;
class FChaosVDTraceProvider;

namespace TraceServices { class IAnalysisSession; }

DECLARE_MULTICAST_DELEGATE(FChaosVDTraceAnalysisComplete)

/** Analyzer class for Chaos VD trace recordings.
 * It processes all Chaos VD trace events and rebuilds the recording in a usable
 * Recording struct handled by a custom Trace Provider
 */
class FChaosVDTraceAnalyzer final : public UE::Trace::IAnalyzer
{
public:
	FChaosVDTraceAnalyzer(TraceServices::IAnalysisSession& Session, const TSharedRef<FChaosVDTraceProvider>& InChaosVDTraceProvider)
	: Session(Session),
	ChaosVDTraceProvider(InChaosVDTraceProvider)
	{
	}

	virtual void OnAnalysisBegin(const FOnAnalysisContext& Context) override;
	virtual void OnAnalysisEnd() override;
	void PushSimulatedSolverTrackForGTData(const UE::Trace::IAnalyzer::FOnEventContext& Context, const UE::Trace::IAnalyzer::FEventData& EventData);
	virtual bool OnEvent(uint16 RouteId, EStyle Style, const FOnEventContext& Context) override;

	FChaosVDTraceAnalysisComplete& OnAnalysisComplete()
	{
		return ChaosVDTraceAnalysisCompleteDelegate;
	}

private:

	enum : uint16
	{
		RouteId_ChaosVDSolverStepStart,
		RouteId_ChaosVDSolverStepEnd,
		RouteId_ChaosVDSolverFrameStart,
		RouteId_ChaosVDSolverFrameEnd,
		RouteId_ChaosVDBinaryDataStart,
		RouteId_ChaosVDBinaryDataContent,
		RouteId_ChaosVDBinaryDataEnd,
		RouteId_ChaosVDSolverSimulationSpace,
		RouteId_BeginFrame,
		RouteId_EndFrame,
		RouteId_ChaosVDParticleDestroyed,
		RouteId_ChaosVDNetworkTickOffset,
		RouteId_ChaosVDRolledBackDataID,
		RouteId_ChaosVDUsesAutoRTFM,
		RouteId_ChaosVDMiscHeaderData,
	};

	TraceServices::IAnalysisSession& Session;

	TSharedRef<FChaosVDTraceProvider> ChaosVDTraceProvider;
	
	FChaosVDTraceAnalysisComplete ChaosVDTraceAnalysisCompleteDelegate;
};

#ifndef CVD_READ_TRACE_VECTOR
	#define CVD_READ_TRACE_VECTOR(Vector, VectorName, ValueType, EventData) \
		Vector.X = EventData.GetValue<ValueType>(CVD_STRINGIZE(VectorName##X)); \
		Vector.Y = EventData.GetValue<ValueType>(CVD_STRINGIZE(VectorName##Y)); \
		Vector.Z = EventData.GetValue<ValueType>(CVD_STRINGIZE(VectorName##Z));
#endif

#ifndef CVD_READ_TRACE_QUAT
	#define CVD_READ_TRACE_QUAT(Rotator, RotatorName, ValueType, EventData) \
		Rotator.X = EventData.GetValue<ValueType>(CVD_STRINGIZE(RotatorName##X)); \
		Rotator.Y = EventData.GetValue<ValueType>(CVD_STRINGIZE(RotatorName##Y)); \
		Rotator.Z = EventData.GetValue<ValueType>(CVD_STRINGIZE(RotatorName##Z)); \
		Rotator.W = EventData.GetValue<ValueType>(CVD_STRINGIZE(RotatorName##W));
#endif

#ifndef CVD_READ_TRACE_TRANSFORM
	#define CVD_READ_TRACE_TRANSFORM(Transform, ValueType, EventData) \
		{ \
			FVector Location; \
			CVD_READ_TRACE_VECTOR(Location, Position, float, EventData); \
			FVector Scale; \
			CVD_READ_TRACE_VECTOR(Location, Position, float, EventData); \
			FQuat Rotation; \
			CVD_READ_TRACE_QUAT(Rotation, Rotation, float, EventData); \
			Transform.SetLocation(Location); \
			Transform.SetScale3D(Scale); \
			Transform.SetRotation(Rotation); \
		}
#endif
