// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/ChaosVDTraceModule.h"

#include "ChaosVDModule.h"
#include "Trace/ChaosVDTraceManager.h"
#include "Trace/ChaosVDTraceAnalyzer.h"
#include "Trace/ChaosVDTraceProvider.h"
#include "TraceServices/Model/AnalysisSession.h"

FName FChaosVDTraceModule::ModuleName("ChaosVDTrace");

void FChaosVDTraceModule::GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo)
{
	OutModuleInfo.Name = ModuleName;
	OutModuleInfo.DisplayName = TEXT("ChaosVisualDebugger");
}

void FChaosVDTraceModule::OnAnalysisBegin(TraceServices::IAnalysisSession& InSession)
{
	const TSharedRef<FChaosVDTraceProvider> Provider = MakeShared<FChaosVDTraceProvider>(InSession);

	// TODO: For multi-file support ideally we should be able to set this in the same call when we start the session but there is no API in trace for that.
	// We can't do it immediately after we start analysis either because analysis starts in a separate thread and when we get a chance to lock it
	// it might be too late (specially when we open multiple trace files in the same frame).

	//This workaround works because this is called in the GT
	ensure(IsInGameThread());
	TWeakPtr<FChaosVDRecording>& PendingRecording = FChaosVDTraceManagerThreadContext::Get().PendingExternalRecordingWeakPtr;
	if (TSharedPtr<FChaosVDRecording> RecordingInstance = PendingRecording.Pin())
	{
		Provider->SetExternalRecordingInstanceForSession(RecordingInstance.ToSharedRef());
		PendingRecording.Reset();
	}

	InSession.AddProvider(FChaosVDTraceProvider::ProviderName, Provider, Provider);
	InSession.AddAnalyzer(new FChaosVDTraceAnalyzer(InSession, Provider));
}

void FChaosVDTraceModule::GetLoggers(TArray<const TCHAR*>& OutLoggers)
{
	OutLoggers.Add(TEXT("ChaosVD"));
}

void FChaosVDTraceModule::GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory)
{
	IModule::GenerateReports(Session, CmdLine, OutputDirectory);
}
