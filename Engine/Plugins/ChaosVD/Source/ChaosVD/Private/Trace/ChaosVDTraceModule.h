// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "TraceServices/ModuleService.h"

/** Modular feature used by ChaosVD to interface with the Trace Services system */
class FChaosVDTraceModule : public TraceServices::IModule
{
public:
	 FChaosVDTraceModule()
		{
		}

	static FName ModuleName;

	// TraceServices::IModule interface
	virtual void GetModuleInfo(TraceServices::FModuleInfo& OutModuleInfo) override;
	virtual void OnAnalysisBegin(TraceServices::IAnalysisSession& InSession) override;
	virtual void GetLoggers(TArray<const TCHAR *>& OutLoggers) override;
	virtual void GenerateReports(const TraceServices::IAnalysisSession& Session, const TCHAR* CmdLine, const TCHAR* OutputDirectory) override;
	virtual const TCHAR* GetCommandLineArgument() override { return TEXT("chaosvd"); }

};
