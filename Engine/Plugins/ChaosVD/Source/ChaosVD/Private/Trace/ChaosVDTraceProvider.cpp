// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/ChaosVDTraceProvider.h"

#include "ChaosVDModule.h"
#include "ChaosVDRecording.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Compression/OodleDataCompressionUtil.h"
#include "Containers/Ticker.h"
#include "DataProcessors/ChaosVDCollisionChannelsInfoDataProcessor.h"
#include "DataProcessors/ChaosVDParticleMetadataProcessor.h"
#include "ExtensionsSystem/ChaosVDExtensionsManager.h"
#include "Application/SlateApplicationBase.h"
#include "HAL/PlatformApplicationMisc.h"
#include "Framework/Application/SlateApplication.h"
#include "Misc/MessageDialog.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SWindow.h"
#include "Widgets/Text/STextBlock.h"
#include "Settings/ChaosVDGeneralSettings.h"
#include "Trace/DataProcessors/ChaosVDCharacterGroundConstraintDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDConstraintDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDJointConstraintDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDMidPhaseDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDSceneQueryDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDSceneQueryVisitDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDSerializedNameEntryDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDTraceImplicitObjectProcessor.h"
#include "Trace/DataProcessors/ChaosVDTraceParticleDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDArchiveHeaderProcessor.h"
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FName FChaosVDTraceProvider::ProviderName("ChaosVDProvider");

FChaosVDTraceProvider::FChaosVDTraceProvider(TraceServices::IAnalysisSession& InSession): Session(InSession)
{
	using namespace Chaos::VisualDebugger;
	NameTable = MakeShared<FChaosVDSerializableNameTable>();

	// Start with a default header data as a fallback
	HeaderData = FChaosVDArchiveHeader::Current();
	
	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		bShouldTrimOutStartEmptyFrames = Settings->bTrimEmptyFrames;
		MaxGameFramesToQueueNum = Settings->MaxGameThreadFramesToQueueNum;
	}
}

void FChaosVDTraceProvider::CreateRecordingInstanceForSession(const FString& InSessionName)
{
	if (bHasRecordingOverride)
	{
		return;
	}

	DeleteRecordingInstanceForSession();

	InternalRecording = MakeShared<FChaosVDRecording>();
	InternalRecording->SessionName = InSessionName;
}

void FChaosVDTraceProvider::SetExternalRecordingInstanceForSession(const TSharedRef<FChaosVDRecording>& InExternalCVDRecording)
{
	bHasRecordingOverride = true;
	if (InternalRecording)
	{
		ensure(InternalRecording->IsEmpty());
	}
	InExternalCVDRecording->AddAttributes(EChaosVDRecordingAttributes::Merged);

	InternalRecording = InExternalCVDRecording;
}

void FChaosVDTraceProvider::DeleteRecordingInstanceForSession()
{
	InternalRecording.Reset();
}

void FChaosVDTraceProvider::StartSolverFrame(const int32 InSolverID, FChaosVDSolverFrameData&& FrameData)
{
	if (!InternalRecording.IsValid())
	{
		return;
	}

	bool bIsInvalidSolverID = InSolverID == INDEX_NONE && !CurrentSolverFramesByID.IsEmpty();

	if (!ensure(!bIsInvalidSolverID))
	{
		return;
	}

	if (FChaosVDSolverFrameData* SolveFrameData = CurrentSolverFramesByID.Find(InSolverID))
	{
		InternalRecording->AddFrameForSolver(InSolverID, MoveTemp(*SolveFrameData));
		CurrentSolverFramesByID[InSolverID] = FrameData;
	}
	else
	{
		CurrentSolverFramesByID.Add(InSolverID, FrameData);
	}
}

void FChaosVDTraceProvider::GetAvailablePendingSolverIDsAtGameFrame(const TSharedRef<FChaosVDGameFrameData>& InProcessedGameFrameData, TArray<int32, TInlineAllocator<16>>& OutSolverIDs)
{
	for (const TPair<int32, FChaosVDSolverFrameData>& FrameDataWithSolverID : CurrentSolverFramesByID)
	{
		if (FrameDataWithSolverID.Value.FrameCycle < InProcessedGameFrameData->FirstCycle)
		{
			OutSolverIDs.Add(FrameDataWithSolverID.Key);
		}
	}
}

FString FChaosVDTraceProvider::GenerateFormattedStringListFromSet(const TSet<FString>& StringsSet) const
{
	FString FormattedString = TEXT("");
	for (const FString& ListEntry : StringsSet)
	{
		FormattedString.Append(TEXT("- "));
		FormattedString.Append(ListEntry);
		FormattedString.Append(TEXT("\n"));
	}

	return MoveTemp(FormattedString);
}

int32 FChaosVDTraceProvider::RemapSolverID(int32 SolverID)
{
	int32 RemappedSolverID = SolverID;

	{
		// Lock the recording until we manage to reserve a unique solver ID replacement
		FWriteScopeLock WriteLock(InternalRecording->RecordingDataLock);

		while (InternalRecording->HasSolverID_AssumesLocked(RemappedSolverID))
		{
			constexpr int32 MaxValue = std::numeric_limits<int32>::max() -1;
			check(RemappedSolverID < MaxValue);
	
			RemappedSolverID = InternalRecording->GetAvailableTrackIDForRemapping();
		}

		InternalRecording->ReserveSolverID_AssumesLocked(RemappedSolverID);
	}

	UE_LOGF(LogChaosVDEditor, Verbose, "[%s] Remapped solver id from [%d] to [%d].", __func__, SolverID, RemappedSolverID);

	RemappedSolversIDs.Emplace(SolverID, RemappedSolverID);

	return RemappedSolverID;
}

int32 FChaosVDTraceProvider::GetRemappedSolverID(int32 SolverID)
{
	int32 RemappedSolverID = SolverID;

	if (int32* SolverIDPtr = RemappedSolversIDs.Find(SolverID))
	{
		RemappedSolverID = *SolverIDPtr;
	}
	else
	{
		UE_LOGF(LogChaosVDEditor, Verbose, "[%s] Failed to get remapped solver id [%d]. Data that references the invalid solver id will be ignored.", __func__, SolverID);
		RemappedSolverID = INDEX_NONE; 
	}

	return RemappedSolverID;
}

void FChaosVDTraceProvider::AddParticleMetadata(uint64 MetadaId, const TSharedPtr<FChaosVDParticleMetadata>& InMetadata)
{
	if (!SerializedParticleMetadata.Contains(MetadaId))
	{
		SerializedParticleMetadata.Add(MetadaId, InMetadata);
	}
}

TSharedPtr<FChaosVDParticleMetadata> FChaosVDTraceProvider::GetParticleMetadata(uint64 MetadataId)
{
	TSharedPtr<FChaosVDParticleMetadata>* MetadataInstance = SerializedParticleMetadata.Find(MetadataId);

	return MetadataInstance ? *MetadataInstance : nullptr;
}

void FChaosVDTraceProvider::CreateGeometryArchiveContext()
{
	GeometryArchiveContext = MakeShared<Chaos::FChaosArchiveContext>();
}

void FChaosVDTraceProvider::CommitProcessedGameFramesToRecording()
{
	TArray<int32, TInlineAllocator<16>> SolverIDs;

	// The Game Frame events are not generated by CVD trace code, and we don't have control over them.
	// we use them as general timestamps.
	// These are generated even when no solvers are available (specially in PIE), so we need to discard any game frame that will not resolve to a solver frame
	// Physics Frames and GT frames lifetimes might not align with async physics enabled, so to make sure we have all the solver data for that time range, we queue a handful of game frames before processing them.

	if (CurrentGameFrameQueueSize > MaxGameFramesToQueueNum)
	{
		TSharedPtr<FChaosVDGameFrameData> ProcessedGameFrameData;
		DeQueueGameFrameForProcessing(ProcessedGameFrameData);

		if (StartLastCommitedFrameTimeSeconds == 0.0)
		{
			StartLastCommitedFrameTimeSeconds = FPlatformTime::Seconds();
		}

		if (ProcessedGameFrameData.IsValid())
		{
			InternalRecording->GetAvailableSolverIDsAtGameFrame(*ProcessedGameFrameData, SolverIDs);

			// Is it possible that the solver data is not commited to the recording yet as it is being processed.
			// Usually this happens on recordings with Async Physics
			if (SolverIDs.IsEmpty())
			{
				GetAvailablePendingSolverIDsAtGameFrame(ProcessedGameFrameData.ToSharedRef(), SolverIDs);
			}
	
			const bool bHasAnySolverData = !SolverIDs.IsEmpty();
			const bool bHasAnyGameFrame = InternalRecording->GetAvailableGameFramesNumber() > 0;
			
			bool bHasRelevantCVDData = true;
			if (bShouldTrimOutStartEmptyFrames)
			{
				bHasRelevantCVDData = bHasAnyGameFrame ? true : bHasAnySolverData;
			}

			if (bHasRelevantCVDData)
			{
				InternalRecording->AddGameFrameData(*ProcessedGameFrameData);	
			}
		}
	}

	SolverIDs.Reset();
}

void FChaosVDTraceProvider::StartGameFrame(const TSharedPtr<FChaosVDGameFrameData>& InFrameData)
{
	if (!InternalRecording.IsValid() || bHasRecordingOverride)
	{
		return;
	}

	CommitProcessedGameFramesToRecording();

	EnqueueGameFrameForProcessing(InFrameData);
}

FChaosVDSolverFrameData* FChaosVDTraceProvider::GetCurrentSolverFrame(const int32 InSolverID)
{
	// If we didn't remap any ID yet, InSolverID might be INDEX_NONE. This is expected as we can have data that was started being recorded in the
	// middle of a frame and therefore the solver hasn't been open in CVD yet.
	bool bIsInvalidSolverID = InSolverID == INDEX_NONE && (!CurrentSolverFramesByID.IsEmpty() && RemappedGameThreadTrackID != INDEX_NONE);

	if (bIsInvalidSolverID)
	{
		UE_LOGF(LogChaosVDEditor, Verbose, "[%s] was called with an invalid solver ID. Data that references the invalid solver id will be ignored.", __func__);
		return nullptr;
	}

	if (FChaosVDSolverFrameData* SolveFrameData = CurrentSolverFramesByID.Find(InSolverID))
	{
		return SolveFrameData;
	}

	return nullptr;
}

TWeakPtr<FChaosVDGameFrameData> FChaosVDTraceProvider::GetCurrentGameFrame()
{
	if (bHasRecordingOverride)
	{
		if (FChaosVDSolverFrameData* FrameData = GetCurrentSolverFrame(GetCurrentGameThreadTrackID()))
		{
			TSharedPtr<FChaosVDGameFrameDataWrapper> GTFrameDataWrapper = FrameData->GetCustomData().GetData<FChaosVDGameFrameDataWrapper>();
			if (ensure(GTFrameDataWrapper))
			{
				return GTFrameDataWrapper->FrameData;
			}
		}
	}
	else if (TSharedPtr<FChaosVDGameFrameData> GTFrameData = CurrentGameFrame.Pin())
	{
		return GTFrameData;
	}

	return nullptr;
}

FChaosVDTraceProvider::FBinaryDataContainer& FChaosVDTraceProvider::FindOrAddUnprocessedData(const int32 DataID)
{
	if (const TSharedPtr<FBinaryDataContainer>* UnprocessedData = UnprocessedDataByID.Find(DataID))
	{
		check(UnprocessedData->IsValid());
		return *UnprocessedData->Get();
	}
	else
	{
		const TSharedPtr<FBinaryDataContainer> DataContainer = MakeShared<FBinaryDataContainer>(DataID);
		UnprocessedDataByID.Add(DataID, DataContainer);
		return *DataContainer.Get();
	}
}

void FChaosVDTraceProvider::RemoveUnprocessedData(const int32 DataID)
{
	// The removal call should always come before the data is processed
	ensure(UnprocessedDataByID.Remove(DataID) != 0);
}

bool FChaosVDTraceProvider::ProcessBinaryData(const int32 DataID)
{
	RegisterDefaultDataProcessorsIfNeeded();

	if (const TSharedPtr<FBinaryDataContainer>* UnprocessedDataPtr = UnprocessedDataByID.Find(DataID))
	{
		const TSharedPtr<FBinaryDataContainer> UnprocessedData = *UnprocessedDataPtr;
		if (UnprocessedData.IsValid())
		{
			UnprocessedData->bIsReady = true;

			const TArray<uint8>* RawData = nullptr;
			TArray<uint8> UncompressedData;
			if (UnprocessedData->bIsCompressed)
			{
				UncompressedData.Reserve(UnprocessedData->UncompressedSize);
				FOodleCompressedArray::DecompressToTArray(UncompressedData, UnprocessedData->RawData);
				RawData = &UncompressedData;
			}
			else
			{
				RawData = &UnprocessedData->RawData;
			}

			if (TSharedPtr<FChaosVDDataProcessorBase>* DataProcessorPtrPtr = RegisteredDataProcessors.Find(UnprocessedData->TypeName))
			{
				if (TSharedPtr<FChaosVDDataProcessorBase> DataProcessorPtr = *DataProcessorPtrPtr)
				{
					if (DataProcessorPtr->ShouldAlwaysSkip())
					{
						UE_LOGF(LogChaosVDEditor, Log,
							"[%ls] Silently skipping [%ls] — processor opted out via ShouldAlwaysSkip().",
							ANSI_TO_TCHAR(__FUNCTION__), *UnprocessedData->TypeName);
						UnprocessedDataByID.Remove(DataID);
						DataProcessedSoFarCounter++;
						return true;
					}

					// If this processor's format is not backwards compatible by design, gate on Safe Loading Mode.
					if (!DataProcessorPtr->IsBackwardsCompatible())
					{
						const UChaosVDGeneralSettings* GeneralSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>();
						if (GeneralSettings && GeneralSettings->bSafeLoadingMode)
						{
							// Treat deliberately-skipped data as a clean success so the rest of the load continues normally.
							TypesSkippedInSafeMode.Add(UnprocessedData->TypeName);
							UE_LOGF(LogChaosVDEditor, Log,
								"[%ls] Skipping non-backwards-compatible data type [%ls]  - Safe Loading Mode is enabled. See CVD General Settings to change this.",
								ANSI_TO_TCHAR(__FUNCTION__), *UnprocessedData->TypeName);
							UnprocessedDataByID.Remove(DataID);
							DataProcessedSoFarCounter++;
							return true;
						}
						else
						{
							// Record that we're about to process potentially unsafe data so we can warn after loading.
							UnsafeTypesProcessed.Add(UnprocessedData->TypeName);
							UE_LOGF(LogChaosVDEditor, Log,
								"[%ls] Processing non-backwards-compatible data type [%ls] with Safe Loading Mode disabled. Enable Safe Loading Mode in CVD General Settings if CVD crashes unexpectedly.",
								ANSI_TO_TCHAR(__FUNCTION__), *UnprocessedData->TypeName);
						}
					}

					if (ensure(DataProcessorPtr->ProcessRawData(*RawData)))
					{
						UnprocessedDataByID.Remove(DataID);
						DataProcessedSoFarCounter++;
						return true;
					}
					else
					{
						UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] Failed to serialize Binary Data with ID [%d] | Type [%ls]", ANSI_TO_TCHAR(__FUNCTION__), DataID, *UnprocessedData->TypeName);
						TypesFailedToSerialize.Add(UnprocessedData->TypeName);
						DataProcessedSoFarCounter++;
					}
				}
			}
			else
			{
				MissingDataProcessors.Add(UnprocessedData->TypeName);	
			}
		}

		UnprocessedDataByID.Remove(DataID);
	}

	return false;
}

TSharedPtr<FChaosVDRecording> FChaosVDTraceProvider::GetRecordingForSession() const
{
	return InternalRecording;
}

void FChaosVDTraceProvider::RegisterDataProcessor(TSharedPtr<FChaosVDDataProcessorBase> InDataProcessor)
{
	RegisteredDataProcessors.Add(InDataProcessor->GetCompatibleTypeName(), InDataProcessor);
}

void FChaosVDTraceProvider::HandleAnalysisComplete()
{
	if (!MissingDataProcessors.IsEmpty())
	{
		FString MissingProcessorNameList = GenerateFormattedStringListFromSet(MissingDataProcessors);
	
		FText MissingDataProcessorsMessage = FText::FormatOrdered(LOCTEXT("MissingDataProcessorMessage", "This recording was made with CVD extensions that are not supported in this version. \n\nAs a result, the following data types could not be read and will be ignored : \n\n {0}"), FText::FromString(MissingProcessorNameList));

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([MissingDataProcessorsMessage](float DeltaTime)
		{
			FMessageDialog::Open(EAppMsgCategory::Warning, EAppMsgType::Ok, MissingDataProcessorsMessage, LOCTEXT("MissingDataProcessorMessageTitle", "Partially unsupported CVD Recording"));
			return false;
		}));
	}

	if (!TypesFailedToSerialize.IsEmpty())
	{
		FString FailedTypeList = GenerateFormattedStringListFromSet(TypesFailedToSerialize);

		FText MissingDataProcessorsMessage = FText::FormatOrdered(LOCTEXT("FailedSerializationMessage", "The following data types were part of the recording, but they couldn't be read : \n\n {0} \n\n Visualization related to that data will not be shown."), FText::FromString(FailedTypeList));

		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([MissingDataProcessorsMessage](float DeltaTime)
		{
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, MissingDataProcessorsMessage, LOCTEXT("FailedSerializationMessageTitle", "Failed to read data"));
			return false;
		}));
	}

	ShowSafeModeSkippedDialog();
	ShowSerializeBinOnlyStrippedDialog();
	ShowUnsafeDataDialog();
	ShowNativeSerializationDialog();

	UnprocessedDataByID.Reset();

	UE_LOGF(LogChaosVDEditor, Log, "Trace Analysis complete for session [%ls] | Calculating data loaded stats...", Session.GetName());

	static const FNumberFormattingOptions SizeFormattingOptions = FNumberFormattingOptions().SetMinimumFractionalDigits(2).SetMaximumFractionalDigits(2);

	uint64 TotalBytes = 0;
	for (const TPair<FStringView, TSharedPtr<FChaosVDDataProcessorBase>>& DataProcessor : RegisteredDataProcessors)
	{
		if (DataProcessor.Value)
		{
			uint64 ProcessedBytes = DataProcessor.Value->GetProcessedBytes();
			TotalBytes += ProcessedBytes;
			UE_LOGF(LogChaosVDEditor, Log, "Data loaded for type [%ls]  => [%ls] ", DataProcessor.Key.IsEmpty() ? TEXT("Invalid") : DataProcessor.Key.GetData(), *FText::AsMemory(ProcessedBytes, &SizeFormattingOptions,nullptr, EMemoryUnitStandard::IEC).ToString());
		}
	}

	if (TSharedPtr<FChaosVDRecording> Recording = GetRecordingForSession())
	{
		double TotalTimeProcessingFrames = FPlatformTime::Seconds() - StartLastCommitedFrameTimeSeconds;

		int32 NumOfGameFramesProcessed = Recording->GetAvailableGameFramesNumber();
		double AvgTimePerFrameSeconds = TotalTimeProcessingFrames / NumOfGameFramesProcessed;
		
		UE_LOGF(LogChaosVDEditor, Log, " [%d] Game frames Processed at [%f] ms per frame on average", NumOfGameFramesProcessed, AvgTimePerFrameSeconds * 1000.0);
	}

	UE_LOGF(LogChaosVDEditor, Log, "Total size of loaded data => [%ls]", *FText::AsMemory(TotalBytes, &SizeFormattingOptions,nullptr, EMemoryUnitStandard::IEC).ToString());
}

void FChaosVDTraceProvider::ShowSafeModeSkippedDialog()
{
	if (TypesSkippedInSafeMode.IsEmpty())
	{
		return;
	}

	FChaosVDSettingsManager* SettingsManager = FChaosVDSettingsManager::TryGet();
	UChaosVDGeneralSettings* Settings = SettingsManager ? SettingsManager->GetSettingsObject<UChaosVDGeneralSettings>() : nullptr;
	if (Settings && !Settings->bShowSafeModeSkippedWarning)
	{
		return;
	}

	using namespace Chaos::VisualDebugger;

	const FChaosVDArchiveHeader CurrentHeader = FChaosVDArchiveHeader::Current();
	const bool bVersionsMatch = HeaderData.EngineVersion.ExactMatch(CurrentHeader.EngineVersion);
	const FString RecordedVersionString = HeaderData.EngineVersion.ToString();
	const FString CurrentVersionString = CurrentHeader.EngineVersion.ToString();

	const FText VersionNote = bVersionsMatch
		? LOCTEXT("SafeModeVersionMatch", "Engine versions match exactly, but local source changes may still affect the binary layout.")
		: FText::FormatOrdered(
			LOCTEXT("SafeModeVersionMismatch", "Recorded with engine version {0}; current version is {1}. Binary format may have changed."),
			FText::FromString(RecordedVersionString),
			FText::FromString(CurrentVersionString));

	FText Message = FText::FormatOrdered(
		LOCTEXT("SafeModeSkippedMessage",
			"Some data was skipped because Safe Loading Mode is enabled and the following types are not guaranteed to be backwards compatible:\n\n"
			"{0}\n\n"
			"{1}\n\n"
			"To load this data anyway, disable Safe Loading Mode in CVD General Settings (Project > Plugins > Chaos Visual Debugger > General).\n"
			"Note: doing so may cause a crash if the binary format has changed since the recording was made."),
		FText::FromString(GenerateFormattedStringListFromSet(TypesSkippedInSafeMode)),
		VersionNote);

	TWeakObjectPtr<UChaosVDGeneralSettings> WeakSettings(Settings);

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([Message, WeakSettings](float)
	{
		TSharedPtr<bool> bSuppressInFuture = MakeShared<bool>(false);

		TSharedRef<SWindow> DialogWindow = SNew(SWindow)
			.Title(LOCTEXT("SafeModeSkippedTitle", "Data Skipped - Safe Loading Mode Active"))
			.SizingRule(ESizingRule::Autosized)
			.SupportsMaximize(false)
			.SupportsMinimize(false);

		DialogWindow->SetContent(
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(16.f, 16.f, 16.f, 8.f)
			[
				SNew(STextBlock).Text(Message).WrapTextAt(500.f)
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(16.f, 4.f)
			[
				SNew(SCheckBox)
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged_Lambda([bSuppressInFuture](ECheckBoxState NewState)
				{
					*bSuppressInFuture = (NewState == ECheckBoxState::Unchecked);
				})
				[
					SNew(STextBlock).Text(LOCTEXT("SafeModeShowWarning", "Show this warning in future"))
				]
			]
			+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Right).Padding(16.f, 8.f, 16.f, 16.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth().Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("SafeModeCopyTooltip", "Copy the text in this message to the clipboard (CTRL+C)"))
					.ContentPadding(2.f)
					.OnClicked_Lambda([Message]() -> FReply
					{
						FPlatformApplicationMisc::ClipboardCopy(*Message.ToString());
						return FReply::Handled();
					})
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("SafeModeOK", "OK"))
					.OnClicked_Lambda([DialogWindowWeak = TWeakPtr<SWindow>(DialogWindow), bSuppressInFuture, WeakSettings]() -> FReply
					{
						if (*bSuppressInFuture)
						{
							if (UChaosVDGeneralSettings* CVDSettings = WeakSettings.Get())
							{
								CVDSettings->bShowSafeModeSkippedWarning = false;
								CVDSettings->SaveConfig();
							}
						}
						if (TSharedPtr<SWindow> DialogWindowPinned = DialogWindowWeak.Pin())
						{
							DialogWindowPinned->RequestDestroyWindow();
						}
						return FReply::Handled();
					})
				]
			]
		);

		FSlateApplication::Get().AddModalWindow(DialogWindow, FSlateApplication::Get().GetActiveTopLevelWindow());
		return false;
	}));
}

void FChaosVDTraceProvider::ShowUnsafeDataDialog()
{
	if (UnsafeTypesProcessed.IsEmpty())
	{
		return;
	}

	FChaosVDSettingsManager* SettingsManager = FChaosVDSettingsManager::TryGet();
	UChaosVDGeneralSettings* Settings = SettingsManager ? SettingsManager->GetSettingsObject<UChaosVDGeneralSettings>() : nullptr;
	if (Settings && !Settings->bShowUnsafeDataWarning)
	{
		return;
	}

	using namespace Chaos::VisualDebugger;

	const FChaosVDArchiveHeader CurrentHeader = FChaosVDArchiveHeader::Current();
	const bool bVersionsMatch = HeaderData.EngineVersion.ExactMatch(CurrentHeader.EngineVersion);
	const FString RecordedVersionString = HeaderData.EngineVersion.ToString();
	const FString CurrentVersionString = CurrentHeader.EngineVersion.ToString();

	const FText VersionNote = bVersionsMatch
		? LOCTEXT("UnsafeVersionMatch", "Engine versions match exactly. The risk of a crash is lower but not zero - local source changes may still affect the binary layout.")
		: FText::FormatOrdered(
			LOCTEXT("UnsafeVersionMismatch", "Recorded with engine version {0}; current version is {1}. The risk of a crash due to binary incompatibility is elevated."),
			FText::FromString(RecordedVersionString),
			FText::FromString(CurrentVersionString));

	FText Message = FText::FormatOrdered(
		LOCTEXT("UnsafeModeProcessedMessage",
			"The following data types are not guaranteed to be backwards compatible and were loaded with Safe Loading Mode disabled:\n\n"
			"{0}\n\n"
			"{1}\n\n"
			"If CVD crashes unexpectedly while viewing this recording, enable Safe Loading Mode in CVD General Settings."),
		FText::FromString(GenerateFormattedStringListFromSet(UnsafeTypesProcessed)),
		VersionNote);

	TWeakObjectPtr<UChaosVDGeneralSettings> WeakSettings(Settings);

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([Message, WeakSettings](float)
	{
		TSharedPtr<bool> bSuppressInFuture = MakeShared<bool>(false);

		TSharedRef<SWindow> DialogWindow = SNew(SWindow)
			.Title(LOCTEXT("UnsafeModeProcessedTitle", "Non-Backwards-Compatible Data Loaded"))
			.SizingRule(ESizingRule::Autosized)
			.SupportsMaximize(false)
			.SupportsMinimize(false);

		DialogWindow->SetContent(
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(16.f, 16.f, 16.f, 8.f)
			[
				SNew(STextBlock).Text(Message).WrapTextAt(500.f)
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(16.f, 4.f)
			[
				SNew(SCheckBox)
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged_Lambda([bSuppressInFuture](ECheckBoxState NewState)
				{
					*bSuppressInFuture = (NewState == ECheckBoxState::Unchecked);
				})
				[
					SNew(STextBlock).Text(LOCTEXT("UnsafeModeShowWarning", "Show this warning in future"))
				]
			]
			+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Right).Padding(16.f, 8.f, 16.f, 16.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth().Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("UnsafeModeCopyTooltip", "Copy the text in this message to the clipboard (CTRL+C)"))
					.ContentPadding(2.f)
					.OnClicked_Lambda([Message]() -> FReply
					{
						FPlatformApplicationMisc::ClipboardCopy(*Message.ToString());
						return FReply::Handled();
					})
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("UnsafeModeOK", "OK"))
					.OnClicked_Lambda([DialogWindowWeak = TWeakPtr<SWindow>(DialogWindow), bSuppressInFuture, WeakSettings]() -> FReply
					{
						if (*bSuppressInFuture)
						{
							if (UChaosVDGeneralSettings* CVDSettings = WeakSettings.Get())
							{
								CVDSettings->bShowUnsafeDataWarning = false;
								CVDSettings->SaveConfig();
							}
						}
						if (TSharedPtr<SWindow> DialogWindowPinned = DialogWindowWeak.Pin())
						{
							DialogWindowPinned->RequestDestroyWindow();
						}
						return FReply::Handled();
					})
				]
			]
		);

		FSlateApplication::Get().AddModalWindow(DialogWindow, FSlateApplication::Get().GetActiveTopLevelWindow());
		return false;
	}));
}

void FChaosVDTraceProvider::ShowNativeSerializationDialog()
{
	// Only show in LoadAll mode — SerializeBinOnly strips native entries silently (notified via
	// ShowSerializeBinOnlyStrippedDialog); SkipAll skips the processor entirely.
	FChaosVDSettingsManager* SettingsManager = FChaosVDSettingsManager::TryGet();
	UChaosVDGeneralSettings* Settings = SettingsManager ? SettingsManager->GetSettingsObject<UChaosVDGeneralSettings>() : nullptr;
	if (!Settings || Settings->ParticleExtraDataLoadingMode != EChaosVDParticleExtraDataLoadingMode::LoadAll
		|| !Settings->bShowNativeSerializationWarning)
	{
		return;
	}

	TMap<FName, FName> AllNativeTypes; // type path → channel id
	for (const TPair<FStringView, TSharedPtr<FChaosVDDataProcessorBase>>& DataProcessor : RegisteredDataProcessors)
	{
		if (DataProcessor.Value)
		{
			DataProcessor.Value->GetPostLoadNativeTypesWithChannels(AllNativeTypes);
		}
	}

	if (AllNativeTypes.IsEmpty())
	{
		return;
	}

	TSet<FString> FormattedTypes;
	for (const TPair<FName, FName>& Pair : AllNativeTypes)
	{
		if (!Pair.Value.IsNone())
		{
			FormattedTypes.Add(FText::FormatOrdered(
				LOCTEXT("NativeTypeWithChannel", "{0} (via {1} channel)"),
				FText::FromName(Pair.Key),
				FText::FromName(Pair.Value)).ToString());
		}
		else
		{
			FormattedTypes.Add(Pair.Key.ToString());
		}
	}

	FText Message = FText::FormatOrdered(
		LOCTEXT("NativeSerializationWarning",
			"This recording contains particle extra data using native struct serialization "
			"for the following types:\n\n{0}\n\n"
			"Native serialization reads bytes at fixed offsets and displays exact recorded values "
			"-- but only when the struct layout matches the one at record time. If the layout has "
			"changed, the result may be a crash or silently corrupt values with no warning.\n\n"
			"Reflection-based entries (also present) are crash-safe but may show inaccurate values "
			"if struct properties were added, removed, or renamed since recording.\n\n"
			"Choose 'Reflection-Based Only' to strip native entries and avoid crash risk. "
			"Choose 'Load All' only if the recording and viewer binaries are known to match."),
		FText::FromString(GenerateFormattedStringListFromSet(FormattedTypes)));

	TWeakObjectPtr<UChaosVDGeneralSettings> WeakSettings(Settings);

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([Message, WeakSettings](float) -> bool
	{
		TSharedPtr<bool> bSuppressInFuture = MakeShared<bool>(false);

		TSharedRef<SWindow> DialogWindow = SNew(SWindow)
			.Title(LOCTEXT("NativeSerializationTitle", "Native-Serialized Extra Data Detected"))
			.SizingRule(ESizingRule::Autosized)
			.SupportsMaximize(false)
			.SupportsMinimize(false);

		DialogWindow->SetContent(
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(16.f, 16.f, 16.f, 8.f)
			[
				SNew(STextBlock).Text(Message).WrapTextAt(500.f)
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(16.f, 4.f)
			[
				SNew(SCheckBox)
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged_Lambda([bSuppressInFuture](ECheckBoxState State)
				{
					*bSuppressInFuture = (State == ECheckBoxState::Unchecked);
				})
				[
					SNew(STextBlock).Text(LOCTEXT("NativeSerializationShowWarning", "Show this warning in future"))
				]
			]
			+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Right).Padding(16.f, 8.f, 16.f, 16.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth().Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(SButton)
					.Text(LOCTEXT("NativeSerializationShowAll", "Load All"))
					.ToolTipText(LOCTEXT("NativeSerializationShowAllTip",
						"Display all extra data including native-serialized entries (may crash if struct layout changed)."))
					.OnClicked_Lambda([DialogWindowWeak = TWeakPtr<SWindow>(DialogWindow), bSuppressInFuture, WeakSettings]() -> FReply
					{
						if (*bSuppressInFuture)
						{
							if (UChaosVDGeneralSettings* CVDSettings = WeakSettings.Get())
							{
								CVDSettings->bShowNativeSerializationWarning = false;
								CVDSettings->SaveConfig();
							}
						}
						if (TSharedPtr<SWindow> DialogWindowPinned = DialogWindowWeak.Pin())
						{
							DialogWindowPinned->RequestDestroyWindow();
						}
						return FReply::Handled();
					})
				]
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("NativeSerializationSkip", "Reflection-Based Only"))
					.ToolTipText(LOCTEXT("NativeSerializationSkipTip",
						"Strip natively-serialized entries; only reflection-based (property-tagged) data is displayed (safe)."))
					.OnClicked_Lambda([DialogWindowWeak = TWeakPtr<SWindow>(DialogWindow), bSuppressInFuture, WeakSettings]() -> FReply
					{
						if (UChaosVDGeneralSettings* CVDSettings = WeakSettings.Get())
						{
							CVDSettings->ParticleExtraDataLoadingMode = EChaosVDParticleExtraDataLoadingMode::SerializeBinOnly;
							if (*bSuppressInFuture)
							{
								CVDSettings->bShowNativeSerializationWarning = false;
							}
							CVDSettings->SaveConfig();
						}
						if (TSharedPtr<SWindow> DialogWindowPinned = DialogWindowWeak.Pin())
						{
							DialogWindowPinned->RequestDestroyWindow();
						}
						return FReply::Handled();
					})
				]
			]
		);

		FSlateApplication::Get().AddModalWindow(DialogWindow, FSlateApplication::Get().GetActiveTopLevelWindow());
		return false;
	}));
}

void FChaosVDTraceProvider::ShowSerializeBinOnlyStrippedDialog()
{
	FChaosVDSettingsManager* SettingsManager = FChaosVDSettingsManager::TryGet();
	UChaosVDGeneralSettings* Settings = SettingsManager ? SettingsManager->GetSettingsObject<UChaosVDGeneralSettings>() : nullptr;
	if (!Settings || Settings->ParticleExtraDataLoadingMode != EChaosVDParticleExtraDataLoadingMode::SerializeBinOnly
		|| !Settings->bShowSafeModeSkippedWarning)
	{
		return;
	}

	TMap<FName, FName> AllNativeTypes;
	for (const TPair<FStringView, TSharedPtr<FChaosVDDataProcessorBase>>& DataProcessor : RegisteredDataProcessors)
	{
		if (DataProcessor.Value)
		{
			DataProcessor.Value->GetPostLoadNativeTypesWithChannels(AllNativeTypes);
		}
	}

	if (AllNativeTypes.IsEmpty())
	{
		return;
	}

	TSet<FString> FormattedTypes;
	for (const TPair<FName, FName>& Pair : AllNativeTypes)
	{
		if (!Pair.Value.IsNone())
		{
			FormattedTypes.Add(FText::FormatOrdered(
				LOCTEXT("NativeTypeWithChannel", "{0} (via {1} channel)"),
				FText::FromName(Pair.Key),
				FText::FromName(Pair.Value)).ToString());
		}
		else
		{
			FormattedTypes.Add(Pair.Key.ToString());
		}
	}

	FText Message = FText::FormatOrdered(
		LOCTEXT("SerializeBinOnlyStrippedMessage",
			"The following particle extra data types were recorded with native struct serialization "
			"and have been stripped because Particle Extra Data Loading Mode is set to "
			"Reflection-Based Only:\n\n{0}\n\n"
			"These entries are not shown. Native serialization reads bytes at fixed offsets -- "
			"if the struct layout has changed since recording, loading them may crash or display "
			"corrupt values.\n\n"
			"To load them anyway, set Particle Extra Data Loading Mode to 'Load All' in CVD "
			"General Settings. Only do this when the recording and viewer binaries are known to match."),
		FText::FromString(GenerateFormattedStringListFromSet(FormattedTypes)));

	TWeakObjectPtr<UChaosVDGeneralSettings> WeakSettings(Settings);

	FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([Message, WeakSettings](float) -> bool
	{
		TSharedPtr<bool> bSuppressInFuture = MakeShared<bool>(false);

		TSharedRef<SWindow> DialogWindow = SNew(SWindow)
			.Title(LOCTEXT("SerializeBinOnlyStrippedTitle", "Native-Serialized Extra Data Stripped"))
			.SizingRule(ESizingRule::Autosized)
			.SupportsMaximize(false)
			.SupportsMinimize(false);

		DialogWindow->SetContent(
			SNew(SVerticalBox)
			+ SVerticalBox::Slot().AutoHeight().Padding(16.f, 16.f, 16.f, 8.f)
			[
				SNew(STextBlock).Text(Message).WrapTextAt(500.f)
			]
			+ SVerticalBox::Slot().AutoHeight().Padding(16.f, 4.f)
			[
				SNew(SCheckBox)
				.IsChecked(ECheckBoxState::Checked)
				.OnCheckStateChanged_Lambda([bSuppressInFuture](ECheckBoxState NewState)
				{
					*bSuppressInFuture = (NewState == ECheckBoxState::Unchecked);
				})
				[
					SNew(STextBlock).Text(LOCTEXT("SerializeBinOnlyStrippedShowWarning", "Show this warning in future"))
				]
			]
			+ SVerticalBox::Slot().AutoHeight().HAlign(HAlign_Right).Padding(16.f, 8.f, 16.f, 16.f)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot().AutoWidth().Padding(0.f, 0.f, 8.f, 0.f)
				[
					SNew(SButton)
					.ButtonStyle(FAppStyle::Get(), "SimpleButton")
					.ToolTipText(LOCTEXT("SerializeBinOnlyStrippedCopyTooltip", "Copy the text in this message to the clipboard (CTRL+C)"))
					.ContentPadding(2.f)
					.OnClicked_Lambda([Message]() -> FReply
					{
						FPlatformApplicationMisc::ClipboardCopy(*Message.ToString());
						return FReply::Handled();
					})
					.Content()
					[
						SNew(SImage)
						.Image(FAppStyle::Get().GetBrush("Icons.Clipboard"))
						.ColorAndOpacity(FSlateColor::UseForeground())
					]
				]
				+ SHorizontalBox::Slot().AutoWidth()
				[
					SNew(SButton)
					.Text(LOCTEXT("SerializeBinOnlyStrippedOK", "OK"))
					.OnClicked_Lambda([DialogWindowWeak = TWeakPtr<SWindow>(DialogWindow), bSuppressInFuture, WeakSettings]() -> FReply
					{
						if (*bSuppressInFuture)
						{
							if (UChaosVDGeneralSettings* CVDSettings = WeakSettings.Get())
							{
								CVDSettings->bShowSafeModeSkippedWarning = false;
								CVDSettings->SaveConfig();
							}
						}
						if (TSharedPtr<SWindow> DialogWindowPinned = DialogWindowWeak.Pin())
						{
							DialogWindowPinned->RequestDestroyWindow();
						}
						return FReply::Handled();
					})
				]
			]
		);

		FSlateApplication::Get().AddModalWindow(DialogWindow, FSlateApplication::Get().GetActiveTopLevelWindow());
		return false;
	}));
}

FChaosVDFrameStageData* FChaosVDTraceProvider::GetCurrentSolverStageDataForCurrentFrame(int32 SolverID, EChaosVDSolverStageAccessorFlags Flags)
{
	auto CreateInBetweenSolverStage = [](FChaosVDSolverFrameData& InFrameData)
	{
		// Add an empty step. It will be filled out by the particle (and later on other objects/elements) events
		FChaosVDFrameStageData& SolverStageData = InFrameData.SolverSteps.AddDefaulted_GetRef();
		SolverStageData.StepName = TEXT("Between Stage Data");
		EnumAddFlags(SolverStageData.StageFlags, EChaosVDSolverStageFlags::Open);

		return &SolverStageData;
	};

	if (FChaosVDSolverFrameData* FrameData = GetCurrentSolverFrame(SolverID))
	{
		if (FrameData->SolverSteps.Num() == 0)
		{
			if (EnumHasAnyFlags(Flags, EChaosVDSolverStageAccessorFlags::CreateNewIfEmpty))
			{
				return CreateInBetweenSolverStage(*FrameData);
			}
		}

		FChaosVDFrameStageData& CurrentSolverStage = FrameData->SolverSteps.Last();
		if (EnumHasAnyFlags(CurrentSolverStage.StageFlags, EChaosVDSolverStageFlags::Open))
		{
			return &CurrentSolverStage;
		}

		if (EnumHasAnyFlags(Flags, EChaosVDSolverStageAccessorFlags::CreateNewIfClosed))
		{
			return CreateInBetweenSolverStage(*FrameData);
		}
	}

	return nullptr;
}

void FChaosVDTraceProvider::RegisterDefaultDataProcessorsIfNeeded()
{
	if (bDefaultDataProcessorsRegistered)
	{
		return;
	}
	
	TSharedPtr<FChaosVDTraceImplicitObjectProcessor> ImplicitObjectProcessor = MakeShared<FChaosVDTraceImplicitObjectProcessor>();
	ImplicitObjectProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(ImplicitObjectProcessor);

	TSharedPtr<FChaosVDTraceParticleDataProcessor> ParticleDataProcessor = MakeShared<FChaosVDTraceParticleDataProcessor>();
	ParticleDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(ParticleDataProcessor);

	TSharedPtr<FChaosVDMidPhaseDataProcessor> MidPhaseDataProcessor = MakeShared<FChaosVDMidPhaseDataProcessor>();
	MidPhaseDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(MidPhaseDataProcessor);

	TSharedPtr<FChaosVDConstraintDataProcessor> ConstraintDataProcessor = MakeShared<FChaosVDConstraintDataProcessor>();
	ConstraintDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(ConstraintDataProcessor);

	TSharedPtr<FChaosVDSceneQueryDataProcessor> SceneQueryDataProcessor = MakeShared<FChaosVDSceneQueryDataProcessor>();
	SceneQueryDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(SceneQueryDataProcessor);

	TSharedPtr<FChaosVDSceneQueryVisitDataProcessor> SceneQueryVisitDataProcessor = MakeShared<FChaosVDSceneQueryVisitDataProcessor>();
	SceneQueryVisitDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(SceneQueryVisitDataProcessor);

	TSharedPtr<FChaosVDSerializedNameEntryDataProcessor> NameEntryDataProcessor = MakeShared<FChaosVDSerializedNameEntryDataProcessor>();
	NameEntryDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(NameEntryDataProcessor);
	
	TSharedPtr<FChaosVDJointConstraintDataProcessor> JointConstraintDataProcessor = MakeShared<FChaosVDJointConstraintDataProcessor>();
	JointConstraintDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(JointConstraintDataProcessor);

	TSharedPtr<FChaosVDCharacterGroundConstraintDataProcessor> CharacterGroundConstraintDataProcessor = MakeShared<FChaosVDCharacterGroundConstraintDataProcessor>();
	CharacterGroundConstraintDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(CharacterGroundConstraintDataProcessor);

	TSharedPtr<FChaosVDArchiveHeaderProcessor> ArchiveHeaderDataProcessor = MakeShared<FChaosVDArchiveHeaderProcessor>();
	ArchiveHeaderDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(ArchiveHeaderDataProcessor);
	
	TSharedPtr<FChaosVDCollisionChannelsInfoDataProcessor> CollisionChannelsInfoDataProcessor = MakeShared<FChaosVDCollisionChannelsInfoDataProcessor>();
	CollisionChannelsInfoDataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(CollisionChannelsInfoDataProcessor);

	TSharedPtr<FChaosVDParticleMetadataProcessor> ParticleMetadataProcessor = MakeShared<FChaosVDParticleMetadataProcessor>();
	ParticleMetadataProcessor->SetTraceProvider(AsShared());
	RegisterDataProcessor(ParticleMetadataProcessor);

	FChaosVDExtensionsManager::Get().EnumerateExtensions([this](const TSharedRef<FChaosVDExtension>& Extension)
	{
		Extension->RegisterDataProcessorsInstancesForProvider(StaticCastSharedRef<FChaosVDTraceProvider>(AsShared()));
		return true;
	});

	bDefaultDataProcessorsRegistered = true;
}

void FChaosVDTraceProvider::EnqueueGameFrameForProcessing(const TSharedPtr<FChaosVDGameFrameData>& FrameData)
{
	CurrentGameFrame = FrameData;
	CurrentGameFrameQueue.Enqueue(FrameData);
	CurrentGameFrameQueueSize++;
}

void FChaosVDTraceProvider::DeQueueGameFrameForProcessing(TSharedPtr<FChaosVDGameFrameData>& OutFrameData)
{
	CurrentGameFrameQueue.Dequeue(OutFrameData);
	CurrentGameFrameQueueSize--;
}

#undef LOCTEXT_NAMESPACE
