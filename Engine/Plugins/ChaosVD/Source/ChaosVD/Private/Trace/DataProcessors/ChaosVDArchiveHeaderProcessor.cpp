// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDArchiveHeaderProcessor.h"

#include "ChaosVDModule.h"

namespace Chaos::VisualDebugger::Private
{
	void PopulateInnerVersionsSet(const FChaosVDArchiveHeader& InHeader, TSet<FGuid>& OutInnerVersionsKeys)
	{
		const FCustomVersionArray& CustomVersions = InHeader.CustomVersionContainer.GetAllVersions();
		OutInnerVersionsKeys.Reserve(CustomVersions.Num());

		Algo::Transform(CustomVersions, OutInnerVersionsKeys, [](const FCustomVersion& Version)-> FGuid
		{
			return Version.Key;
		});
	}

	bool IsCompatibleHeader(const FChaosVDArchiveHeader& InHeaderA, const FChaosVDArchiveHeader& InHeaderB)
	{
		// This is a slow operation but we only expect this to be called once when multiple CVD files are loaded
		if (InHeaderA.EngineVersion.ExactMatch(InHeaderB.EngineVersion))
		{
			TSet<FGuid> VersionsASet;
			PopulateInnerVersionsSet(InHeaderA, VersionsASet);
	
			TSet<FGuid> VersionsBSet;
			PopulateInnerVersionsSet(InHeaderB, VersionsBSet);
		
			return VersionsASet.Num() == VersionsBSet.Num() && VersionsASet.Includes(VersionsBSet);
		}

		return false;
	} 
}

FChaosVDArchiveHeaderProcessor::FChaosVDArchiveHeaderProcessor() : FChaosVDDataProcessorBase(Chaos::VisualDebugger::FChaosVDArchiveHeader::WrapperTypeName)
{
}

bool FChaosVDArchiveHeaderProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	using namespace Chaos::VisualDebugger;

	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	FChaosVDArchiveHeader RecordedHeaderData;
	const bool bSuccess = ReadDataFromBuffer(InData, RecordedHeaderData, ProviderSharedPtr.ToSharedRef());

	// This works under the assumption the header is the first thing written and therefore the first thing read
	// If that didn't happen, we need to know to investigate further.

	constexpr int64 DataExpectedToBeProcessedAtThisPoint = 0;

	// Note: This is not a fatal error, unless pretty drastic serialization changes were made and the loaded file is old.
	// CVD can gracefully handle serialization errors (as long the types serializers themselves don't crash and properly error out as expected).

	if (!ensure(ProviderSharedPtr->GetDataProcessedSoFarNum() == DataExpectedToBeProcessedAtThisPoint))
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Processed an archive header while the recording already had data loaded. That initially loaded data used the default header and serialization errors might have occured | This should not happen...", ANSI_TO_TCHAR(__FUNCTION__));
	}

	ProviderSharedPtr->SetHeaderData(RecordedHeaderData);

	return bSuccess;
}
