// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

/**
 * Data processor implementation that is able to deserialize traced Archive headers
 */
class FChaosVDArchiveHeaderProcessor final : public FChaosVDDataProcessorBase
{
public:
	explicit FChaosVDArchiveHeaderProcessor();
	
	virtual bool ProcessRawData(const TArray<uint8>& InData) override;
};