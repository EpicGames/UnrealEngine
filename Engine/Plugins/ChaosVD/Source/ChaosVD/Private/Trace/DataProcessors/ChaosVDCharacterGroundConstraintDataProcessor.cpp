// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDCharacterGroundConstraintDataProcessor.h"

#include "DataWrappers/ChaosVDCharacterGroundConstraintDataWrappers.h"
#include "Trace/ChaosVDTraceProvider.h"


FChaosVDCharacterGroundConstraintDataProcessor::FChaosVDCharacterGroundConstraintDataProcessor()
	: FChaosVDDataProcessorBase(FChaosVDCharacterGroundConstraint::WrapperTypeName)
{
}

bool FChaosVDCharacterGroundConstraintDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	TSharedPtr<FChaosVDCharacterGroundConstraint> Constraint = MakeShared<FChaosVDCharacterGroundConstraint>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *Constraint, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		Constraint->SolverID = ProviderSharedPtr->GetRemappedSolverID(Constraint->SolverID);

		if (FChaosVDSolverFrameData* CurrentFrameData = ProviderSharedPtr->GetCurrentSolverFrame(Constraint->SolverID))
		{
			CurrentFrameData->RecordedCharacterGroundConstraints.Add(Constraint);
		}
	}

	return bSuccess;
}
