// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCollisionChannelsInfoDataProcessor.h"

#include "ChaosVDModule.h"
#include "Containers/Ticker.h"


FChaosVDCollisionChannelsInfoDataProcessor::FChaosVDCollisionChannelsInfoDataProcessor() : FChaosVDDataProcessorBase(FChaosVDCollisionChannelsInfoContainer::WrapperTypeName)
{
	
}

bool FChaosVDCollisionChannelsInfoDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	const bool bNeedsLoadingOrderFixup = ProviderSharedPtr->GetDataProcessedSoFarNum() == 0;

	if (bNeedsLoadingOrderFixup)
	{
		// All serializable data in CVD needs to be backwards compatible. In order to do that the archive header we use needs to be the first thing we serialize and the first thing we read
		// There was an issue in the current implementation where the collision data container was being traced before the header data, therefore as a workaround if we are trying to read the collision
		// container data first, we need to delay it at least one frame. This is ok as this data is only used for the UI to decode collision channel ids to human-readable names.
		FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this, WeakTraceProvider = ProviderSharedPtr.ToWeakPtr()](float DeltaTime)
		{
			TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = WeakTraceProvider.Pin();

			if (ProviderSharedPtr)
			{
				if (!ProcessPendingCollisionChannelData(PendingCollisionChannelDataToProcess))
				{
					UE_LOGF(LogChaosVDEditor, Warning, "[%s] Failed to process deferred collision channel data.", __func__);
					ProviderSharedPtr->TypesFailedToSerialize.Add(FString(CompatibleType));
				}

				ProviderSharedPtr->DataProcessedSoFarCounter++;
				PendingCollisionChannelDataToProcess.Empty();
			}
				
			return false;
		}));

		PendingCollisionChannelDataToProcess = InData;
		ProviderSharedPtr->DataProcessedSoFarCounter--;
		
		UE_LOGF(LogChaosVDEditor, Warning, "[%s] Deferring loading collision channel data to the next frame, because the archive header is not loaded yet.", __func__);

		return true;
	}
	else
	{
		return ProcessPendingCollisionChannelData(InData);
	}
}

bool FChaosVDCollisionChannelsInfoDataProcessor::ProcessPendingCollisionChannelData(const TArray<uint8>& InData)
{
	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	TSharedPtr<FChaosVDRecording> RecordingInstancePtr = ProviderSharedPtr ? ProviderSharedPtr->GetRecordingForSession() : nullptr;
	if (!ensure(ProviderSharedPtr))
	{
		return false;
	}

	TSharedPtr<FChaosVDCollisionChannelsInfoContainer> CollisionChannelsData = MakeShared<FChaosVDCollisionChannelsInfoContainer>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *CollisionChannelsData, ProviderSharedPtr.ToSharedRef());
	
	if (bSuccess)
	{
		if (RecordingInstancePtr->GetCollisionChannelsInfoContainer())
		{
			EChaosVDRecordingAttributes RecordingAttributes = RecordingInstancePtr->GetAttributes();
			if (EnumHasAnyFlags(RecordingAttributes, EChaosVDRecordingAttributes::Merged))
			{
				UE_LOGF(LogChaosVDEditor, Warning, "[%s] Multiple Collision data channel in multi file mode is not supported yet. \n The collision channel data from the last loaded recording will be used. This is used for to decode collision channels ids into names, \n so as long the recordings are from the same build, this warning should not cause issues.", __func__);
			}
			else
			{
				UE_LOGF(LogChaosVDEditor, Error, "[%s] Collision channel data already loaded. This should not happen. Was the collision data serialized twice?", __func__);
			}
			
		}
		ProviderSharedPtr->GetRecordingForSession()->SetCollisionChannelsInfoContainer(CollisionChannelsData);
	}

	return bSuccess;
}
