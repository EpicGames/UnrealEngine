// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

struct FChaosVDSolverFrameData;

/**
 * Data processor implementation that read any recorded collision channels info
 */
class FChaosVDCollisionChannelsInfoDataProcessor final : public FChaosVDDataProcessorBase
{
public:
	explicit FChaosVDCollisionChannelsInfoDataProcessor();
	
	virtual bool ProcessRawData(const TArray<uint8>& InData) override;

private:

	bool ProcessPendingCollisionChannelData(const TArray<uint8>& InData);
	
	TArray<uint8> PendingCollisionChannelDataToProcess;
};

