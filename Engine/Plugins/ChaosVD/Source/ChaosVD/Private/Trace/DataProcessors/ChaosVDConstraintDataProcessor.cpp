// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDConstraintDataProcessor.h"

#include "ChaosVDRecording.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "Trace/ChaosVDTraceProvider.h"

FChaosVDConstraintDataProcessor::FChaosVDConstraintDataProcessor() : FChaosVDDataProcessorBase(FChaosVDConstraint::WrapperTypeName)
{
}

bool FChaosVDConstraintDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	FChaosVDConstraint RecordedConstraint;
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, RecordedConstraint, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		RecordedConstraint.SolverID = ProviderSharedPtr->GetRemappedSolverID(RecordedConstraint.SolverID);

		FChaosVDFrameStageData* CurrentSolverStage = ProviderSharedPtr->GetCurrentSolverStageDataForCurrentFrame(RecordedConstraint.SolverID, EChaosVDSolverStageAccessorFlags::None);

		if (ensureMsgf(CurrentSolverStage, TEXT("A MidPhase was traced without a valid step scope")))
		{
			AddConstraintToParticleIDMap(RecordedConstraint, RecordedConstraint.Particle0Index, *CurrentSolverStage);
			AddConstraintToParticleIDMap(RecordedConstraint, RecordedConstraint.Particle1Index, *CurrentSolverStage);
		}
	}

	return bSuccess;
}

void FChaosVDConstraintDataProcessor::AddConstraintToParticleIDMap(const FChaosVDConstraint& InConstraintData, int32 ParticleID, FChaosVDFrameStageData& InSolverStageData)
{
	if (TArray<FChaosVDConstraint>* ParticleConstraints = InSolverStageData.RecordedConstraintsByParticleID.Find(ParticleID))
	{
		ParticleConstraints->Add(InConstraintData);
	}
	else
	{
		InSolverStageData.RecordedConstraintsByParticleID.Add(ParticleID, { InConstraintData });
	}
}
