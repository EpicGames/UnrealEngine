// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

struct FChaosVDSolverFrameData;
struct FChaosVDFrameStageData;
struct FChaosVDConstraint;

/**
 * Data processor implementation that is able to deserialize traced Constraints
 */
class FChaosVDConstraintDataProcessor final : public FChaosVDDataProcessorBase
{
public:
	explicit FChaosVDConstraintDataProcessor();
	
	virtual bool ProcessRawData(const TArray<uint8>& InData) override;

	void AddConstraintToParticleIDMap(const FChaosVDConstraint& InConstraintData, int32 ParticleID, FChaosVDFrameStageData& InSolverStageData);
};
