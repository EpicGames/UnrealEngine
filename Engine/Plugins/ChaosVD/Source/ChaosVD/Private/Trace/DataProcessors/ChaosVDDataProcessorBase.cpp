// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

FChaosVDDataProcessorBase::~FChaosVDDataProcessorBase()
{
}

FStringView FChaosVDDataProcessorBase::GetCompatibleTypeName() const
{
	return CompatibleType;
}

bool FChaosVDDataProcessorBase::ProcessRawData(const TArray<uint8>& InData)
{
	ProcessedBytes += InData.Num();
	return true;
}

uint64 FChaosVDDataProcessorBase::GetProcessedBytes() const
{
	return ProcessedBytes;
}

void FChaosVDDataProcessorBase::SetTraceProvider(const TSharedPtr<FChaosVDTraceProvider>& InProvider)
{
	TraceProvider = InProvider;
}

FChaosVDGenericDataProcessor::FChaosVDGenericDataProcessor(FStringView InCompatibleType, const TFunction<bool(const TArray<uint8>&)>& InProcessDataCallback) : FChaosVDDataProcessorBase(InCompatibleType), ProcessDataCallback(InProcessDataCallback)
{
}

bool FChaosVDGenericDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	return FChaosVDDataProcessorBase::ProcessRawData(InData) && ProcessDataCallback && ProcessDataCallback(InData);
}
