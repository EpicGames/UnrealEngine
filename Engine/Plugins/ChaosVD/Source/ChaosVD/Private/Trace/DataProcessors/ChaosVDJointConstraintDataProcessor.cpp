// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDJointConstraintDataProcessor.h"

#include "DataWrappers/ChaosVDJointDataWrappers.h"
#include "Trace/ChaosVDTraceProvider.h"


FChaosVDJointConstraintDataProcessor::FChaosVDJointConstraintDataProcessor()
	: FChaosVDDataProcessorBase(FChaosVDJointConstraint::WrapperTypeName)
{
}

bool FChaosVDJointConstraintDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	TSharedPtr<FChaosVDJointConstraint> JointConstraint = MakeShared<FChaosVDJointConstraint>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *JointConstraint, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		JointConstraint->SolverID = ProviderSharedPtr->GetRemappedSolverID(JointConstraint->SolverID);

		FChaosVDFrameStageData* CurrentSolverStage = ProviderSharedPtr->GetCurrentSolverStageDataForCurrentFrame(JointConstraint->SolverID, EChaosVDSolverStageAccessorFlags::None);

		if (ensureMsgf(CurrentSolverStage, TEXT("A Joint Constraint was traced without a valid step scope")))
		{
			CurrentSolverStage->RecordedJointConstraints.Add(JointConstraint);
		}
	}

	return bSuccess;
}
