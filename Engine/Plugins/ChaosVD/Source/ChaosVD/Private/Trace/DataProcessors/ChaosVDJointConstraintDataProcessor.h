// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#pragma once
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

struct FChaosVDSolverFrameData;
struct FChaosVDConstraint;

/**
 * Data processor implementation that is able to deserialize traced joint constraints
 */
class FChaosVDJointConstraintDataProcessor final : public FChaosVDDataProcessorBase
{
public:
	explicit FChaosVDJointConstraintDataProcessor();
	
	virtual bool ProcessRawData(const TArray<uint8>& InData) override;
};


