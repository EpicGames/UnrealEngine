// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDMidPhaseDataProcessor.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "ChaosVDRecording.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "Trace/ChaosVDTraceProvider.h"

FChaosVDMidPhaseDataProcessor::FChaosVDMidPhaseDataProcessor() : FChaosVDDataProcessorBase(FChaosVDParticlePairMidPhase::WrapperTypeName)
{
}

bool FChaosVDMidPhaseDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	TSharedPtr<FChaosVDParticlePairMidPhase> MidPhase = MakeShared<FChaosVDParticlePairMidPhase>();

	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *MidPhase, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		MidPhase->SolverID = ProviderSharedPtr->GetRemappedSolverID(MidPhase->SolverID);

		FChaosVDFrameStageData* CurrentSolverStage = ProviderSharedPtr->GetCurrentSolverStageDataForCurrentFrame(MidPhase->SolverID, EChaosVDSolverStageAccessorFlags::None);

		if (ensureMsgf(CurrentSolverStage, TEXT("A MidPhase was traced without a valid step scope")))
		{
			CurrentSolverStage->RecordedMidPhases.Add(MidPhase);

			AddMidPhaseToParticleIDMap(MidPhase, MidPhase->Particle0Idx, *CurrentSolverStage);
			AddMidPhaseToParticleIDMap(MidPhase, MidPhase->Particle1Idx, *CurrentSolverStage);
		}
	}

	return bSuccess;
}

void FChaosVDMidPhaseDataProcessor::AddMidPhaseToParticleIDMap(const TSharedPtr<FChaosVDParticlePairMidPhase>& MidPhaseData, int32 ParticleID, FChaosVDFrameStageData& InSolverStageData)
{
	if (TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>* ParticleMidPhases = InSolverStageData.RecordedMidPhasesByParticleID.Find(ParticleID))
	{
		ParticleMidPhases->Add(MidPhaseData);
	}
	else
	{
		InSolverStageData.RecordedMidPhasesByParticleID.Add(ParticleID, { MidPhaseData });
	}
}
