// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"
#include "Templates/SharedPointer.h"

struct FChaosVDSolverFrameData;
struct FChaosVDFrameStageData;
struct FChaosVDParticlePairMidPhase;

/**
 * Data processor implementation that is able to deserialize traced MidPhases 
 */
class FChaosVDMidPhaseDataProcessor final : public FChaosVDDataProcessorBase
{
public:
	explicit FChaosVDMidPhaseDataProcessor();

	virtual bool ProcessRawData(const TArray<uint8>& InData) override;

	void AddMidPhaseToParticleIDMap(const TSharedPtr<FChaosVDParticlePairMidPhase>& MidPhaseData, int32 ParticleID, FChaosVDFrameStageData& InSolverStageData);
};
