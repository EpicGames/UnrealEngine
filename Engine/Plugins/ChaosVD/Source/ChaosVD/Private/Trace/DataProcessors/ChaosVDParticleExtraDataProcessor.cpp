// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDParticleExtraDataProcessor.h"

#include "ChaosVDModule.h"
#include "ChaosVisualDebugger/ChaosVDParticleExtraData.h"
#include "DataWrappers/ChaosVDParticleExtraDataContainer.h"
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"
#include "Trace/ChaosVDTraceProvider.h"
#include "ChaosVDSettingsManager.h"
#include "Settings/ChaosVDGeneralSettings.h"

using namespace Chaos::VisualDebugger;

FChaosVDParticleExtraDataProcessor::FChaosVDParticleExtraDataProcessor()
	: FChaosVDDataProcessorBase(FChaosVDParticleExtraData::WrapperTypeName)
	, CachedLoadingMode(EChaosVDParticleExtraDataLoadingMode::SerializeBinOnly)
{
	// Raw struct serialisation with no explicit versioning  - not safe to read across binary-incompatible builds.
	bBackwardsCompatible = false;

	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		CachedLoadingMode = Settings->ParticleExtraDataLoadingMode;
		Settings->OnSettingsChanged().AddRaw(this, &FChaosVDParticleExtraDataProcessor::HandleSettingsChanged);
	}
}

FChaosVDParticleExtraDataProcessor::~FChaosVDParticleExtraDataProcessor()
{
	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		Settings->OnSettingsChanged().RemoveAll(this);
	}
}

void FChaosVDParticleExtraDataProcessor::HandleSettingsChanged(UObject* InSettings)
{
	if (const UChaosVDGeneralSettings* Settings = Cast<UChaosVDGeneralSettings>(InSettings))
	{
		CachedLoadingMode = Settings->ParticleExtraDataLoadingMode;
	}
}

bool FChaosVDParticleExtraDataProcessor::ShouldAlwaysSkip() const
{
	return CachedLoadingMode == EChaosVDParticleExtraDataLoadingMode::SkipAll;
}

bool FChaosVDParticleExtraDataProcessor::IsBackwardsCompatible() const
{
	// SerializeBinOnly strips NativeSerialization entries internally -- no crash risk.
	// Tell the provider we're "safe" so it won't add us to UnsafeTypesProcessed.
	// LoadAll returns false so the unsafe-data warning is shown.
	return CachedLoadingMode == EChaosVDParticleExtraDataLoadingMode::SerializeBinOnly;
}

void FChaosVDParticleExtraDataProcessor::GetPostLoadNativeTypesWithChannels(TMap<FName, FName>& OutTypeToChannel) const
{
	OutTypeToChannel.Append(NativeTypesToChannel);
}

bool FChaosVDParticleExtraDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	const TSharedPtr<FChaosVDParticleExtraData> ExtraData = MakeShared<FChaosVDParticleExtraData>();
	const bool bSuccess = ReadDataFromBuffer(InData, *ExtraData, ProviderSharedPtr.ToSharedRef());

	if (bSuccess && ExtraData->HasValidData())
	{
		if (FChaosVDSolverFrameData* SolverFrame = ProviderSharedPtr->GetCurrentSolverFrame(ExtraData->SolverID))
		{
			if (TSharedPtr<FChaosVDParticleExtraDataContainer> Container =
				SolverFrame->GetCustomData().GetOrAddDefaultData<FChaosVDParticleExtraDataContainer>())
			{
				// Cache the recording's name table in the container so the display component
				// can deserialize inner struct FNames without a separate route to the provider.
				if (!Container->NameTable)
				{
					Container->NameTable = ProviderSharedPtr->GetNameTableInstance();
				}

				Container->DataBySolverAndParticleID.FindOrAdd(ExtraData->SolverID).Add(ExtraData->ParticleID, ExtraData);

				const bool bFilterNative = CachedLoadingMode == EChaosVDParticleExtraDataLoadingMode::SerializeBinOnly;

				for (FChaosVDExtraDataCategory& Category : ExtraData->Categories)
				{
					// Register native entries first (for the post-load dialog), then strip them if filtering.
					for (const FChaosVDExtraDataStructEntry& Entry : Category.Entries)
					{
						if (Entry.SerializationMode == EChaosVDExtraDataSerializationMode::NativeSerialization)
						{
							NativeTypesToChannel.FindOrAdd(Entry.StructTypePath) = Category.SourceChannelId;
							Container->NativeSerializedStructTypes.Add(Entry.StructTypePath);
						}
					}

					if (bFilterNative)
					{
						Category.Entries.RemoveAll([](const FChaosVDExtraDataStructEntry& Entry)
						{
							return Entry.SerializationMode == EChaosVDExtraDataSerializationMode::NativeSerialization;
						});
					}
				}
			}
		}
	}

	return bSuccess;
}
