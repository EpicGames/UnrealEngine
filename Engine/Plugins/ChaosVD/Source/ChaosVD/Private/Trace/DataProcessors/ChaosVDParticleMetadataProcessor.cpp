// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDParticleMetadataProcessor.h"

FChaosVDParticleMetadataProcessor::FChaosVDParticleMetadataProcessor() : FChaosVDDataProcessorBase(FChaosVDParticleMetadata::WrapperTypeName)
{
}

bool FChaosVDParticleMetadataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	TSharedPtr<FChaosVDParticleMetadata> ParticleMetaData = MakeShared<FChaosVDParticleMetadata>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *ParticleMetaData, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		ProviderSharedPtr->AddParticleMetadata(ParticleMetaData->MetadataID, ParticleMetaData);
	}

	return bSuccess;
}
