// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDSceneQueryDataProcessor.h"

#include "ChaosVDRecording.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVDSerializedNameTable.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#include "Trace/ChaosVDTraceProvider.h"

FChaosVDSceneQueryDataProcessor::FChaosVDSceneQueryDataProcessor() : FChaosVDDataProcessorBase(FChaosVDQueryDataWrapper::WrapperTypeName)
{
}

bool FChaosVDSceneQueryDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	const TSharedPtr<FChaosVDQueryDataWrapper> QueryData = MakeShared<FChaosVDQueryDataWrapper>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *QueryData, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		if (TSharedPtr<FChaosVDGameFrameData> CurrentFrameData = ProviderSharedPtr->GetCurrentGameFrame().Pin())
		{
			if (TSharedPtr<FChaosVDSceneQueriesDataContainer> SQDataContainer = CurrentFrameData->GetCustomDataHandler().GetOrAddDefaultData<FChaosVDSceneQueriesDataContainer>())
			{
				QueryData->WorldSolverID = ProviderSharedPtr->GetRemappedSolverID(QueryData->WorldSolverID);

				// If ParentQueryID was set, this is a sub query, so find the parent add it to the sub-queries list so we can navigate through the query "hierarchy" later on
				if (QueryData->ParentQueryID != INDEX_NONE)
				{
					if (TMap<int32, TSharedPtr<FChaosVDQueryDataWrapper>>* ParentQueryDataByQueryIDPtr = SQDataContainer->RecordedSceneQueriesBySolverID.Find(QueryData->WorldSolverID))
					{
						if (const TSharedPtr<FChaosVDQueryDataWrapper>* ParentQueryData = ParentQueryDataByQueryIDPtr->Find(QueryData->ParentQueryID))
						{
							(*ParentQueryData)->SubQueriesIDs.Add(QueryData->ID);
						}
					}
				}

				SQDataContainer->RecordedSceneQueriesByQueryID.Add(QueryData->ID, QueryData);
				SQDataContainer->RecordedSceneQueriesBySolverID.FindOrAdd(QueryData->WorldSolverID).Add(QueryData->ID, QueryData);

				CurrentFrameData->MarkDirty();
			}
		}
	}

	return bSuccess;
}

