// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDSceneQueryVisitDataProcessor.h"

#include "ChaosVDRecording.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#include "Trace/ChaosVDTraceProvider.h"

FChaosVDSceneQueryVisitDataProcessor::FChaosVDSceneQueryVisitDataProcessor() : FChaosVDDataProcessorBase(FChaosVDQueryVisitStep::WrapperTypeName)
{
}

bool FChaosVDSceneQueryVisitDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	FChaosVDQueryVisitStep VisitStepData;
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, VisitStepData, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		if (TSharedPtr<FChaosVDGameFrameData> CurrentFrameData = ProviderSharedPtr->GetCurrentGameFrame().Pin())
		{
			if (TSharedPtr<FChaosVDSceneQueriesDataContainer> SQDataContainer = CurrentFrameData->GetCustomDataHandler().GetOrAddDefaultData<FChaosVDSceneQueriesDataContainer>())
			{
				if (TSharedPtr<FChaosVDQueryDataWrapper>* QueryDataPtrPtr = SQDataContainer->RecordedSceneQueriesByQueryID.Find(VisitStepData.OwningQueryID))
				{
					TSharedPtr<FChaosVDQueryDataWrapper> QueryDataPtr = *QueryDataPtrPtr;
					if (QueryDataPtrPtr->IsValid())
					{
						//TODO: There is an existing issue where CVD is recording garbage data for the Hit Face Normal if the query is a line trace.
						// That value is not used in line traces, so for now just clear it out until we can implement a better solution
						// where we include any post-processing done on the hit data during the HitConversion step.
						if (QueryDataPtr->Type == EChaosVDSceneQueryType::RayCast)
						{
							VisitStepData.HitData.FaceNormal = FVector::ZeroVector;
						}
						
						VisitStepData.SolverID_Editor = QueryDataPtr->WorldSolverID;

						if (VisitStepData.HitData.HasValidData())
						{
							// Quick and dirty way of show the hits in the details panel. If copying this data around becomes a bottle neck we can write a customization layout for it
							QueryDataPtr->Hits.Add(VisitStepData);
						}

						QueryDataPtr->SQVisitData.Emplace(MoveTemp(VisitStepData));
					}
				}
			}
		}
	}

	return bSuccess;
}