// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

/**
 * Data processor implementation that is able to deserialize traced SQ Visits
 */
class FChaosVDSceneQueryVisitDataProcessor final : public FChaosVDDataProcessorBase
{
public:
	explicit FChaosVDSceneQueryVisitDataProcessor();

	virtual bool ProcessRawData(const TArray<uint8>& InData) override;
};

