// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDSerializedNameEntryDataProcessor.h"

#include "ChaosVisualDebugger/ChaosVDSerializedNameTable.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Serialization/MemoryReader.h"
#include "Trace/ChaosVDTraceProvider.h"

FChaosVDSerializedNameEntryDataProcessor::FChaosVDSerializedNameEntryDataProcessor()
	: FChaosVDDataProcessorBase(Chaos::VisualDebugger::FChaosVDSerializedNameEntry::WrapperTypeName)
{
}

bool FChaosVDSerializedNameEntryDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);
	
	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	TSharedPtr<Chaos::VisualDebugger::FChaosVDSerializableNameTable> NameTableInstance = ProviderSharedPtr->GetNameTableInstance();
	if (!ensure(NameTableInstance.IsValid()))
	{
		return false;
	}

	Chaos::VisualDebugger::FChaosVDSerializedNameEntry NameEntry;
	
	FMemoryReader MemReader(InData);

	const Chaos::VisualDebugger::FChaosVDArchiveHeader& RecordedHeader = ProviderSharedPtr->GetHeaderData();
	ApplyHeaderDataToArchive(MemReader, RecordedHeader);
	
	MemReader << NameEntry;

	NameTableInstance->AddNameToTable(NameEntry);

	return true;
}
