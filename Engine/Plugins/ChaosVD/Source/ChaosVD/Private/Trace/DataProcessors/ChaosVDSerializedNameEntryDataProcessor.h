// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

/**
 * Data processor implementation that is able to deserialize traced Name Entries
 */
class FChaosVDSerializedNameEntryDataProcessor final : public FChaosVDDataProcessorBase
{
public:
	explicit FChaosVDSerializedNameEntryDataProcessor();

	virtual bool ProcessRawData(const TArray<uint8>& InData) override;
};
