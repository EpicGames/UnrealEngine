// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDTraceImplicitObjectProcessor.h"

#include "Chaos/ImplicitObject.h"
#include "ChaosVDRecording.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "DataWrappers/ChaosVDImplicitObjectDataWrapper.h"
#include "Trace/ChaosVDTraceProvider.h"

using FChaosVDImplicitObjectWrapper = FChaosVDImplicitObjectDataWrapper<Chaos::FImplicitObjectPtr, Chaos::FChaosArchive>;

FChaosVDTraceImplicitObjectProcessor::FChaosVDTraceImplicitObjectProcessor(): FChaosVDDataProcessorBase(FChaosVDImplicitObjectWrapper::WrapperTypeName)
{
}

bool FChaosVDTraceImplicitObjectProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	FChaosVDImplicitObjectWrapper WrappedGeometryData;

	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer<FChaosVDImplicitObjectWrapper>(InData, WrappedGeometryData, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		if (TSharedPtr<FChaosVDRecording> Recording = ProviderSharedPtr->GetRecordingForSession())
		{
			Recording->AddImplicitObject(WrappedGeometryData.Hash, WrappedGeometryData.ImplicitObject);
		}
	}

	return bSuccess;
}
