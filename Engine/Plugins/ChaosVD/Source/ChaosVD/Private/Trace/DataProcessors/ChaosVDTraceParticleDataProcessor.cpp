// Copyright Epic Games, Inc. All Rights Reserved.

#include "Trace/DataProcessors/ChaosVDTraceParticleDataProcessor.h"

#include "ChaosVDModule.h"
#include "ChaosVDRecording.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "Trace/ChaosVDTraceProvider.h"

FChaosVDTraceParticleDataProcessor::FChaosVDTraceParticleDataProcessor(): FChaosVDDataProcessorBase(FChaosVDParticleDataWrapper::WrapperTypeName)
{
}

bool FChaosVDTraceParticleDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	TSharedPtr<FChaosVDParticleDataWrapper> ParticleData = MakeShared<FChaosVDParticleDataWrapper>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *ParticleData, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		EChaosVDSolverStageAccessorFlags StageAccessorFlags = EChaosVDSolverStageAccessorFlags::CreateNewIfEmpty | EChaosVDSolverStageAccessorFlags::CreateNewIfClosed;

		ParticleData->SolverID = ProviderSharedPtr->GetRemappedSolverID(ParticleData->SolverID);

		if (FChaosVDFrameStageData* CurrentSolverStage = ProviderSharedPtr->GetCurrentSolverStageDataForCurrentFrame(ParticleData->SolverID, StageAccessorFlags))
		{
			int32 NewParticleDataIndex = INDEX_NONE;

			if (ParticleData->HasLegacyDebugName())
			{
				const FString& DebugName = ParticleData->GetDebugName();
				ParticleData->MetadataId = CityHash64(reinterpret_cast<const char*>(*DebugName), DebugName.Len() * sizeof(TCHAR));
			}
			else
			{
				ParticleData->SetMetadataInstance(ProviderSharedPtr->GetParticleMetadata(ParticleData->MetadataId));
			}

			// Some stages like the auto generated "In-Between" stage, or "Rewind Frame" might have duplicated data if a particle was traced twice.
			// As CVD only shows the last state per stage, we need to just keep the last seen state
			// We do this on trace analysis to not do the de-duplication process during playback
			if (int32* ExistingParticleDataIndex = CurrentSolverStage->CurrentRecordedParticlesIndexes.Find(ParticleData->ParticleIndex))
			{
				CurrentSolverStage->RecordedParticlesData[*ExistingParticleDataIndex] = ParticleData;
			}
			else
			{
				NewParticleDataIndex = CurrentSolverStage->RecordedParticlesData.Add(ParticleData);
			}	

			if (NewParticleDataIndex != INDEX_NONE)
			{
				CurrentSolverStage->CurrentRecordedParticlesIndexes.Add(ParticleData->ParticleIndex, NewParticleDataIndex);
			}
			
		}
	}

	return bSuccess;
}
