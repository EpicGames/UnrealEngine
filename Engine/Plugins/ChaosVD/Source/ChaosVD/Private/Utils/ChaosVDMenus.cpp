// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDMenus.h"

#include "ChaosVDCommands.h"
#include "ChaosVDPlaybackViewportClient.h"
#include "ToolMenus.h"
#include "ShowFlagFilter.h"
#include "ShowFlagMenuCommands.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SChaosVDPlaybackViewport.h"
#include "Widgets/Input/SSpinBox.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

namespace Chaos::VisualDebugger::Menus
{

namespace Private
{

template <typename TNumericValue>
TSharedRef<SWidget> GenerateSpinBoxMenuEntryWidget(const FText& ToolTipText, TNumericValue MinValue, TNumericValue MaxValue, typename SSpinBox<TNumericValue>::FOnValueChanged&& ValueChangedDelegate, const TAttribute<TNumericValue>&& ValueAttribute, const TAttribute<bool>&& EnabledAttribute)
{
	return SNew(SBox)
		.HAlign(HAlign_Right)
		[
			SNew(SBox)
			.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
			.WidthOverride(100.0f)
			[
				SNew ( SBorder )
				.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
				.Padding(FMargin(1.0f))
				[
					SNew(SSpinBox<TNumericValue>)
					.Style(&FAppStyle::Get(), "Menu.SpinBox")
					.ToolTipText(ToolTipText)
					.MinValue(MinValue)
					.MaxValue(MaxValue)
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
					.Value(ValueAttribute)
					.OnValueChanged(ValueChangedDelegate)
					.IsEnabled(EnabledAttribute)
				]
			]
		];
}

FToolMenuEntry CreateFramerateOverrideEntry()
{
	return FToolMenuEntry::InitDynamicEntry("DynamicFramerateOverride", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
	{
		TSharedPtr<SChaosVDPlaybackViewport> Viewport = StaticCastSharedPtr<SChaosVDPlaybackViewport>(UUnrealEdViewportToolbarContext::GetEditorViewport(Section.Context));
		if (!Viewport)
		{
			return;
		}
	
		TAttribute<int32> ValueAttribute = TAttribute<int32>::CreateSP(Viewport.Get(), &SChaosVDPlaybackViewport::GetCurrentTargetFrameRateOverride);
		TAttribute<bool> EnabledAttribute = TAttribute<bool>::CreateSP(Viewport.Get(), &SChaosVDPlaybackViewport::IsUsingFrameRateOverride);
		
		SSpinBox<int32>::FOnValueChanged OnValueChanged = SSpinBox<int32>::FOnValueChanged::CreateSP(Viewport.Get(), &SChaosVDPlaybackViewport::SetCurrentTargetFrameRateOverride);
		
		constexpr int32 MinValue = 1;
		constexpr int32 MaxValue = 1000;
		
		Section.AddEntry(FToolMenuEntry::InitWidget(
			"FramerateOverride",
			GenerateSpinBoxMenuEntryWidget(
				LOCTEXT("FramerateOverrideTooltip", "Target framerate we should play the loaded recording at"),
				MinValue,
				MaxValue,
				MoveTemp(OnValueChanged),
				MoveTemp(ValueAttribute),
				MoveTemp(EnabledAttribute)
			),
			LOCTEXT("FrameRateOverride", "Target Framerate")
		));
	}));
}

FToolMenuEntry CreateTrackingDistanceEntry()
{
	return FToolMenuEntry::InitDynamicEntry("DynamicTrackingDistance", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
	{
		TSharedPtr<SEditorViewport> Viewport = UUnrealEdViewportToolbarContext::GetEditorViewport(Section.Context);
		if (!Viewport)
		{
			return;
		}
		
		TSharedPtr<FChaosVDPlaybackViewportClient> Client = StaticCastSharedPtr<FChaosVDPlaybackViewportClient>(Viewport->GetViewportClient());
		if (!Client)
		{
			return;
		}
	
		TAttribute<float> ValueAttribute = TAttribute<float>::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FChaosVDPlaybackViewportClient> Client = WeakClient.Pin())
			{
				return Client->GetAutoTrackingViewDistance();
			}
			return 0.0f;
		});
		
		TAttribute<bool> EnabledAttribute = TAttribute<bool>::CreateLambda([WeakClient = Client.ToWeakPtr()]
		{
			if (TSharedPtr<FChaosVDPlaybackViewportClient> Client = WeakClient.Pin())
			{
				return Client->IsAutoTrackingSelectedObject();
			}
			return false;
		});
		
		SSpinBox<float>::FOnValueChanged OnValueChanged = SSpinBox<float>::FOnValueChanged::CreateLambda([WeakClient = Client.ToWeakPtr()](float Value)
		{
			if (TSharedPtr<FChaosVDPlaybackViewportClient> Client = WeakClient.Pin())
			{
				Client->SetAutoTrackingViewDistance(Value);
			}
		});
		
		constexpr float MinValue = 1.0f;
		constexpr float MaxValue = 100000.0f;
		
		Section.AddEntry(FToolMenuEntry::InitWidget(
			"TrackingDistance",
			GenerateSpinBoxMenuEntryWidget(
				LOCTEXT("TrackingDistanceTooltip", "Distance from which we want to track the selected object"),
				MinValue,
				MaxValue,
				MoveTemp(OnValueChanged),
				MoveTemp(ValueAttribute),
				MoveTemp(EnabledAttribute)
			),
			LOCTEXT("TrackingDistance", "Follow Distance")
		));
	}));
}

FToolMenuEntry CreateGoToLocationEntry()
{
	return FToolMenuEntry::InitDynamicEntry("DynamicGoToLocation", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& Section)
	{
		TSharedPtr<SChaosVDPlaybackViewport> Viewport = StaticCastSharedPtr<SChaosVDPlaybackViewport>(UUnrealEdViewportToolbarContext::GetEditorViewport(Section.Context));
		if (!Viewport)
		{
			return;
		}
		
		TSharedRef<SWidget> Widget = SNew(SBox)
			.HAlign(HAlign_Right)
			[
				SNew(SBox)
				.Padding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
				.WidthOverride(100.0f)
				[
					SNew (SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("Menu.WidgetBorder"))
					.Padding(FMargin(1.0f))
					[
						SNew(SEditableText)
						.ToolTipText(LOCTEXT("GoToLocationTooltip", "Location to teleport to."))
						.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
						.OnTextCommitted(FOnTextCommitted::CreateLambda([WeakViewport = Viewport.ToWeakPtr()](const FText& InLocationAsText, ETextCommit::Type Type)
						{
							TSharedPtr<SChaosVDPlaybackViewport> Viewport = WeakViewport.Pin();
							if (!Viewport || Type != ETextCommit::OnEnter)
							{
								return;
							}
							
							FVector Location;
							Location.InitFromString(InLocationAsText.ToString());
							Viewport->GoToLocation(Location);
						}))
					]
				]
			];
		
		Section.AddEntry(FToolMenuEntry::InitWidget(
			"GoToLocation",
			Widget,
			LOCTEXT("GoToLocation", "Go to Location")
		));
	}));
}

} // End Private

FToolMenuEntry CreateShowSubmenu()
{
	FToolMenuEntry Entry = UE::UnrealEd::CreateShowSubmenu(FNewToolMenuDelegate::CreateLambda([](UToolMenu* ShowMenu)
	{
		FToolMenuSection& CommonFlags = ShowMenu->AddSection("CommonViewportFlags", LOCTEXT("ToolbarCommonViewportFlags", "Common Show Flags"));
		
		constexpr bool bOpenSubMenuOnClick = false;
		CommonFlags.AddSubMenu(
			"CommonViewportFlags",
			LOCTEXT("CommonShowFlagsMenuLabel", "Common Show Flags"),
			LOCTEXT("CommonShowFlagsMenuToolTip", "Set of flags to enable/disable specific viewport features"),
			FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
			{
				// Only include the flags that might be helpful.
				static const FShowFlagFilter ShowFlagFilter = FShowFlagFilter(FShowFlagFilter::ExcludeAllFlagsByDefault)
					.IncludeFlag(FEngineShowFlags::SF_AntiAliasing)
					.IncludeFlag(FEngineShowFlags::SF_Grid)
					.IncludeFlag(FEngineShowFlags::SF_Translucency)
					.IncludeFlag(FEngineShowFlags::SF_MeshEdges)
					.IncludeFlag(FEngineShowFlags::SF_HitProxies)
					.IncludeFlag(FEngineShowFlags::SF_Fog)
					.IncludeFlag(FEngineShowFlags::SF_Pivot);

				FShowFlagMenuCommands::Get().BuildShowFlagsMenu(Menu, ShowFlagFilter);
			}),
			bOpenSubMenuOnClick,
			FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbar.Settings"))
		);
	}));
	
	// Re-enable the "show" label
	Entry.ToolBarData.LabelOverride = TAttribute<FText>();
	
	return Entry;
}

FToolMenuEntry CreateSettingsSubmenu()
{
	FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu("Settings",
		LOCTEXT("SettingsMenuLabel", "Settings"),
		LOCTEXT("SettingsMenuTooltip", "Additional settings for this viewport."),
		FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
		{
			FToolMenuSection& OptionsSection = Menu->AddSection("ViewportOptions", LOCTEXT("ViewportOptionsMenuHeader", "Viewport Options"));
			{
				OptionsSection.AddSubMenu(
					"PlaybackFramerate",
					LOCTEXT("FrameRateOptionsMenuLabel", "Playback Framerate"),
					LOCTEXT("FrameRateOptionsMenuToolTip", "Options that control how CVD plays a recording."),
					FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
					{
						FToolMenuSection& Section = Menu->AddSection("Framerate", LOCTEXT("FramerateMenuHeader", "Framerate"));
						Section.AddMenuEntry(FChaosVDCommands::Get().OverridePlaybackFrameRate);
						Section.AddEntry(Private::CreateFramerateOverrideEntry());
					}),
					false,
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("EditorViewport.ToggleFPS"))
				);
			
				OptionsSection.AddSubMenu(
					"ObjectTracking",
					LOCTEXT("ObjectTrackingMenuLabel", "Object Tracking"),
					LOCTEXT("ObjectTrackingMenuToolTip", "Options that control how objects are tracked in the scene by the camera."),
					FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
					{
						FToolMenuSection& Section = Menu->AddSection("ObjectTracking", LOCTEXT("ObjectTrackingMenuHeader", "Object Tracking"));
						Section.AddMenuEntry(FChaosVDCommands::Get().ToggleFollowSelectedObject);
						Section.AddEntry(Private::CreateTrackingDistanceEntry());
					}),
					false,
					FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("AnimViewportMenu.CameraFollow.Small"))
				);
				
				OptionsSection.AddSeparator(NAME_None);
				
				OptionsSection.AddMenuEntry(FChaosVDCommands::Get().AllowTranslucentSelection);
			}
			
			FToolMenuSection& UtilsSection = Menu->AddSection("Utilities", LOCTEXT("ViewportUtilMenuHeader", "Utils"));
			{
				UtilsSection.AddEntry(Private::CreateGoToLocationEntry());
			}
		})
	);
	
	Entry.ToolBarData.LabelOverride = FText::GetEmpty();
	Entry.Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "EditorViewportToolBar.OptionsDropdown");
	
	return Entry;
}

}

#undef LOCTEXT_NAMESPACE
