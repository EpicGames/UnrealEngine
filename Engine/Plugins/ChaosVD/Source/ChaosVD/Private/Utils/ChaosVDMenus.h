// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

struct FToolMenuEntry;

namespace Chaos::VisualDebugger::Menus
{

const FName ShowMenuName = "ChaosVDViewportToolbarBase.Show";

FToolMenuEntry CreateShowSubmenu();

FToolMenuEntry CreateSettingsSubmenu();

}
