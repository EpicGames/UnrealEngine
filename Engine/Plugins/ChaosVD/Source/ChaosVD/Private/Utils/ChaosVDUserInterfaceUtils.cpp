// Copyright Epic Games, Inc. All Rights Reserved.

#include "Utils/ChaosVDUserInterfaceUtils.h"
#include "PropertyEditorModule.h"
#include "Modules/ModuleManager.h"
#include "ToolMenu.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Settings/ChaosVDCoreSettings.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

TSharedRef<IStructureDetailsView> Chaos::VisualDebugger::Utils::MakeStructDetailsViewForMenu()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	const FStructureDetailsViewArgs StructDetailsViewArgs;
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowScrollBar = false;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.ColumnWidth = 1.0f;

	return PropertyEditorModule.CreateStructureDetailView(DetailsViewArgs,StructDetailsViewArgs, nullptr);
}

TSharedRef<IDetailsView> Chaos::VisualDebugger::Utils::MakeObjectDetailsViewForMenu()
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = true;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowScrollBar = false;
	DetailsViewArgs.bShowObjectLabel = false;
	DetailsViewArgs.bCustomNameAreaLocation = true;
	DetailsViewArgs.ColumnWidth = 0.45f;
	DetailsViewArgs.bShowModifiedPropertiesOption = true;

	return PropertyEditorModule.CreateDetailView(DetailsViewArgs);
}

void Chaos::VisualDebugger::Utils::CreateMenuEntryForObject(UToolMenu* Menu, UObject* Object, EChaosVDSaveSettingsOptions MenuEntryOptions)
{
	if (!Menu)
	{
		return;
	}

	if (!Object)
	{
		TSharedRef<SWidget> ErrorMessageWidget = SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("ChaosVisualDebugger", "CreateMenuEntryForObjectErrorMessage", "Failed to create menu for object. The provied object is null"))
		];
		
		FToolMenuEntry ErrorMenuEntry = FToolMenuEntry::InitWidget(TEXT("InvalidObject"), ErrorMessageWidget, FText::GetEmpty());
		Menu->AddMenuEntry(NAME_None, ErrorMenuEntry);
		return;
	}
	
	TSharedRef<IDetailsView> DetailsView = MakeObjectDetailsViewForMenu();
	DetailsView->SetObject(Object);
	FToolMenuEntry MenuEntry = FToolMenuEntry::InitWidget(Object->GetFName(), DetailsView, FText::GetEmpty());
	Menu->AddMenuEntry(NAME_None, MenuEntry);

	if (EnumHasAnyFlags(MenuEntryOptions, EChaosVDSaveSettingsOptions::ShowSaveButton | EChaosVDSaveSettingsOptions::ShowResetButton))
	{
		Menu->AddMenuEntry(NAME_None,FToolMenuEntry::InitSeparator(NAME_None));

		if (EnumHasAnyFlags(MenuEntryOptions, EChaosVDSaveSettingsOptions::ShowSaveButton))
		{
			const FString SaveObjectButtonName = Object->GetName() + TEXT("SaveButton");
			
			FToolMenuEntry SaveMenuEntry = FToolMenuEntry::InitMenuEntry(FName(SaveObjectButtonName),
				NSLOCTEXT("ChaosVisualDebugger","CreateMenuEntryForObjectSaveButtonLabel", "Save Settings"),
				NSLOCTEXT("ChaosVisualDebugger","CreateMenuEntryForObjectSaveButtonToolTip", "Saves the current settings into the Editor's configuration file"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("LevelEditor.Save")),
				FUIAction(
					FExecuteAction::CreateLambda([ObjectWeakPtr = TWeakObjectPtr(Object)]()
					{
						if (UObject* Object = ObjectWeakPtr.Get())
						{
							constexpr bool bAllowCopyToDefaultObject = false;
							Object->SaveConfig(CPF_Config,nullptr, GConfig, bAllowCopyToDefaultObject);
						}
					})));

			Menu->AddMenuEntry(NAME_None, SaveMenuEntry);
		}
		
		if (EnumHasAnyFlags(MenuEntryOptions, EChaosVDSaveSettingsOptions::ShowResetButton))
		{
			const FString ResetObjectButtonName = Object->GetName() + TEXT("Reset");

			FToolMenuEntry ResetMenuEntry = FToolMenuEntry::InitMenuEntry(FName(ResetObjectButtonName),
				NSLOCTEXT("ChaosVisualDebugger","CreateMenuEntryForObjectResetButtonLabel", "Reset to defaults"),
				NSLOCTEXT("ChaosVisualDebugger","CreateMenuEntryForObjectResetButtonToolTip", "Reset this settings section to its defaults values and save it to the Editor's configuration file"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("PropertyWindow.DiffersFromDefault")),
				FUIAction(
				FExecuteAction::CreateLambda([ObjectWeakPtr = TWeakObjectPtr(Object)]()
				{
					if (UObject* Object =ObjectWeakPtr.Get())
					{
						FChaosVDSettingsManager::Get().ResetSettings(Object->GetClass());
					}
				})));

			Menu->AddMenuEntry(NAME_None, ResetMenuEntry);
		}
	}
}
