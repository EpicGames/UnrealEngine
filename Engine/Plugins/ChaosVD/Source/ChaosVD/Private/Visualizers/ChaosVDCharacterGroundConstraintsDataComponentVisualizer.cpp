// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/ChaosVDCharacterGroundConstraintsDataComponentVisualizer.h"

#include "ChaosVDScene.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDTabsIDs.h"
#include "EditorViewportClient.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Components/ChaosVDSolverCharacterGroundConstraintDataComponent.h"
#include "Settings/ChaosVDCharacterConstraintsVisualizationSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Utils/ChaosVDMenus.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

bool FChaosVDCharacterGroundConstraintVisualizationDataContext::IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags Flag) const
{
	const EChaosVDCharacterGroundConstraintDataVisualizationFlags FlagsAsParticleFlags = static_cast<EChaosVDCharacterGroundConstraintDataVisualizationFlags>(VisualizationFlags);
	return EnumHasAnyFlags(FlagsAsParticleFlags, Flag);
}

FChaosVDCharacterGroundConstraintDataComponentVisualizer::FChaosVDCharacterGroundConstraintDataComponentVisualizer()
{
	FChaosVDCharacterGroundConstraintDataComponentVisualizer::RegisterVisualizerMenus();
	InspectorTabID = FChaosVDTabID::ConstraintsInspector;
}

void FChaosVDCharacterGroundConstraintDataComponentVisualizer::RegisterVisualizerMenus()
{
	FName MenuSection("CharacterGroundConstraintDataVisualization.Show");
	FText MenuSectionLabel = LOCTEXT("CharacterConstraintDataVisualizationShowMenuLabel", "Character Ground Constraints Data Visualization");
	FText FlagsMenuLabel = LOCTEXT("CharacterConstraintDataVisualizationFlagsMenuLabel", "Character Ground Constraints Data Flags");
	FText FlagsMenuTooltip = LOCTEXT("CharacterConstraintDataVisualizationFlagsMenuToolTip", "Set of flags to enable/disable visibility of specific types of Character Constraints data");
	FSlateIcon FlagsMenuIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("ClassIcon.Character"));

	FText SettingsMenuLabel = LOCTEXT("CharacterConstraintDataVisualizationMenuLabel", "Character Ground Constraints Visualization Settings");
	FText SettingsMenuTooltip = LOCTEXT("CharacterConstraintDataVisualizationMenuToolTip", "Options to change how the recorded Character Constraints data is debug drawn");
	
	CreateGenericVisualizerMenu<UChaosVDCharacterConstraintsVisualizationSettings, EChaosVDCharacterGroundConstraintDataVisualizationFlags>(Chaos::VisualDebugger::Menus::ShowMenuName, MenuSection, MenuSectionLabel, FlagsMenuLabel, FlagsMenuTooltip, FlagsMenuIcon, SettingsMenuLabel, SettingsMenuTooltip);
}

void FChaosVDCharacterGroundConstraintDataComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDSolverCharacterGroundConstraintDataComponent* ConstraintDataComponent = Cast<UChaosVDSolverCharacterGroundConstraintDataComponent>(Component);
	if (!ConstraintDataComponent)
	{
		return;
	}
	
	AChaosVDSolverInfoActor* SolverInfoActor = Cast<AChaosVDSolverInfoActor>(Component->GetOwner());
	if (!SolverInfoActor)
	{
		return;
	}

	if (!SolverInfoActor->IsVisible())
	{
		return;
	}

	const TSharedPtr<FChaosVDScene> CVDScene = SolverInfoActor->GetScene().Pin();
	if (!CVDScene)
	{
		return;
	}

	const TSharedPtr<FChaosVDRecording> CVDRecording = CVDScene->GetLoadedRecording();
	if (!CVDRecording)
	{
		return;
	}

	FChaosVDCharacterGroundConstraintVisualizationDataContext VisualizationContext;
	VisualizationContext.CVDScene = CVDScene;
	VisualizationContext.SpaceTransform = SolverInfoActor->GetSimulationTransform();
	VisualizationContext.SolverInfoActor = SolverInfoActor;

	VisualizationContext.VisualizationFlags = static_cast<uint32>(UChaosVDCharacterConstraintsVisualizationSettings::GetDataVisualizationFlags());
	VisualizationContext.DebugDrawSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCharacterConstraintsVisualizationSettings>();

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::EnableDraw))
	{
		return;
	}
	
	TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = CVDScene->GetSolverDataSelectionObject().Pin();
	TSharedPtr<FChaosVDSolverDataSelectionHandle> SelectionHandle;
	if (SelectionObject)
	{
		SelectionHandle = SelectionObject->GetCurrentSelectionHandle();
	}

	// If nothing is selected, fallback to draw all character ground constraints
	const bool bDrawOnlySelected = VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::OnlyDrawSelected) && SelectionHandle && SelectionHandle->IsSelected();
	if (bDrawOnlySelected)
	{
		if (FChaosVDCharacterGroundConstraint* Constraint = SelectionHandle->GetData<FChaosVDCharacterGroundConstraint>())
		{
			VisualizationContext.DataSelectionHandle = SelectionHandle;
			DrawConstraint(Component, *Constraint, VisualizationContext, View, PDI);
		}
	}
	else
	{
		for (const TSharedPtr<FChaosVDConstraintDataWrapperBase>& Constraint : ConstraintDataComponent->GetAllConstraints())
		{
			static_assert(std::is_base_of_v<FChaosVDConstraintDataWrapperBase, FChaosVDCharacterGroundConstraint>, "Only FChaosVDCharacterGroundConstraint is supported");
			if (FChaosVDCharacterGroundConstraint* CharacterGroundConstraint = static_cast<FChaosVDCharacterGroundConstraint*>(Constraint.Get()))
			{
				VisualizationContext.DataSelectionHandle = SelectionObject->MakeSelectionHandle(StaticCastSharedPtr<FChaosVDCharacterGroundConstraint>(Constraint));
				DrawConstraint(Component, *CharacterGroundConstraint, VisualizationContext, View, PDI);
			}
		}
	}
}

bool FChaosVDCharacterGroundConstraintDataComponentVisualizer::CanHandleClick(const HChaosVDComponentVisProxy& VisProxy)
{
	return VisProxy.DataSelectionHandle && VisProxy.DataSelectionHandle->IsA<FChaosVDCharacterGroundConstraint>();
}

void FChaosVDCharacterGroundConstraintDataComponentVisualizer::DrawConstraint(const UActorComponent* Component, const FChaosVDCharacterGroundConstraint& InConstraintData, FChaosVDCharacterGroundConstraintVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::DrawDisabled))
	{
		if (InConstraintData.State.bDisabled)
		{
			return;
		}
	}

	if (!Component)
	{
		return;
	}

	const UChaosVDCharacterConstraintsVisualizationSettings* DebugDrawSettings =  Cast<UChaosVDCharacterConstraintsVisualizationSettings>(VisualizationContext.DebugDrawSettings);
	if (!DebugDrawSettings)
	{
		return;
	}

	if (!PDI)
	{
		return;
	}

	if (!View)
	{
		return;
	}

	TSharedPtr<const FChaosVDParticleDataWrapper> CharacterParticleData = nullptr;

	if (TSharedPtr<FChaosVDSceneParticle> CharacterParticle = VisualizationContext.SolverInfoActor->GetParticleInstance(InConstraintData.CharacterParticleIndex))
	{
		CharacterParticleData = CharacterParticle->GetParticleData();
	}

	if (!CharacterParticleData)
	{
		return;
	}

	if (!CharacterParticleData->ParticleMassProps.HasValidData())
	{
		// If we don't have mass data, all the following calculations will be off
		// TODO: Should we draw just a line between the two particles as fallback?
		return;
	}

	PDI->SetHitProxy(new HChaosVDComponentVisProxy(Component, VisualizationContext.DataSelectionHandle));

	const float LineThickness = VisualizationContext.DataSelectionHandle && VisualizationContext.DataSelectionHandle->IsSelected() ?  DebugDrawSettings->BaseLineThickness * 1.5f :  DebugDrawSettings->BaseLineThickness;

	const FVector CharacterPos = CharacterParticleData->ParticlePositionRotation.MX;
	const FVector UpDir = InConstraintData.Settings.VerticalAxis;

	const double GroundDistance = InConstraintData.Data.GroundDistance;
	const double TargetHeight = InConstraintData.Settings.TargetHeight;

	// TODO: The target data is valid for pre-sim positions and the force/torque for post sim
	// but we don't have a way of differentiating here which state the particle is in so just draw
	// everything for now and leave it up to the user to interpret what they're seeing

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::TargetDeltaPosition))
	{
		FChaosVDDebugDrawUtils::DrawArrowVector(PDI, CharacterPos, CharacterPos + InConstraintData.Data.TargetDeltaPosition, FText::GetEmpty(), FColor::Blue, DebugDrawSettings->DepthPriority, 0.5f * LineThickness);
	}

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::TargetDeltaFacing))
	{
		const FVector Forward = CharacterParticleData->ParticlePositionRotation.MR * FVector::XAxisVector * DebugDrawSettings->GeneralScale * 10.0f;
		const FVector TargetForward = FQuat(UpDir, InConstraintData.Data.TargetDeltaFacing) * Forward;
		FChaosVDDebugDrawUtils::DrawArrowVector(PDI, CharacterPos, CharacterPos + Forward, FText::GetEmpty(), FColor::Silver, DebugDrawSettings->DepthPriority, 0.25f * LineThickness);
		FChaosVDDebugDrawUtils::DrawArrowVector(PDI, CharacterPos, CharacterPos + TargetForward, FText::GetEmpty(), FColor::White, DebugDrawSettings->DepthPriority, 0.25f * LineThickness);
	}

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::GroundQueryDistance))
	{
		if (GroundDistance >= TargetHeight)
		{
			if (GroundDistance <= 4.0f * TargetHeight)
			{
				FChaosVDDebugDrawUtils::DrawLine(PDI, CharacterPos, CharacterPos - UpDir * TargetHeight, FColor::Green, FText::GetEmpty(), DebugDrawSettings->DepthPriority, LineThickness);
				FChaosVDDebugDrawUtils::DrawLine(PDI, CharacterPos - UpDir * TargetHeight, CharacterPos - UpDir * GroundDistance, FColor::Silver, FText::GetEmpty(), DebugDrawSettings->DepthPriority, LineThickness);
			}
		}
		else
		{
			FChaosVDDebugDrawUtils::DrawLine(PDI, CharacterPos, CharacterPos - UpDir * GroundDistance, FColor::Green, FText::GetEmpty(), DebugDrawSettings->DepthPriority, LineThickness);
			FChaosVDDebugDrawUtils::DrawLine(PDI, CharacterPos - UpDir * GroundDistance, CharacterPos - UpDir * TargetHeight, FColor::Red, FText::GetEmpty(), DebugDrawSettings->DepthPriority, LineThickness);
		}
	}

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::GroundQueryNormal))
	{
		if (GroundDistance < 4.0f * TargetHeight)
		{
			const FVector ScaledGroundNormal = 10.0f * InConstraintData.Data.GroundNormal * DebugDrawSettings->GeneralScale;
			const FVector GroundPos = CharacterPos - UpDir * GroundDistance;
			FChaosVDDebugDrawUtils::DrawArrowVector(PDI, GroundPos, GroundPos + ScaledGroundNormal, FText::GetEmpty(), FColor::Cyan, DebugDrawSettings->DepthPriority, 0.25f * LineThickness);
		}
	}
	
	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::AppliedNormalForce))
	{
		const FVector NormalForce = DebugDrawSettings->ForceScale * InConstraintData.Data.GroundNormal.Dot(InConstraintData.State.SolverAppliedForce) * InConstraintData.Data.GroundNormal;
		FChaosVDDebugDrawUtils::DrawArrowVector(PDI, CharacterPos, CharacterPos + NormalForce, FText::GetEmpty(), DebugDrawSettings->NormalForceColor, DebugDrawSettings->DepthPriority, LineThickness);
	}

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::AppliedRadialForce))
	{
		const FVector RadialForce = DebugDrawSettings->ForceScale * (InConstraintData.State.SolverAppliedForce - InConstraintData.Data.GroundNormal.Dot(InConstraintData.State.SolverAppliedForce) * InConstraintData.Data.GroundNormal);
		FChaosVDDebugDrawUtils::DrawArrowVector(PDI, CharacterPos, CharacterPos + RadialForce, FText::GetEmpty(), DebugDrawSettings->NormalForceColor, DebugDrawSettings->DepthPriority, LineThickness);
	}

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags::AppliedTorque))
	{
		const FVector Torque = DebugDrawSettings->TorqueScale * InConstraintData.State.SolverAppliedTorque;
		FChaosVDDebugDrawUtils::DrawArrowVector(PDI, CharacterPos, CharacterPos + Torque, FText::GetEmpty(), DebugDrawSettings->TorqueColor, DebugDrawSettings->DepthPriority, LineThickness);
	}

	PDI->SetHitProxy(nullptr);
}

#undef LOCTEXT_NAMESPACE
