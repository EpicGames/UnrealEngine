// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Visualizers/ChaosVDComponentVisualizerBase.h"

#include "Chaos/Core.h"

struct FChaosVDSolverDataSelectionHandle;
class AChaosVDSolverInfoActor;
enum class EChaosVDCharacterGroundConstraintDataVisualizationFlags : uint32;
struct FChaosVDCharacterGroundConstraintDebugDrawSettings;
struct FChaosVDCharacterGroundConstraint;

/** Visualization context structure specific for character ground constraint visualizations */
struct FChaosVDCharacterGroundConstraintVisualizationDataContext : public FChaosVDVisualizationContext
{
	TSharedPtr<FChaosVDSolverDataSelectionHandle> DataSelectionHandle = nullptr;
	
	AChaosVDSolverInfoActor* SolverInfoActor = nullptr;

	bool IsVisualizationFlagEnabled(EChaosVDCharacterGroundConstraintDataVisualizationFlags Flag) const;
};

/**
 * Component visualizer in charge of generating debug draw visualizations for character ground constraints in a UChaosVDSolverCharacterGroundConstraintDataComponent
 */
class FChaosVDCharacterGroundConstraintDataComponentVisualizer final : public FChaosVDComponentVisualizerBase
{
public:
	FChaosVDCharacterGroundConstraintDataComponentVisualizer();

	virtual void RegisterVisualizerMenus() override;

	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;

	virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy) override;

protected:

	void DrawConstraint(const UActorComponent* Component, const FChaosVDCharacterGroundConstraint& InConstraintData, FChaosVDCharacterGroundConstraintVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI);
};
