// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/ChaosVDComponentVisualizerBase.h"

#include "ChaosVDEngine.h"
#include "ChaosVDScene.h"
#include "ChaosVDSolverDataSelection.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "Widgets/SChaosVDMainTab.h"

IMPLEMENT_HIT_PROXY(HChaosVDComponentVisProxy, HComponentVisProxy)


bool FChaosVDComponentVisualizerBase::CanHandleClick(const HChaosVDComponentVisProxy& VisProxy)
{
	return false;
}

bool FChaosVDComponentVisualizerBase::VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click)
{
	const HChaosVDComponentVisProxy* ChaosVDHitProxy = HitProxyCast<HChaosVDComponentVisProxy>(VisProxy);
	if (ChaosVDHitProxy == nullptr)
	{
		return false;
	}

	if (!CanHandleClick(*ChaosVDHitProxy))
	{
		return false;
	}

	const TSharedPtr<SChaosVDMainTab> MainTabToolkitHost = InViewportClient->GetModeTools() ? StaticCastSharedPtr<SChaosVDMainTab>(InViewportClient->GetModeTools()->GetToolkitHost()) : nullptr;
	if (!MainTabToolkitHost)
	{
		return false;
	}

	if (TSharedPtr<FChaosVDScene> CVDScene = MainTabToolkitHost->GetChaosVDEngineInstance()->GetCurrentScene())
	{
		// Bring the constraint details tab into focus if available
		if (const TSharedPtr<FTabManager> TabManager = MainTabToolkitHost ? MainTabToolkitHost->GetTabManager() : nullptr)
		{
			TabManager->TryInvokeTab(InspectorTabID);
		}

		return SelectVisualizedData(*ChaosVDHitProxy, CVDScene.ToSharedRef(), MainTabToolkitHost.ToSharedRef());
	}

	return false;
}

bool FChaosVDComponentVisualizerBase::SelectVisualizedData(const HChaosVDComponentVisProxy& VisProxy, const TSharedRef<FChaosVDScene>& InCVDScene, const TSharedRef<SChaosVDMainTab>& InMainTabToolkitHost)
{
	if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = InCVDScene->GetSolverDataSelectionObject().Pin())
	{
		SelectionObject->SelectData(VisProxy.DataSelectionHandle);

		return true;
	}

	return false;
}

