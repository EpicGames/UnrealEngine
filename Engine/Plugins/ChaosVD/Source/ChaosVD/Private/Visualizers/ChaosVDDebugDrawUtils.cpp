// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/ChaosVDDebugDrawUtils.h"

#include "CanvasItem.h"
#include "ChaosVDGeometryBuilder.h"
#include "Generators/MeshShapeGenerator.h"
#include "Engine/Engine.h"
#include "SceneView.h"

TQueue<FChaosVDDebugDrawUtils::FChaosVDQueuedTextToDraw> FChaosVDDebugDrawUtils::TextToDrawQueue = TQueue<FChaosVDQueuedTextToDraw>();
int32 FChaosVDDebugDrawUtils::CurrentLinesDrawn = 0;
int32 FChaosVDDebugDrawUtils::MaxLinesToDrawPerFrame = 300000;
bool FChaosVDDebugDrawUtils::bIsShowingDebugDrawLimitWarning = false;
int32 FChaosVDDebugDrawUtils::CurrentWarningsBeingDrawn = 0;

namespace Chaos::VisualDebugger::Cvars
{
	static FAutoConsoleVariableRef CVarChaosVDMaxDebugDrawLinesPerFrame(
		TEXT("p.Chaos.VD.Tool.MaxDebugDrawLinesPerFrame"),
		FChaosVDDebugDrawUtils::MaxLinesToDrawPerFrame,
		TEXT("Sets the max number of lines CVD is allowed to draw between all instances in a single frame."));
}

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void FChaosVDDebugDrawUtils::DrawArrowVector(FPrimitiveDrawInterface* PDI, const FVector& StartLocation, const FVector& EndLocation, const FText& InDebugText, const FColor& Color, ESceneDepthPriorityGroup DepthPriority, float Thickness)
{
	if (!PDI)
	{
		return;
	}

	if (!CanDebugDraw())
	{
		return;
	}

	const FVector LineVectorToDraw = EndLocation - StartLocation;

	FVector ArrowDir;
	float ArrowLength;
	LineVectorToDraw.ToDirectionAndLength(ArrowDir, ArrowLength);

	FVector YAxis, ZAxis;
	ArrowDir.FindBestAxisVectors(YAxis,ZAxis);
	const FMatrix ArrowTransformMatrix(ArrowDir, YAxis, ZAxis,StartLocation);

	constexpr float MinTipOfArrowSize = 0.2f;
	constexpr float MaxTipOfArrowSize = 10.0f;
	constexpr float MaxVectorSizeForArrow = 100.0f; // The vector size that is the upper limit after which we just use the max size for the tip of the arrow

	const float ProportionalArrowSize = MaxTipOfArrowSize * (ArrowLength / MaxVectorSizeForArrow);
	const float ArrowSize = FMath::Clamp(ProportionalArrowSize, MinTipOfArrowSize, MaxTipOfArrowSize);
	
	DrawDirectionalArrow(PDI, ArrowTransformMatrix, Color, ArrowLength, ArrowSize, static_cast<uint8>(DepthPriority), Thickness);

	IncreaseDebugDrawLineCounter();

	if (!InDebugText.IsEmpty())
	{
		// Draw the text in the middle of the vector line
		const FVector TextWorldPosition = StartLocation + LineVectorToDraw  * 0.5f;
		DrawText(InDebugText, TextWorldPosition , Color);
	}
}

void FChaosVDDebugDrawUtils::DrawPoint(FPrimitiveDrawInterface* PDI, const FVector& Location, const FText& InDebugText, const FColor& Color, float Size, ESceneDepthPriorityGroup DepthPriority)
{
	if (!PDI)
	{
		return;
	}

	if (InDebugText.IsEmpty())
	{
		return;
	}

	if (!CanDebugDraw())
	{
		return;
	}

	PDI->DrawPoint(Location, Color, Size, static_cast<uint8>(DepthPriority));

	if (!InDebugText.IsEmpty())
	{
		DrawText(InDebugText, Location, Color);
	}
}

void FChaosVDDebugDrawUtils::DrawString(FStringView StringToDraw, const FVector& Location, const FColor& Color, EChaosVDDebugDrawTextLocationMode LocationMode)
{
	if (!StringToDraw.IsEmpty())
	{
		TextToDrawQueue.Enqueue({ FText::AsCultureInvariant(StringToDraw.GetData()), LocationMode, Location, Color });
	}
}

void FChaosVDDebugDrawUtils::DrawText(const FText& InText, const FVector& Location, const FColor& Color, EChaosVDDebugDrawTextLocationMode LocationMode)
{
	if (!InText.IsEmptyOrWhitespace())
	{
		TextToDrawQueue.Enqueue({ InText, LocationMode, Location, Color });
	}
}

void FChaosVDDebugDrawUtils::DrawOnScreenWarning(const FText& InText, const FColor& Color)
{
	if (!GEngine)
	{
		return;
	}

	static FVector WarningStartPosition = FVector(45.0, 40.0 ,0.0);
	constexpr int32 LineSpaceY = 20;
	CurrentWarningsBeingDrawn++;

	const FVector WarningPosition = WarningStartPosition + FVector(0.0, LineSpaceY * CurrentWarningsBeingDrawn,0.0);
	TextToDrawQueue.Enqueue({ InText, EChaosVDDebugDrawTextLocationMode::Screen, WarningPosition, Color });
}

void FChaosVDDebugDrawUtils::DrawCircle(FPrimitiveDrawInterface* PDI, const FVector& Origin, float Radius, int32 Segments, const FColor& Color, float Thickness, const FVector& XAxis, const FVector& YAxis, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority)
{
	if (!PDI)
	{
		return;
	}

	if (!CanDebugDraw())
	{
		return;
	}

	constexpr float DepthBias = 0.0f;
	const bool bScreenSpace = Thickness > 0;

	// Need at least 2 sides
	Segments = FMath::Max(Segments, 2);
	const float	AngleDelta = 2.0f * UE_PI / static_cast<float>(Segments);
	FVector	LastVertex = Origin + XAxis * Radius;

	PDI->AddReserveLines(static_cast<uint8>(DepthPriority), Segments, false, Thickness > SMALL_NUMBER);

	for (int32 SideIndex = 0; SideIndex < Segments; SideIndex++)
	{
		const float NextSideIndex = static_cast<float>(SideIndex + 1);
		const FVector Vertex = Origin + (XAxis * FMath::Cos(AngleDelta * NextSideIndex) + YAxis * FMath::Sin(AngleDelta * NextSideIndex)) * Radius;

		PDI->DrawLine(LastVertex, Vertex, Color, static_cast<uint8>(DepthPriority), Thickness, DepthBias, bScreenSpace);

		LastVertex = Vertex;
	}
	
	if (!InDebugText.IsEmpty())
	{
		DrawText(InDebugText, Origin, Color);
	}
}

void FChaosVDDebugDrawUtils::DrawBox(FPrimitiveDrawInterface* PDI, const FVector& InExtents, const FColor& InColor, const FTransform& InTransform, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority, float Thickness)
{
	if (!PDI)
	{
		return;
	}

	if (!CanDebugDraw())
	{
		return;
	}

	constexpr int32 MaxBoxLines = 12;
	
	PDI->AddReserveLines(static_cast<uint8>(DepthPriority), MaxBoxLines, false, Thickness > SMALL_NUMBER);

	// Array for direction offsets for the start/end point of each line
	static TPair<FVector, FVector> VertexOffsetDirectionFromOrigin[MaxBoxLines] =
	{
		{FVector(1,1,1), FVector(1,-1,1)},
		{FVector(1,-1,1) ,FVector(-1,-1,1)},
		{FVector(-1,-1,1), FVector(-1,1,1)},
		{FVector(-1,1,1), FVector(1,1,1)},
		{FVector(1,1,-1), FVector(1,-1,-1)},
		{FVector(1, -1,-1), FVector(-1,-1,-1)},
		{FVector(-1,-1,-1), FVector(-1,1,-1)},
		{FVector(-1,1,-1), FVector(1,1,-1)},
		{FVector(1,1,1), FVector(1,1,-1)},
		{FVector(1,-1,1), FVector(1,-1,-1)},
		{FVector(-1,-1,1), FVector(-1,-1,-1)},
		{FVector(-1, 1,1), FVector(-1,1,-1)},
	};

	constexpr float DepthBias = 0.0f;
	const bool bScreenSpace = Thickness > 0;

	for (int32 BoxLineIndex = 0; BoxLineIndex < MaxBoxLines; BoxLineIndex++)
	{
		FVector LineStart = InTransform.TransformPosition(InExtents * VertexOffsetDirectionFromOrigin[BoxLineIndex].Key);
		FVector LineEnd = InTransform.TransformPosition(InExtents * VertexOffsetDirectionFromOrigin[BoxLineIndex].Value);

		PDI->DrawLine(LineStart, LineEnd, InColor, static_cast<uint8>(DepthPriority), Thickness,DepthBias, bScreenSpace);
	}
	
	if (!InDebugText.IsEmpty())
	{
		DrawText(InDebugText, InTransform.GetLocation(), InColor);
	}
}

void FChaosVDDebugDrawUtils::DrawLine(FPrimitiveDrawInterface* PDI, const FVector& InStartPosition, const FVector& InEndPosition, const FColor& InColor, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority, float Thickness)
{
	if (!PDI)
	{
		return;
	}

	if (!CanDebugDraw())
	{
		return;
	}

	constexpr float DepthBias = 0.0f;
	const bool bScreenSpace = Thickness > 0;

	PDI->DrawLine(InStartPosition, InEndPosition, InColor, static_cast<uint8>(DepthPriority), Thickness, DepthBias, bScreenSpace);

	IncreaseDebugDrawLineCounter();

	if (!InDebugText.IsEmpty())
	{
		// Draw the text in the middle of the line
		const FVector TextWorldPosition = InStartPosition + ((InEndPosition - InStartPosition)  * 0.5f);
		DrawText(InDebugText, TextWorldPosition, InColor);
	}
}

void FChaosVDDebugDrawUtils::DrawImplicitObject(FPrimitiveDrawInterface* PDI, const TSharedPtr<FChaosVDGeometryBuilder>& GeometryGenerator, const Chaos::FConstImplicitObjectPtr& ImplicitObject, const FTransform& InWorldTransform, const FColor& InColor, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority, float Thickness)
{
	if (!PDI)
	{
		return;
	}

	if (!CanDebugDraw())
	{
		return;
	}

	if (!ImplicitObject.IsValid())
	{
	  return;
	}

	using namespace Chaos;
	const EImplicitObjectType InnerType = GetInnerType(ImplicitObject->GetType());
		
	if (InnerType == ImplicitObjectType::Union || InnerType == ImplicitObjectType::UnionClustered)
	{
		if (const FImplicitObjectUnion* Union = ImplicitObject->template AsA<FImplicitObjectUnion>())
		{
			for (int32 ObjectIndex = 0; ObjectIndex < Union->GetObjects().Num(); ++ObjectIndex)
			{
				const FImplicitObjectPtr& UnionImplicit = Union->GetObjects()[ObjectIndex];
				DrawImplicitObject(PDI, GeometryGenerator, UnionImplicit.GetReference(), InWorldTransform, InColor, InDebugText, DepthPriority, Thickness);
			}
		}

		return;
	}

	if (InnerType == ImplicitObjectType::Transformed)
	{
		if (const TImplicitObjectTransformed<FReal, 3>* Transformed = ImplicitObject->template GetObject<TImplicitObjectTransformed<FReal, 3>>())
		{
			DrawImplicitObject(PDI, GeometryGenerator, Transformed->GetTransformedObject(), Transformed->GetTransform() * InWorldTransform, InColor, InDebugText, DepthPriority, Thickness);
		}
		
		return;
	}

	constexpr float SimpleShapesComplexityFactor = 0.5f;

	FRigidTransform3 ExtractedTransform = InWorldTransform;
	const bool bNeedsUnpack = GeometryGenerator->ImplicitObjectNeedsUnpacking(ImplicitObject);
	const FImplicitObject* ImplicitObjectToProcess = bNeedsUnpack ? GeometryGenerator->UnpackImplicitObject(ImplicitObject, ExtractedTransform) : ImplicitObject.GetReference();

	if (const TSharedPtr<UE::Geometry::FMeshShapeGenerator> MeshGenerator = GeometryGenerator->CreateMeshGeneratorForImplicitObject(ImplicitObjectToProcess, SimpleShapesComplexityFactor))
	{
		GeometryGenerator->AdjustedTransformForImplicit(ImplicitObject, ExtractedTransform);
		MeshGenerator->Generate();

		PDI->AddReserveLines(static_cast<uint8>(DepthPriority), MeshGenerator->Triangles.Num() * 3, false, Thickness > SMALL_NUMBER);

		for (const UE::Geometry::FIndex3i& Triangle : MeshGenerator->Triangles)
		{
			FVector VertexA = ExtractedTransform.TransformPosition(MeshGenerator->Vertices[Triangle.A]);
			FVector VertexB = ExtractedTransform.TransformPosition(MeshGenerator->Vertices[Triangle.B]);
			FVector VertexC = ExtractedTransform.TransformPosition(MeshGenerator->Vertices[Triangle.C]);
	
			DrawLine(PDI, VertexA, VertexB, InColor, FText::GetEmpty(), DepthPriority, Thickness);
			DrawLine(PDI, VertexB, VertexC, InColor, FText::GetEmpty(), DepthPriority, Thickness);
			DrawLine(PDI, VertexC, VertexA, InColor, FText::GetEmpty(), DepthPriority, Thickness);
		}	
	}

	if (!InDebugText.IsEmpty())
	{
		DrawText(InDebugText, InWorldTransform.GetLocation(), InColor);
	}
}

void FChaosVDDebugDrawUtils::DrawSphere(FPrimitiveDrawInterface* PDI, const FVector& Center, float Radius, int32 Segments, const FColor& InColor, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority, float Thickness)
{
	// TODO: This implementation is taken as is from ULineBatchComponent::DrawSphere. At some point in the future we want to migrate CVD to use the line batcher directly, but we need to add support for hit proxies

	// Need at least 4 segments
	Segments = FMath::Max(Segments, 4);

	const float AngleInc = 2.f * UE_PI / static_cast<float>(Segments);
	int32 NumSegmentsY = Segments;
	float Latitude = AngleInc;
	float SinY1 = 0.0f, CosY1 = 1.0f;

	PDI->AddReserveLines(static_cast<uint8>(DepthPriority), NumSegmentsY * Segments * 2, false, Thickness > SMALL_NUMBER);
	while (NumSegmentsY--)
	{
		const float SinY2 = FMath::Sin(Latitude);
		const float CosY2 = FMath::Cos(Latitude);

		FVector Vertex1 = FVector(SinY1, 0.0f, CosY1) * Radius + Center;
		FVector Vertex3 = FVector(SinY2, 0.0f, CosY2) * Radius + Center;
		float Longitude = AngleInc;

		int32 NumSegmentsX = Segments;
		while (NumSegmentsX--)
		{
			const float SinX = FMath::Sin(Longitude);
			const float CosX = FMath::Cos(Longitude);

			const FVector Vertex2 = FVector((CosX * SinY1), (SinX * SinY1), CosY1) * Radius + Center;
			const FVector Vertex4 = FVector((CosX * SinY2), (SinX * SinY2), CosY2) * Radius + Center;

			DrawLine(PDI, Vertex1, Vertex2, InColor, FText::GetEmpty(), DepthPriority, Thickness);
			DrawLine(PDI, Vertex1, Vertex3, InColor, FText::GetEmpty(), DepthPriority, Thickness);

			Vertex1 = Vertex2;
			Vertex3 = Vertex4;
			Longitude += AngleInc;
		}
		SinY1 = SinY2;
		CosY1 = CosY2;
		Latitude += AngleInc;
	}

	if (!InDebugText.IsEmpty())
	{
		DrawText(InDebugText, Center, InColor);
	}
}

void FChaosVDDebugDrawUtils::DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas)
{
	if (!GEngine)
	{
		return;
	}

	while (!TextToDrawQueue.IsEmpty())
	{
		FChaosVDQueuedTextToDraw TextToDraw;
		if (TextToDrawQueue.Dequeue(TextToDraw))
		{
			FVector2D LocationToDraw;
			bool bHasValidLocation = false;
			switch (TextToDraw.LocationMode)
			{
				case EChaosVDDebugDrawTextLocationMode::World:
				{
					bHasValidLocation = View.ViewFrustum.IntersectPoint(TextToDraw.Location) && View.WorldToPixel(TextToDraw.Location, LocationToDraw);
					LocationToDraw /= View.Family->DebugDPIScale;
					break;
				}
				case EChaosVDDebugDrawTextLocationMode::Screen:
				{
					LocationToDraw = FVector2D(TextToDraw.Location.X, TextToDraw.Location.Y);
					bHasValidLocation = true;
				}
				default:
					break;
			}
			
			if (bHasValidLocation)
			{
				FCanvasTextItem TextItem(LocationToDraw, TextToDraw.Text, GEngine->GetSmallFont(), TextToDraw.Color);
				TextItem.Scale = FVector2D::UnitVector;
				TextItem.EnableShadow(FLinearColor::Black);
				TextItem.Draw(&Canvas);
			}
		}
	}
}

bool FChaosVDDebugDrawUtils::CanDebugDraw()
{
	return CurrentLinesDrawn <= MaxLinesToDrawPerFrame;
}

void FChaosVDDebugDrawUtils::DebugDrawFrameEnd()
{
	bIsShowingDebugDrawLimitWarning = false;
	CurrentLinesDrawn = 0;
	CurrentWarningsBeingDrawn = 0;
}

void FChaosVDDebugDrawUtils::IncreaseDebugDrawLineCounter()
{
	CurrentLinesDrawn++;

	if (!CanDebugDraw() && !bIsShowingDebugDrawLimitWarning)
	{
		bIsShowingDebugDrawLimitWarning = true;

		DrawOnScreenWarning(LOCTEXT("DebugDrawLimitWarning", "Max Debug Draw lines limit reached!. Try selecting fewer debug draw categories of focus the camera in a narrower area."), FColor::Yellow);
	}
}

FString Chaos::VisualDebugger::Utils::GenerateDebugTextForVector(const FVector& InVector, const FString& VectorName, const FString& InVectorUnits)
{
	return FString::Format(TEXT("{5} : {0} {4} \n	|-- X : {1} {4} \n	|-- Y : {2} {4} \n	|-- Z : {3} {4}"), {InVector.Size(), InVector.X, InVector.Y, InVector.Z, InVectorUnits, VectorName });
}

FBox Chaos::VisualDebugger::Utils::CalculateSceneQueryShapeBounds(const TSharedRef<FChaosVDQueryDataWrapper>& InSceneQueryData, const TSharedRef<FChaosVDRecording> InRecordedData)
{
	FBoxSphereBounds::Builder BoundsBuilder;
	const FConstImplicitObjectPtr* InputShapePtrPtr = InRecordedData->GetGeometryMap().Find(InSceneQueryData->InputGeometryKey);
	const FConstImplicitObjectPtr InputShapePtr = InputShapePtrPtr ? *InputShapePtrPtr : nullptr;
	
	if (InputShapePtr && InputShapePtr->HasBoundingBox())
	{
		FAABB3 StartBounds = InputShapePtr->CalculateTransformedBounds(FRigidTransform3(InSceneQueryData->StartLocation, InSceneQueryData->GeometryOrientation));
		BoundsBuilder += FBox(StartBounds.Min(), StartBounds.Max());

		if (InSceneQueryData->Type != EChaosVDSceneQueryType::Overlap)
		{
			FAABB3 EndBounds = InputShapePtr->CalculateTransformedBounds(FRigidTransform3(InSceneQueryData->EndLocation, InSceneQueryData->GeometryOrientation));
			BoundsBuilder += FBox(EndBounds.Min(), EndBounds.Max());
		}
	}
	else
	{
		BoundsBuilder+= InSceneQueryData->EndLocation;
		BoundsBuilder+= InSceneQueryData->StartLocation;
	}
	
	return FBoxSphereBounds(BoundsBuilder).GetBox();
}

#undef LOCTEXT_NAMESPACE
