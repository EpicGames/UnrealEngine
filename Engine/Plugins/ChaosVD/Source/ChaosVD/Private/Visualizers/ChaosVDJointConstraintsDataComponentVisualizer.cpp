// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/ChaosVDJointConstraintsDataComponentVisualizer.h"

#include "ChaosVDEngine.h"
#include "ChaosVDScene.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "EditorViewportClient.h"
#include "PrimitiveDrawingUtils.h"
#include "SceneView.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Components/ChaosVDSolverJointConstraintDataComponent.h"
#include "Settings/ChaosVDJointConstraintVisualizationSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Utils/ChaosVDMenus.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

namespace Chaos::VisualDebugger::Utils
{
	FVector GetCoMWorldPosition(const FChaosVDParticleDataWrapper& ParticleData)
	{
		if (ensure(ParticleData.ParticlePositionRotation.HasValidData() && ParticleData.ParticleMassProps.HasValidData()))
		{
			return ParticleData.ParticlePositionRotation.MX + ParticleData.ParticlePositionRotation.MR.RotateVector(ParticleData.ParticleMassProps.MCenterOfMass);
		}

		return FVector::ZeroVector;
	}

	FQuat GetCoMWorldRotation(const FChaosVDParticleDataWrapper& ParticleData)
	{
		if (ensure(ParticleData.ParticlePositionRotation.HasValidData() && ParticleData.ParticleMassProps.HasValidData()))
		{
			return ParticleData.ParticlePositionRotation.MR * ParticleData.ParticleMassProps.MRotationOfMass;
		}

		return FQuat::Identity;
	}

	FVector ParticleLocalToCoMLocal(const FChaosVDParticleDataWrapper& ParticleData, const FVector& Position)
	{
		if (ensure(ParticleData.ParticleMassProps.HasValidData()))
		{
			return ParticleData.ParticleMassProps.MRotationOfMass.UnrotateVector(Position - ParticleData.ParticleMassProps.MCenterOfMass);
		}

		return FVector::ZeroVector;
	}

	FQuat ParticleLocalToCoMLocal(const FChaosVDParticleDataWrapper& ParticleData, const FQuat& Rotation)
	{
		if (ensure(ParticleData.ParticleMassProps.HasValidData()))
		{
			return ParticleData.ParticleMassProps.MRotationOfMass.Inverse()* Rotation;
		}

		return FQuat::Identity;
	}

	FTransform ParticleLocalToCoMLocal(const FChaosVDParticleDataWrapper& ParticleData, const FTransform& Transform)
	{
		return FTransform(ParticleLocalToCoMLocal(ParticleData, Transform.GetRotation()), ParticleLocalToCoMLocal(ParticleData, Transform.GetTranslation()));
	}

	void CalculateConstraintSpace(const FChaosVDParticleDataWrapper& ParticleData0, const FChaosVDParticleDataWrapper& ParticleData1, const FChaosVDJointConstraint& InJointConstraintData, FVector& OutX0, FMatrix33& OutR0, FVector& OutX1, FMatrix33& OutR1)
	{
		// Replicating what is done in FPBDJointConstraints::CalculateConstraintSpace. There invert the index for the particles. As we are not dealing with particle indexes here, just invert the references so Particle 0 is now 1 and viceversa 
		// Copying the explanation given in that method
		// "In solvers we need Particle0 to be the parent particle but ConstraintInstance has Particle1 as the parent, so by default
		// we need to flip the indices before we pass them to the solver."

		const FChaosVDParticleDataWrapper& ParticleDataToEvaluate0 = ParticleData1;
		const FChaosVDParticleDataWrapper& ParticleDataToEvaluate1 = ParticleData0;

		const FVector P0 = GetCoMWorldPosition(ParticleDataToEvaluate0);
		const FRotation3 Q0 = GetCoMWorldRotation(ParticleDataToEvaluate0);
		const FVector P1 = GetCoMWorldPosition(ParticleDataToEvaluate1);
		const FRotation3 Q1 = GetCoMWorldRotation(ParticleDataToEvaluate1);
		const FRigidTransform3& XL0 = ParticleLocalToCoMLocal(ParticleDataToEvaluate0, InJointConstraintData.JointSettings.ConnectorTransforms[1]);
		const FRigidTransform3& XL1 = ParticleLocalToCoMLocal(ParticleDataToEvaluate1,  InJointConstraintData.JointSettings.ConnectorTransforms[0]);

		OutX0 = P0 + Q0 * XL0.GetTranslation();
		OutX1 = P1 + Q1 * XL1.GetTranslation();
		OutR0 = FRotation3(Q0 * XL0.GetRotation()).ToMatrix();
		OutR1 = FRotation3(Q1 * XL1.GetRotation()).ToMatrix();
	}

	/** Generates a random color based on the selection state and query ID, which will be used to debug draw the scene query */
	FColor GenerateSelectionAwareDebugColor(FLinearColor InBaseColor, bool bIsSelected)
	{
		return bIsSelected ? InBaseColor.ToFColorSRGB() :  (InBaseColor * 0.75f).ToFColorSRGB();
	}

	bool IsParticleSleepingOrDynamic(const FChaosVDParticleDataWrapper& ParticleData)
	{
		return ParticleData.ParticleDynamicsMisc.MObjectState == EChaosVDObjectStateType::Dynamic || ParticleData.ParticleDynamicsMisc.MObjectState == EChaosVDObjectStateType::Sleeping;
	}
}

FChaosVDJointConstraintsDataComponentVisualizer::FChaosVDJointConstraintsDataComponentVisualizer()
{
	FChaosVDJointConstraintsDataComponentVisualizer::RegisterVisualizerMenus();

	InspectorTabID = FChaosVDTabID::ConstraintsInspector;
}

void FChaosVDJointConstraintsDataComponentVisualizer::RegisterVisualizerMenus()
{
	FName MenuSection("JointConstraintDataVisualization.Show");
	FText MenuSectionLabel = LOCTEXT("JointConstraintDataVisualizationShowMenuLabel", "Joint Constraint Data Visualization");
	FText FlagsMenuLabel = LOCTEXT("JointConstraintDataVisualizationFlagsMenuLabel", "Joint Constraints Data Flags");
	FText FlagsMenuTooltip = LOCTEXT("JointConstraintDataVisualizationFlagsMenuToolTip", "Set of flags to enable/disable visibility of specific types of joint constraint data");
	FSlateIcon FlagsMenuIcon = FSlateIcon(FChaosVDStyle::Get().GetStyleSetName(), TEXT("ConnectionIcon"));

	FText SettingsMenuLabel = LOCTEXT("JointConstraintDataVisualizationMenuLabel", "Joint Constraint Visualization Settings");
	FText SettingsMenuTooltip = LOCTEXT("JointConstraintDataVisualizationMenuToolTip", "Options to change how the recorded joint constraint data is debug drawn");
	
	CreateGenericVisualizerMenu<UChaosVDJointConstraintsVisualizationSettings, EChaosVDJointsDataVisualizationFlags>(Chaos::VisualDebugger::Menus::ShowMenuName, MenuSection, MenuSectionLabel, FlagsMenuLabel, FlagsMenuTooltip, FlagsMenuIcon, SettingsMenuLabel, SettingsMenuTooltip);
}

void FChaosVDJointConstraintsDataComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDSolverJointConstraintDataComponent* JointConstraintDataComponent = Cast<UChaosVDSolverJointConstraintDataComponent>(Component);
	if (!JointConstraintDataComponent)
	{
		return;
	}
	
	AChaosVDSolverInfoActor* SolverInfoActor = Cast<AChaosVDSolverInfoActor>(Component->GetOwner());
	if (!SolverInfoActor)
	{
		return;
	}

	if (!SolverInfoActor->IsVisible())
	{
		return;
	}

	const TSharedPtr<FChaosVDScene> CVDScene = SolverInfoActor->GetScene().Pin();
	if (!CVDScene)
	{
		return;
	}

	const TSharedPtr<FChaosVDRecording> CVDRecording = CVDScene->GetLoadedRecording();
	if (!CVDRecording)
	{
		return;
	}

	FChaosVDJointVisualizationDataContext VisualizationContext;
	VisualizationContext.CVDScene = CVDScene;
	VisualizationContext.SpaceTransform = SolverInfoActor->GetSimulationTransform();
	VisualizationContext.SolverInfoActor = SolverInfoActor;

	if (const UChaosVDJointConstraintsVisualizationSettings* EditorSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDJointConstraintsVisualizationSettings>())
	{
		VisualizationContext.VisualizationFlags = static_cast<uint32>(UChaosVDJointConstraintsVisualizationSettings::GetDataVisualizationFlags());
		VisualizationContext.bShowDebugText = EditorSettings->bShowDebugText;
		VisualizationContext.DebugDrawSettings = EditorSettings;
	}

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::EnableDraw))
	{
		return;
	}

	TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = CVDScene->GetSolverDataSelectionObject().Pin();
	TSharedPtr<FChaosVDSolverDataSelectionHandle> SelectionHandle;
	if (SelectionObject)
	{
		SelectionHandle = SelectionObject->GetCurrentSelectionHandle();
	}
	
	// If nothing is selected, fallback to draw all 
	const bool bDrawOnlySelected = VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::OnlyDrawSelected) && SelectionHandle && SelectionHandle->IsSelected();
	if (bDrawOnlySelected)
	{
		if (FChaosVDJointConstraint* JointConstraintData = SelectionHandle->GetData<FChaosVDJointConstraint>())
		{
			VisualizationContext.DataSelectionHandle = SelectionHandle;
			DrawJointConstraint(Component, *JointConstraintData, VisualizationContext, View, PDI);
		}
	}
	else
	{
		for (const TSharedPtr<FChaosVDConstraintDataWrapperBase>& Constraint : JointConstraintDataComponent->GetAllConstraints())
		{
			static_assert(std::is_base_of_v<FChaosVDConstraintDataWrapperBase, FChaosVDJointConstraint>, "Only FChaosVDJointConstraint is supported");

			if (FChaosVDJointConstraint* JointConstraint = static_cast<FChaosVDJointConstraint*>(Constraint.Get()))
			{
				VisualizationContext.DataSelectionHandle = SelectionObject->MakeSelectionHandle(StaticCastSharedPtr<FChaosVDJointConstraint>(Constraint));

				DrawJointConstraint(Component, *JointConstraint, VisualizationContext, View, PDI);
			}
		}
	}
}

bool FChaosVDJointConstraintsDataComponentVisualizer::CanHandleClick(const HChaosVDComponentVisProxy& VisProxy)
{
	return VisProxy.DataSelectionHandle && VisProxy.DataSelectionHandle->IsA<FChaosVDJointConstraint>();
}

void FChaosVDJointConstraintsDataComponentVisualizer::DebugDrawAllAxis(const FChaosVDJointConstraint& InJointConstraintData, FChaosVDJointVisualizationDataContext& VisualizationContext, FPrimitiveDrawInterface* PDI, const float LineThickness, const FVector& InPosition, const Chaos::FMatrix33& InRotationMatrix, TConstArrayView<FLinearColor> AxisColors,  bool bIsSelected)
{
	for(int32 AxisIndex = 0; AxisIndex < 3 ; AxisIndex++)
	{
		FChaosVDDebugDrawUtils::DrawArrowVector(PDI, InPosition, InPosition + VisualizationContext.DebugDrawSettings->GeneralScale * VisualizationContext.DebugDrawSettings->ConstraintAxisLength * VisualizationContext.SpaceTransform.TransformVector(InRotationMatrix.GetAxis(AxisIndex)), FText::GetEmpty(), Chaos::VisualDebugger::Utils::GenerateSelectionAwareDebugColor(AxisColors[AxisIndex], bIsSelected), VisualizationContext.DebugDrawSettings->DepthPriority, LineThickness * 0.2f);
	}
}

void FChaosVDJointConstraintsDataComponentVisualizer::DrawJointConstraint(const UActorComponent* Component, const FChaosVDJointConstraint& InJointConstraintData, FChaosVDJointVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	using namespace Chaos::VisualDebugger::Utils;

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::DrawDisabled))
	{
		if (InJointConstraintData.PhysicsThreadJointState.bDisabled)
		{
			return;
		}
	}

	if (!Component)
	{
		return;
	}
	
	if (!VisualizationContext.DebugDrawSettings)
	{
		return;
	}

	if (!PDI)
	{
		return;
	}

	if (!View)
	{
		return;
	}

	TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData0 = nullptr;
	TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData1 = nullptr;

	if (TSharedPtr<FChaosVDSceneParticle> Particle0 = VisualizationContext.SolverInfoActor->GetParticleInstance(InJointConstraintData.ParticleParIndexes[0]))
	{
		ParticleData0 = Particle0->GetParticleData();
	}

	if (TSharedPtr<FChaosVDSceneParticle> Particle1 = VisualizationContext.SolverInfoActor->GetParticleInstance(InJointConstraintData.ParticleParIndexes[1]))
	{
		ParticleData1 = Particle1->GetParticleData();
	}

	if (!ParticleData0 || !ParticleData1)
	{
		return;
	}

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::DrawKinematic))
	{
		if (!IsParticleSleepingOrDynamic(*ParticleData0) || !IsParticleSleepingOrDynamic(*ParticleData1))
		{
			return;
		}
	}

	if (!ParticleData0->ParticleMassProps.HasValidData() || !ParticleData1->ParticleMassProps.HasValidData())
	{
		// If we don't have mass data, all the following calculations will be off
		// TODO: Should we draw just a line between the two particles as fallback?
		return;
	}

	// Create a sphere containing the two particle positions to use as a pseudo bounds to determine if we should draw this joint data
	FVector DiameterSphereViewVector = ParticleData1->ParticlePositionRotation.MX - ParticleData0->ParticlePositionRotation.MX;
	float ViewRadius = static_cast<float>(DiameterSphereViewVector.Size() * 0.5f);
	FVector MiddleViewPoint = ParticleData0->ParticlePositionRotation.MX + DiameterSphereViewVector.GetSafeNormal() * ViewRadius;

	bool bCurrentViewContainsBothLocations = View->ViewFrustum.IntersectSphere(MiddleViewPoint, ViewRadius);
	if (!bCurrentViewContainsBothLocations)
	{
		return;
	}

	bool bIsSelected = VisualizationContext.DataSelectionHandle && VisualizationContext.DataSelectionHandle->IsSelected();

	PDI->SetHitProxy(new HChaosVDComponentVisProxy(Component, VisualizationContext.DataSelectionHandle));

	const float LineThickness = bIsSelected ? VisualizationContext.DebugDrawSettings->BaseLineThickness * 1.5f :  VisualizationContext.DebugDrawSettings->BaseLineThickness;

	FVector Pa = ParticleData1->ParticlePositionRotation.MX;
	FVector Pb = ParticleData0->ParticlePositionRotation.MX;
	FVector Ca = GetCoMWorldPosition(*ParticleData1);
	FVector Cb =  GetCoMWorldPosition(*ParticleData0);
	FVector Xa, Xb;
	Chaos::FMatrix33 Ra, Rb;
	CalculateConstraintSpace(*ParticleData0, *ParticleData1, InJointConstraintData, Xa, Ra, Xb, Rb);

	Pa = VisualizationContext.SpaceTransform.TransformPosition(Pa);
	Pb = VisualizationContext.SpaceTransform.TransformPosition(Pb);
	Ca = VisualizationContext.SpaceTransform.TransformPosition(Ca);
	Cb = VisualizationContext.SpaceTransform.TransformPosition(Cb);
	Xa = VisualizationContext.SpaceTransform.TransformPosition(Xa);
	Xb = VisualizationContext.SpaceTransform.TransformPosition(Xb);

	const FLinearColor Red = FLinearColor::Red;
	const FLinearColor Green = FLinearColor::Green;
	const FLinearColor Blue = FLinearColor::Blue;
	const FLinearColor Cyan = FLinearColor::FromSRGBColor(FColor::Cyan);
	const FLinearColor Magenta = FLinearColor::FromSRGBColor(FColor::Magenta);
	const FLinearColor Yellow = FLinearColor::Yellow;
	const FLinearColor White = FLinearColor::White;
	const FLinearColor Black = FLinearColor::Black;

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::ActorConnector))
	{
		const float ConnectorThickness = 1.5f * LineThickness;
		const double CoMSize = VisualizationContext.DebugDrawSettings->CenterOfMassSize;
		// Leave a gap around the actor position so we can see where the center is
		FVector Sa = Pa;
		const double Lena = (Xa - Pa).Size();
		if (Lena > UE_KINDA_SMALL_NUMBER)
		{
			Sa = FMath::Lerp(Pa, Xa, FMath::Clamp<double>(CoMSize / Lena, 0., 1.));
		}
		FVector Sb = Pb;
		const double Lenb = (Xb - Pb).Size();
		if (Lenb > UE_KINDA_SMALL_NUMBER)
		{
			Sb = FMath::Lerp(Pb, Xb, FMath::Clamp<double>(CoMSize / Lena, 0., 1.));
		}

		FChaosVDDebugDrawUtils::DrawLine(PDI, Pa, Sa, GenerateSelectionAwareDebugColor(White, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, ConnectorThickness);
		FChaosVDDebugDrawUtils::DrawLine(PDI, Pb, Sb, GenerateSelectionAwareDebugColor(White,bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, ConnectorThickness);
		FChaosVDDebugDrawUtils::DrawLine(PDI, Sa, Xa, GenerateSelectionAwareDebugColor(Red, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, ConnectorThickness);
		FChaosVDDebugDrawUtils::DrawLine(PDI, Sb, Xb, GenerateSelectionAwareDebugColor(Cyan, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, ConnectorThickness);
	}

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::CenterOfMassConnector))
	{
		const float ConnectorThickness = 1.5f * LineThickness;
		const double CoMSize = VisualizationContext.DebugDrawSettings->CenterOfMassSize;
		// Leave a gap around the body position so we can see where the center is
		FVector Sa = Ca;
		const double Lena = (Xa - Ca).Size();
		if (Lena > UE_KINDA_SMALL_NUMBER)
		{
			Sa = FMath::Lerp(Ca, Xa, FMath::Clamp<double>(CoMSize / Lena, 0., 1.));
		}
		FVector Sb = Cb;
		const double Lenb = (Xb - Cb).Size();
		if (Lenb > UE_KINDA_SMALL_NUMBER)
		{
			Sb = FMath::Lerp(Cb, Xb, FMath::Clamp<double>(CoMSize / Lena, 0., 1.));
		}

		FChaosVDDebugDrawUtils::DrawLine(PDI, Ca, Sa, GenerateSelectionAwareDebugColor(Black, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, ConnectorThickness);
		FChaosVDDebugDrawUtils::DrawLine(PDI, Cb, Sb, GenerateSelectionAwareDebugColor(Black, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, ConnectorThickness);
		FChaosVDDebugDrawUtils::DrawLine(PDI, Sa, Xa, GenerateSelectionAwareDebugColor(Red, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, ConnectorThickness);
		FChaosVDDebugDrawUtils::DrawLine(PDI, Sb, Xb, GenerateSelectionAwareDebugColor(Cyan, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, ConnectorThickness);
	}

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::Stretch))
	{
		const float StretchThickness = 3.0f * LineThickness;
		FChaosVDDebugDrawUtils::DrawLine(PDI, Xa, Xb, GenerateSelectionAwareDebugColor(Magenta, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, StretchThickness);
	}

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::Axes))
	{
		const FLinearColor AxisAColors[3] = {Red, Green, Blue};
		const FLinearColor AxisBColors[3] = {Cyan, Magenta, Yellow};
		DebugDrawAllAxis(InJointConstraintData, VisualizationContext, PDI, LineThickness, Xa, Ra, AxisAColors, bIsSelected);
		DebugDrawAllAxis(InJointConstraintData, VisualizationContext, PDI, LineThickness, Xb, Rb, AxisBColors, bIsSelected);
	}

	// NOTE: GetLinearImpulse is the positional impulse (pushout)
	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags::PushOut))
	{
		FLinearColor PushOutImpulseColor = FColor(0, 250, 250);
		FChaosVDDebugDrawUtils::DrawLine(PDI, Xa, Xa + VisualizationContext.DebugDrawSettings->LinearImpulseScale * VisualizationContext.SpaceTransform.TransformVectorNoScale(InJointConstraintData.PhysicsThreadJointState.LinearImpulse), GenerateSelectionAwareDebugColor(PushOutImpulseColor, bIsSelected), FText::GetEmpty(), VisualizationContext.DebugDrawSettings->DepthPriority, LineThickness);
	}

	//TODO: Should we draw the Angular Impulse
	
	PDI->SetHitProxy(nullptr);
}

#undef LOCTEXT_NAMESPACE
