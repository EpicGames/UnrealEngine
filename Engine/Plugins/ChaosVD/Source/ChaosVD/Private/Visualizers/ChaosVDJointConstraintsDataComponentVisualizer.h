// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Visualizers/ChaosVDComponentVisualizerBase.h"
#include "ChaosVDSolverDataSelection.h"
#include "Chaos/Core.h"
#include "Components/ChaosVDSolverJointConstraintDataComponent.h"
#include "Settings/ChaosVDJointConstraintVisualizationSettings.h"

struct FChaosVDJointConstraint;

class AChaosVDSolverInfoActor;
class UChaosVDJointConstraintsVisualizationSettings;

/** Visualization context structure specific for Joints visualizations */
struct FChaosVDJointVisualizationDataContext : public FChaosVDVisualizationContext
{
	TSharedPtr<FChaosVDSolverDataSelectionHandle> DataSelectionHandle = MakeShared<FChaosVDSolverDataSelectionHandle>();

	bool bIsServerVisualizationEnabled = false;
	
	AChaosVDSolverInfoActor* SolverInfoActor = nullptr;

	const UChaosVDJointConstraintsVisualizationSettings* DebugDrawSettings = nullptr;

	bool bShowDebugText = false;

	bool IsVisualizationFlagEnabled(EChaosVDJointsDataVisualizationFlags Flag) const
	{
		const EChaosVDJointsDataVisualizationFlags FlagsAsParticleFlags = static_cast<EChaosVDJointsDataVisualizationFlags>(VisualizationFlags);
		return EnumHasAnyFlags(FlagsAsParticleFlags, Flag);
	}
};

/**
 * Component visualizer in charge of generating debug draw visualizations for Joint Constraints in a UChaosVDSolverJointConstraintDataComponent
 */
class FChaosVDJointConstraintsDataComponentVisualizer final : public FChaosVDComponentVisualizerBase
{
public:
	FChaosVDJointConstraintsDataComponentVisualizer();

	virtual void RegisterVisualizerMenus() override;

	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;

	virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy) override;

protected:

	void DebugDrawAllAxis(const FChaosVDJointConstraint& InJointConstraintData, FChaosVDJointVisualizationDataContext& VisualizationContext, FPrimitiveDrawInterface* PDI, const float LineThickness, const FVector& InPosition, const Chaos::FMatrix33& InRotationMatrix, TConstArrayView<FLinearColor> AxisColors, bool bIsSelected);
	void DrawJointConstraint(const UActorComponent* Component, const FChaosVDJointConstraint& InJointConstraintData, FChaosVDJointVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI);
};
