// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/ChaosVDParticleDataComponentVisualizer.h"

#include "Actors/ChaosVDSolverInfoActor.h"
#include "ChaosVDScene.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDTabsIDs.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "SceneView.h"
#include "Settings/ChaosVDParticleVisualizationSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"
#include "Widgets/SChaosVDEnumFlagsMenu.h"
#include "Chaos/ImplicitObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/TriangleMeshImplicitObject.h"
#include "Utils/ChaosVDMenus.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

/** Sets a Hit proxy which will be cleared out as soon this struct goes out of scope*/
struct FChaosVDScopedParticleHitProxy
{
	FChaosVDScopedParticleHitProxy(FPrimitiveDrawInterface* PDI, HHitProxy* HitProxy)
	{
		PDIPtr = PDI;
		if (PDIPtr)
		{
			PDIPtr->SetHitProxy(HitProxy);
		}
	}

	~FChaosVDScopedParticleHitProxy()
	{
		if (PDIPtr)
		{
			PDIPtr->SetHitProxy(nullptr);
		}
	}
	
	FPrimitiveDrawInterface* PDIPtr = nullptr;
};

bool FChaosVDParticleDataVisualizationContext::IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags Flag) const
{
	const EChaosVDParticleDataVisualizationFlags FlagsAsParticleFlags = static_cast<EChaosVDParticleDataVisualizationFlags>(VisualizationFlags);
	return EnumHasAnyFlags(FlagsAsParticleFlags, Flag);
}

FChaosVDParticleDataComponentVisualizer::FChaosVDParticleDataComponentVisualizer()
{
	FChaosVDParticleDataComponentVisualizer::RegisterVisualizerMenus();

	InspectorTabID = FChaosVDTabID::DetailsPanel;
}

void FChaosVDParticleDataComponentVisualizer::RegisterVisualizerMenus()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	
	if (!ensure(ToolMenus))
	{
		return;
	}

	if (UToolMenu* Menu = ToolMenus->ExtendMenu(Chaos::VisualDebugger::Menus::ShowMenuName))
	{
		FToolMenuSection& Section = Menu->AddSection("ParticleVisualization.Show", LOCTEXT("ParticleVisualizationShowMenuLabel", "Particle Visualization"));

		FNewToolMenuDelegate GeometryVisualizationFlagsMenuBuilder = FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
		{
			if (Menu)
			{
				TSharedRef<SWidget> VisualizationFlagsWidget = SNew(SChaosVDEnumFlagsMenu<EChaosVDGeometryVisibilityFlags>)
					.CurrentValue_Static(&UChaosVDParticleVisualizationSettings::GetGeometryVisualizationFlags)
					.OnEnumSelectionChanged_Lambda(&UChaosVDParticleVisualizationSettings::SetGeometryVisualizationFlags);

				Menu->AddMenuEntry(NAME_None, FToolMenuEntry::InitWidget("GeometryVisualizationFlags",VisualizationFlagsWidget,FText::GetEmpty()));
			}
		});

		using namespace Chaos::VisualDebugger::Utils;

		FNewToolMenuDelegate ParticleDataVisualizationFlagsMenuBuilder = FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
		{
			if (Menu)
			{
				TSharedRef<SWidget> VisualizationFlagsWidget = SNew(SChaosVDEnumFlagsMenu<EChaosVDParticleDataVisualizationFlags>)
					.CurrentValue_Static(&UChaosVDParticleVisualizationDebugDrawSettings::GetDataDebugDrawVisualizationFlags)
					.OnEnumSelectionChanged_Static(&UChaosVDParticleVisualizationDebugDrawSettings::SetDataDebugDrawVisualizationFlags)
					.IsFlagEnabled_Static(&ShouldSettingsObjectVisFlagBeEnabledInUI<UChaosVDParticleVisualizationDebugDrawSettings,EChaosVDParticleDataVisualizationFlags>);

				Menu->AddMenuEntry(NAME_None, FToolMenuEntry::InitWidget("ParticleDebugDrawDataVisualizationFlags", VisualizationFlagsWidget,FText::GetEmpty()));
			}
		});
		
		FNewToolMenuDelegate GeometryVisualizationSettingsMenuBuilder = FNewToolMenuDelegate::CreateStatic(&CreateMenuEntryForSettingsObject<UChaosVDParticleVisualizationSettings>, EChaosVDSaveSettingsOptions::ShowResetButton);
		FNewToolMenuDelegate ParticleDataVisualizationSettingsMenuBuilder = FNewToolMenuDelegate::CreateStatic(&CreateMenuEntryForSettingsObject<UChaosVDParticleVisualizationDebugDrawSettings>, EChaosVDSaveSettingsOptions::ShowResetButton);
		FNewToolMenuDelegate ParticleColorizationMenuBuilder = FNewToolMenuDelegate::CreateStatic(&CreateMenuEntryForSettingsObject<UChaosVDParticleVisualizationColorSettings>, EChaosVDSaveSettingsOptions::ShowResetButton);

		constexpr bool bOpenSubMenuOnClick = false;
		
		Section.AddSubMenu(TEXT("GeometryVisualizationFlags"), LOCTEXT("GeometryVisualizationFlagsMenuLabel", "Geometry Flags"), LOCTEXT("GeometryVisualizationFlagsMenuToolTip", "Set of flags to enable/disable visibility of specific types of geometry/particles"), GeometryVisualizationFlagsMenuBuilder, bOpenSubMenuOnClick, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("ShowFlagsMenu.StaticMeshes")));
		Section.AddSubMenu(TEXT("GeometryVisualizationSettings"), LOCTEXT("GeometryVisualizationSettingsMenuLabel", "Geometry Visualization Settings"), LOCTEXT("GeometryVisualizationSettingsMenuToolTip", "Options to control how particle data is debug geometry is visualized"), GeometryVisualizationSettingsMenuBuilder, bOpenSubMenuOnClick, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbar.Settings")));
		Section.AddSubMenu(TEXT("ParticleDataVisualizationFlags"), LOCTEXT("ParticleDataVisualizationFlagsMenuLabel", "Particle Data Flags"), LOCTEXT("ParticleDataVisualizationFlagsMenuToolTip", "Set of flags to enable/disable visualization of specific particle data as debug draw"), ParticleDataVisualizationFlagsMenuBuilder, bOpenSubMenuOnClick, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("StaticMeshEditor.SetDrawAdditionalData")));
		Section.AddSubMenu(TEXT("ParticleDataVisualizationSettings"), LOCTEXT("ParticleDataVisualizationSettingsMenuLabel", "Particle Data Visualization Settings"), LOCTEXT("ParticleDataVisualizationSettingsMenuToolTip", "Options to control how particle data is debug drawn"), ParticleDataVisualizationSettingsMenuBuilder, bOpenSubMenuOnClick, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbar.Settings")));
		Section.AddSubMenu(TEXT("ParticleColorizationFlags"), LOCTEXT("ParticleColorizationOptionsMenuLabel", "Particle Colorization"), LOCTEXT("Particle ColorizationMenuToolTip", "Changes what colors are used to draw the particles and its data"), ParticleColorizationMenuBuilder, bOpenSubMenuOnClick, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("ColorPicker.ColorThemes")));
	}
}

void FChaosVDParticleDataComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDParticleVisualizationDebugDrawSettings* VisualizationSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDParticleVisualizationDebugDrawSettings>();
	if (!VisualizationSettings)
	{
		return;
	}

	if (VisualizationSettings->GetDataDebugDrawVisualizationFlags() == EChaosVDParticleDataVisualizationFlags::None)
	{
		// Nothing to visualize
		return;
	}

	const UChaosVDParticleDataComponent* ParticleDataComponent = Cast<UChaosVDParticleDataComponent>(Component);
	if (!ParticleDataComponent)
	{
		return;
	}

	AChaosVDSolverInfoActor* SolverDataActor = Cast<AChaosVDSolverInfoActor>(Component->GetOwner());
	if (!SolverDataActor)
	{
		return;
	}

	if (!SolverDataActor->IsVisible())
	{
		return;
	}

	const TSharedPtr<FChaosVDScene> CVDScene = SolverDataActor->GetScene().Pin();
	if (!CVDScene)
	{
		return;
	}

	FChaosVDParticleDataVisualizationContext VisualizationContext;
	VisualizationContext.VisualizationFlags = static_cast<uint32>(VisualizationSettings->GetDataDebugDrawVisualizationFlags());
	VisualizationContext.SpaceTransform = SolverDataActor->GetSimulationTransform();
	VisualizationContext.CVDScene = CVDScene;
	VisualizationContext.GeometryGenerator = CVDScene->GetGeometryGenerator();
	VisualizationContext.bShowDebugText = VisualizationSettings->bShowDebugText;
	VisualizationContext.DebugDrawSettings = VisualizationSettings;
	VisualizationContext.SolverDataSelectionObject = CVDScene->GetSolverDataSelectionObject().Pin();

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags::EnableDraw))
	{
		return;
	}

	FChaosVDSceneParticle* SelectedParticle = ParticleDataComponent->GetSelectedParticle();

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags::DrawDataOnlyForSelectedParticle))
	{
		if (SelectedParticle)
		{
			VisualizationContext.bIsSelectedData = true;
			DrawVisualizationForParticleData(Component, PDI, View, VisualizationContext, *SelectedParticle);
		}
	}
	else
	{
		ParticleDataComponent->VisitAllParticleInstances([this, PDI, View, &VisualizationContext, Component, SelectedParticle](const TSharedRef<FChaosVDSceneParticle>& InParticleDataViewer)
		{
			FChaosVDSceneParticle& ParticleInstanceRef = InParticleDataViewer.Get();
			VisualizationContext.bIsSelectedData = SelectedParticle && (SelectedParticle == &ParticleInstanceRef);
			DrawVisualizationForParticleData(Component, PDI, View, VisualizationContext, ParticleInstanceRef);

			// If we reach the debug draw limit for this frame, there is no need to continue processing particles
			return FChaosVDDebugDrawUtils::CanDebugDraw();
		});
	}
}

bool FChaosVDParticleDataComponentVisualizer::CanHandleClick(const HChaosVDComponentVisProxy& VisProxy)
{
	return VisProxy.DataSelectionHandle && VisProxy.DataSelectionHandle->IsA<FChaosVDParticleDataWrapper>();
}

bool FChaosVDParticleDataComponentVisualizer::SelectVisualizedData(const HChaosVDComponentVisProxy& VisProxy, const TSharedRef<FChaosVDScene>& InCVDScene, const TSharedRef<SChaosVDMainTab>& InMainTabToolkitHost)
{
	bool bHandled = false;
	const UChaosVDParticleDataComponent* ParticleDataComponent = Cast<UChaosVDParticleDataComponent>(VisProxy.Component.Get());
	if (!ParticleDataComponent)
	{
		return bHandled;
	}

	AChaosVDSolverInfoActor* SolverDataActor = Cast<AChaosVDSolverInfoActor>(ParticleDataComponent->GetOwner());
	if (!SolverDataActor)
	{
		return bHandled;
	}

	if (TSharedPtr<const FChaosVDParticleDataWrapper> ParticleDataViewer = VisProxy.DataSelectionHandle ?  VisProxy.DataSelectionHandle->GetDataAsShared<const FChaosVDParticleDataWrapper>() : nullptr)
	{
		using namespace Chaos::VD::TypedElementDataUtil;
		if (TSharedPtr<FChaosVDSceneParticle> ParticleInstance = SolverDataActor->GetParticleInstance(ParticleDataViewer->ParticleIndex))
		{
			InCVDScene->SetSelected(AcquireTypedElementHandleForStruct(ParticleInstance.Get(), true));
			bHandled = true;
		}
	}

	return bHandled;
}

void FChaosVDParticleDataComponentVisualizer::DrawParticleVector(FPrimitiveDrawInterface* PDI, const FVector& StartLocation, const FVector& InVector, EChaosVDParticleDataVisualizationFlags VectorID, const FChaosVDParticleDataVisualizationContext& InVisualizationContext, float LineThickness)
{
	if (!InVisualizationContext.IsVisualizationFlagEnabled(VectorID))
	{
		return;
	}

	if (!ensure(InVisualizationContext.DebugDrawSettings))
	{
		return;
	}

	const FString DebugText = InVisualizationContext.bShowDebugText ? Chaos::VisualDebugger::Utils::GenerateDebugTextForVector(InVector, UEnum::GetDisplayValueAsText(VectorID).ToString(), Chaos::VisualDebugger::ParticleDataUnitsStrings::GetUnitByID(VectorID)) : TEXT("");
	FChaosVDDebugDrawUtils::DrawArrowVector(PDI, StartLocation, StartLocation +  InVisualizationContext.DebugDrawSettings->GetScaleFortDataID(VectorID) * InVector, FText::AsCultureInvariant(DebugText), InVisualizationContext.DebugDrawSettings->ColorSettings.GetColorForDataID(VectorID, InVisualizationContext.bIsSelectedData),  InVisualizationContext.DebugDrawSettings->DepthPriority, LineThickness);
}

void FChaosVDParticleDataComponentVisualizer::DrawVisualizationForParticleData(const UActorComponent* Component, FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDParticleDataVisualizationContext& InVisualizationContext, const FChaosVDSceneParticle& InParticleInstance)
{
	using namespace Chaos::VisualDebugger::ParticleDataUnitsStrings;

	if (!View)
	{
		return;
	}

	if (!ensure(InVisualizationContext.DebugDrawSettings))
	{
		return;
	}

	if (!ensure(InVisualizationContext.SolverDataSelectionObject))
	{
		return;
	}

	if (!InParticleInstance.IsVisible())
	{
		return;
	}

	FBox ParticleBounds = InParticleInstance.GetBoundingBox();

	if (!View->ViewFrustum.IntersectBox(ParticleBounds.GetCenter(), ParticleBounds.GetExtent()))
	{
		// If this particle location is not even visible, just ignore it.
		return;
	}

	TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = InParticleInstance.GetParticleData();

	if (!ParticleData)
	{
		return;
	}

	const FVector& OwnerLocation = InVisualizationContext.SpaceTransform.TransformPosition(ParticleData->ParticlePositionRotation.MX);

	const FQuat& OwnerRotation =  InVisualizationContext.SpaceTransform.TransformRotation(ParticleData->ParticlePositionRotation.MR);
	const FVector OwnerCoMLocation = InVisualizationContext.SpaceTransform.TransformPosition(ParticleData->ParticlePositionRotation.MX + (ParticleData->ParticlePositionRotation.MR *  ParticleData->ParticleMassProps.MCenterOfMass));
	
	FChaosVDScopedParticleHitProxy ScopedHitProxy(PDI, new HChaosVDComponentVisProxy(Component, InVisualizationContext.SolverDataSelectionObject->MakeSelectionHandle(ConstCastSharedPtr<FChaosVDParticleDataWrapper>(ParticleData))));

	constexpr float DefaultLineThickness = 1.5f;
	constexpr float SelectedLineThickness = 3.5f;
	const float LineThickness = InVisualizationContext.bIsSelectedData ? SelectedLineThickness : DefaultLineThickness;

	if (InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags::Bounds))
	{
		FTransform Location;
		Location.SetLocation(ParticleBounds.GetCenter());
		FChaosVDDebugDrawUtils::DrawBox(PDI, ParticleBounds.GetExtent(), FColor::Red, Location, FText::GetEmpty(), InVisualizationContext.DebugDrawSettings->DepthPriority, LineThickness);
	}

	if (InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags::InflatedBounds))
	{
		FBox ParticleInflatedBounds = InParticleInstance.GetInflatedBoundingBox();
		bool bShowInflatedBounds = (ParticleInflatedBounds != ParticleBounds) || !InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags::Bounds);
		if (bShowInflatedBounds)
		{
			FTransform Location;
			Location.SetLocation(ParticleInflatedBounds.GetCenter());
			FColor InflatedBoundsColour = FColor(0xC7, 0x6E, 0x10);	// Dark Orange
			FChaosVDDebugDrawUtils::DrawBox(PDI, ParticleInflatedBounds.GetExtent(), InflatedBoundsColour, Location, FText::GetEmpty(), InVisualizationContext.DebugDrawSettings->DepthPriority, LineThickness);
		}
	}

	if (ParticleData->ParticleVelocities.HasValidData())
	{
		DrawParticleVector(PDI, OwnerCoMLocation, ParticleData->ParticleVelocities.MV,EChaosVDParticleDataVisualizationFlags::Velocity, InVisualizationContext, LineThickness); 
		DrawParticleVector(PDI, OwnerCoMLocation, ParticleData->ParticleVelocities.MW,EChaosVDParticleDataVisualizationFlags::AngularVelocity, InVisualizationContext, LineThickness); 
	}

	if (ParticleData->ParticleDynamics.HasValidData())
	{
		DrawParticleVector(PDI, OwnerCoMLocation, ParticleData->ParticleDynamics.MAcceleration,EChaosVDParticleDataVisualizationFlags::Acceleration, InVisualizationContext, LineThickness); 
		DrawParticleVector(PDI, OwnerCoMLocation, ParticleData->ParticleDynamics.MAngularAcceleration,EChaosVDParticleDataVisualizationFlags::AngularAcceleration, InVisualizationContext, LineThickness); 
		DrawParticleVector(PDI, OwnerCoMLocation, ParticleData->ParticleDynamics.MLinearImpulseVelocity,EChaosVDParticleDataVisualizationFlags::LinearImpulse, InVisualizationContext, LineThickness); 
		DrawParticleVector(PDI, OwnerCoMLocation, ParticleData->ParticleDynamics.MAngularImpulseVelocity,EChaosVDParticleDataVisualizationFlags::AngularImpulse, InVisualizationContext, LineThickness);
	}

	if (ParticleData->ParticleMassProps.HasValidData())
	{
		if (InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags::CenterOfMass))
		{
			if (const TSharedPtr<FChaosVDGeometryBuilder> GeometryGenerator = InVisualizationContext.GeometryGenerator.Pin())
			{
				FCollisionShape Sphere;
				Sphere.SetSphere(InVisualizationContext.DebugDrawSettings->CenterOfMassRadius);
				const FPhysicsShapeAdapter SphereShapeAdapter(FQuat::Identity, Sphere);

				FChaosVDDebugDrawUtils::DrawImplicitObject(PDI, GeometryGenerator, &SphereShapeAdapter.GetGeometry(), FTransform(OwnerCoMLocation), InVisualizationContext.DebugDrawSettings->ColorSettings.GetColorForDataID(EChaosVDParticleDataVisualizationFlags::CenterOfMass, InVisualizationContext.bIsSelectedData), UEnum::GetDisplayValueAsText(EChaosVDParticleDataVisualizationFlags::CenterOfMass), InVisualizationContext.DebugDrawSettings->DepthPriority, LineThickness);
			}
		}
	}

	// TODO: This is a Proof of concept to test how debug draw connectivity data will look
	if (ParticleData->ParticleCluster.HasValidData())
	{
		if (InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags::ClusterConnectivityEdge))
		{
			for (const FChaosVDConnectivityEdge& ConnectivityEdge : ParticleData->ParticleCluster.ConnectivityEdges)
			{
				if (const TSharedPtr<FChaosVDScene> ScenePtr = InVisualizationContext.CVDScene.Pin())
				{
					if (TSharedPtr<FChaosVDSceneParticle> SiblingParticle = ScenePtr->GetParticleInstance(InVisualizationContext.SolverID, ConnectivityEdge.SiblingParticleID))
					{
						if (TSharedPtr<const FChaosVDParticleDataWrapper> SiblingParticleData = SiblingParticle->GetParticleData())
						{
							FColor DebugDrawColor = InVisualizationContext.DebugDrawSettings->ColorSettings.GetColorForDataID(EChaosVDParticleDataVisualizationFlags::ClusterConnectivityEdge, InVisualizationContext.bIsSelectedData);
							FVector BoxExtents(2,2,2);
							FTransform BoxTransform(OwnerRotation, OwnerLocation);
							FChaosVDDebugDrawUtils::DrawBox(PDI, BoxExtents, DebugDrawColor, BoxTransform, FText::GetEmpty(), InVisualizationContext.DebugDrawSettings->DepthPriority, LineThickness);

							FVector SiblingParticleLocation = InVisualizationContext.SpaceTransform.TransformPosition(SiblingParticleData->ParticlePositionRotation.MX);
							FChaosVDDebugDrawUtils::DrawLine(PDI, OwnerLocation, SiblingParticleLocation, DebugDrawColor, FText::FormatOrdered(LOCTEXT("StrainDebugDraw","Strain {0}"), ConnectivityEdge.Strain), InVisualizationContext.DebugDrawSettings->DepthPriority, LineThickness);
						}
					}	
				}
			}
		}
	}

	// Draw the selected tri mesh's BVH
	if (InVisualizationContext.DebugDrawSettings->bDrawTriMeshBVH)
	{
		if (const TSharedPtr<FChaosVDInstancedMeshData> SelectedMeshInstance = InParticleInstance.GetSelectedMeshInstance().Pin())
		{
			const FChaosVDParticlePositionRotation& ParticlePositionRotation = InParticleInstance.GetParticleData()->ParticlePositionRotation;
			FTransform Transform = FTransform::Identity;
			Transform.SetTranslation(ParticlePositionRotation.MX);
			Transform.SetRotation(ParticlePositionRotation.MR);

			const TSharedRef<FChaosVDExtractedGeometryDataHandle>& GeometryHandle = SelectedMeshInstance->GetGeometryHandle();
			if (const Chaos::FImplicitObject* ImplicitObject = GeometryHandle->GetImplicitObject())
			{
				if (const Chaos::TImplicitObjectScaled<Chaos::FTriangleMeshImplicitObject>* ScaledTriMesh = ImplicitObject->AsA<Chaos::TImplicitObjectScaled<Chaos::FTriangleMeshImplicitObject>>())
				{
					FChaosVDTrimeshBVHVisualizer TrimeshBVHVisualizer;
					const FVector Scale = ScaledTriMesh->GetScale();
					Transform.SetScale3D(Scale);
					TrimeshBVHVisualizer.Draw(PDI, InVisualizationContext, Transform, *ScaledTriMesh->GetUnscaledObject());
				}
				else if (const Chaos::TImplicitObjectInstanced<Chaos::FTriangleMeshImplicitObject>* InstancedTriMesh = ImplicitObject->AsA<Chaos::TImplicitObjectInstanced<Chaos::FTriangleMeshImplicitObject>>())
				{
					FChaosVDTrimeshBVHVisualizer TrimeshBVHVisualizer;
					TrimeshBVHVisualizer.Draw(PDI, InVisualizationContext, Transform, *InstancedTriMesh->GetInstancedObject());
				}
				else if (const Chaos::FTriangleMeshImplicitObject* TriMesh = ImplicitObject->AsA<Chaos::FTriangleMeshImplicitObject>())
				{
					FChaosVDTrimeshBVHVisualizer TrimeshBVHVisualizer;
					TrimeshBVHVisualizer.Draw(PDI, InVisualizationContext, Transform, *TriMesh);
				}
			}
		}
	}
}

void FChaosVDTrimeshBVHVisualizer::Draw(FPrimitiveDrawInterface* PDI, const FChaosVDParticleDataVisualizationContext& InVisualizationContext, const FTransform& LocalToWorldTransform, const Chaos::FTriangleMeshImplicitObject& TriMesh) const
{
	const int32 TargetLevel = InVisualizationContext.DebugDrawSettings->TriMeshBVHDrawLevel;
	DrawBVH(PDI, LocalToWorldTransform, TriMesh.GetBVH(), TargetLevel);
}

void FChaosVDTrimeshBVHVisualizer::DrawBVH(FPrimitiveDrawInterface* PDI, const FTransform& LocalToWorldTransform, const Chaos::FTrimeshBVH& BVH, const int32 TargetLevel) const
{
	const int32 RootIndex = 0;
	// The root has to be handled differently as there is no aabb in the tree for the root, it's implicit from the first two child aabbs.
	if (TargetLevel == 0)
	{
		Chaos::FAABB3 RootAabb = ToAabb(BVH.Nodes[RootIndex].Children.GetBounds(0));
		RootAabb.GrowToInclude(ToAabb(BVH.Nodes[RootIndex].Children.GetBounds(1)));
		DrawAabb(PDI, LocalToWorldTransform, RootAabb, 0);
	}
	else
	{
		// Start at level 1 instead of 0 as the root is handled above and the aabbs for the children are embedded in the node
		DrawBVHLevel(PDI, LocalToWorldTransform, BVH, RootIndex, 1, TargetLevel);
	}
}

void FChaosVDTrimeshBVHVisualizer::DrawBVHLevel(FPrimitiveDrawInterface* PDI, const FTransform& LocalToWorldTransform, const Chaos::FTrimeshBVH& BVH, const int32 NodeIndex, const int32 CurrentLevel, const int32 TargetLevel) const
{
	const Chaos::FTrimeshBVH::FNode& Node = BVH.Nodes[NodeIndex];

	if (TargetLevel == -1 || TargetLevel == CurrentLevel)
	{
		DrawAabb(PDI, LocalToWorldTransform, ToAabb(Node.Children.GetBounds(0)), CurrentLevel);
		DrawAabb(PDI, LocalToWorldTransform, ToAabb(Node.Children.GetBounds(1)), CurrentLevel);
	}
	if (TargetLevel == -1 || CurrentLevel < TargetLevel)
	{
		for (int32 ChildIndex = 0; ChildIndex < 2; ++ChildIndex)
		{
			const bool bIsLeaf = Node.Children.GetFaceCount(ChildIndex) != 0;
			if (!bIsLeaf)
			{
				const int32 ChildNodeIndex = Node.Children.GetChildOrFaceIndex(ChildIndex);
				DrawBVHLevel(PDI, LocalToWorldTransform, BVH, ChildNodeIndex, CurrentLevel + 1, TargetLevel);
			}
		}
	}
}

void FChaosVDTrimeshBVHVisualizer::DrawAabb(FPrimitiveDrawInterface* PDI, const FTransform& LocalToWorldTransform, const Chaos::FAABB3& Aabb, const int32 ColorSeed) const
{
	// Build the local to world transform for a "unit" aabb (unit being half-extents of 0.5)
	const FVector UnitHalfExtent(0.5f);
	const FTransform LocalAabbTransform(FQuat::Identity, Aabb.GetCenter(), Aabb.Extents());
	const FTransform WorldAabbTransform = LocalAabbTransform * LocalToWorldTransform;
	const FColor Color = FColor::MakeRandomSeededColor(ColorSeed);
	FChaosVDDebugDrawUtils::DrawBox(PDI, UnitHalfExtent, Color, WorldAabbTransform, FText());
}

Chaos::FAABB3 FChaosVDTrimeshBVHVisualizer::ToAabb(const Chaos::FAABBVectorized& AabbVectorized) const
{
	auto ToVector = [](VectorRegister4Float VectorRegister)
	{
		double Values[4];
		VectorStore(VectorRegister, Values);
		return FVector(Values[0], Values[1], Values[2]);
	};
	const FVector Min = ToVector(AabbVectorized.GetMin());
	const FVector Max = ToVector(AabbVectorized.GetMax());
	return Chaos::FAABB3(Min, Max);
}

#undef LOCTEXT_NAMESPACE
