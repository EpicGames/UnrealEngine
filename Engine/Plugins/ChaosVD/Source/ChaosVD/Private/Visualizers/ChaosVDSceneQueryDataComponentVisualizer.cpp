// Copyright Epic Games, Inc. All Rights Reserved.

#include "Visualizers/ChaosVDSceneQueryDataComponentVisualizer.h"

#include "Actors/ChaosVDGameFrameInfoActor.h"
#include "Components/ChaosVDSceneQueryDataComponent.h"
#include "ChaosVDGeometryBuilder.h"
#include "ChaosVDScene.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "EditorViewportClient.h"
#include "SceneView.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Settings/ChaosVDSceneQueryVisualizationSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Utils/ChaosVDMenus.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSceneQueryDataComponentVisualizer)

class UChaosVDCoreSettings;

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDSceneQueryDataComponentVisualizer::FChaosVDSceneQueryDataComponentVisualizer()
{
	FChaosVDSceneQueryDataComponentVisualizer::RegisterVisualizerMenus();

	InspectorTabID = FChaosVDTabID::SceneQueryDataDetails;
}

void FChaosVDSceneQueryDataComponentVisualizer::RegisterVisualizerMenus()
{
	FName MenuSection("SceneQueryDataVisualization.Show");
	FText MenuSectionLabel = LOCTEXT("SceneQueryDataShowMenuLabel", "Scene Query Data Visualization");
	FText FlagsMenuLabel = LOCTEXT("SceneQueryDataFlagsMenuLabel", "Scene Query Data Flags");
	FText FlagsMenuTooltip = LOCTEXT("SceneQueryDataFlagsMenuToolTip", "Set of flags to enable/disable visibility of specific types of scene query data");
	FSlateIcon FlagsMenuIcon = FSlateIcon(FChaosVDStyle::Get().GetStyleSetName(), TEXT("SceneQueriesInspectorIcon"));

	FText SettingsMenuLabel = LOCTEXT("SceneQuerySettingsMenuLabel", "Scene Query Visualization Settings");
	FText SettingsMenuTooltip = LOCTEXT("SceneQuerySettingsMenuToolTip", "Options to change how the recorded scene query data is debug drawn");
	
	CreateGenericVisualizerMenu<UChaosVDSceneQueriesVisualizationSettings, EChaosVDSceneQueryVisualizationFlags>(Chaos::VisualDebugger::Menus::ShowMenuName, MenuSection, MenuSectionLabel, FlagsMenuLabel, FlagsMenuTooltip, FlagsMenuIcon, SettingsMenuLabel, SettingsMenuTooltip);
}

bool FChaosVDSceneQueryDataComponentVisualizer::CanHandleClick(const HChaosVDComponentVisProxy& VisProxy)
{
	return VisProxy.DataSelectionHandle && VisProxy.DataSelectionHandle->IsA<FChaosVDQueryDataWrapper>();
}

void FChaosVDSceneQueryDataComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDSceneQueryDataComponent* SceneQueryDataComponent = Cast<UChaosVDSceneQueryDataComponent>(Component);
	if (!SceneQueryDataComponent)
	{
		return;
	}

	const AChaosVDSolverInfoActor* SolverInfoActor = Cast<AChaosVDSolverInfoActor>(SceneQueryDataComponent->GetOwner());
	if (!SolverInfoActor)
	{
		return;
	}

	if (!SolverInfoActor->IsVisible())
	{
		return;
	}

	const TSharedPtr<FChaosVDScene> CVDScene = SolverInfoActor->GetScene().Pin();
	if (!CVDScene)
	{
		return;
	}

	const TSharedPtr<FChaosVDGeometryBuilder> GeometryGenerator = CVDScene->GetGeometryGenerator().Pin();
	if (!GeometryGenerator)
	{
		return;
	}

	const TSharedPtr<FChaosVDRecording> CVDRecording = CVDScene->GetLoadedRecording();
	if (!CVDRecording)
	{
		return;
	}

	TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelectionObject = CVDScene->GetSolverDataSelectionObject().Pin();
	if (!ensure(SolverDataSelectionObject))
	{
		return;
	}

	UChaosVDSceneQueriesVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSceneQueriesVisualizationSettings>();
	if (!Settings)
	{
		return;
	}

	FChaosVDSceneQueryVisualizationDataContext VisualizationContext;
	VisualizationContext.CVDScene = SolverInfoActor->GetScene();
	VisualizationContext.SpaceTransform = FTransform::Identity;
	VisualizationContext.GeometryGenerator = GeometryGenerator;
	VisualizationContext.SolverDataSelectionObject = SolverDataSelectionObject;
	VisualizationContext.VisualizationFlags = static_cast<uint32>(UChaosVDSceneQueriesVisualizationSettings::GetDataVisualizationFlags());
	VisualizationContext.DebugDrawSettings = Settings;

	if (EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::EnableDraw, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags)))
	{
		// If Draw only selected Query is enabled, but no query is selected, just draw all queries

		TSharedPtr<FChaosVDSolverDataSelectionHandle> SelectionHandle = SolverDataSelectionObject->GetCurrentSelectionHandle();
		const bool bHasSelectedQuery = SelectionHandle && SelectionHandle->IsA<FChaosVDQueryDataWrapper>();
		const bool bOnlyDrawSelected = Settings->CurrentVisualizationMode == EChaosVDSQFrameVisualizationMode::PerSolverRecordingOrder ||
										EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::OnlyDrawSelectedQuery, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags));

		if (bHasSelectedQuery && bOnlyDrawSelected)
		{
			DrawSceneQuery(Component, View, PDI, CVDScene, CVDRecording, VisualizationContext, SelectionHandle->GetDataAsShared<FChaosVDQueryDataWrapper>());
		}
		else
		{
			for (const TSharedPtr<FChaosVDQueryDataWrapper>& Query : SceneQueryDataComponent->GetAllQueries())
			{
				DrawSceneQuery(Component, View, PDI, CVDScene, CVDRecording, VisualizationContext, Query);
			}
		}
	}
}

void FChaosVDSceneQueryDataComponentVisualizer::DrawLineTraceQuery(const UActorComponent* Component, const FChaosVDQueryDataWrapper& SceneQueryData, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDSceneQueriesVisualizationSettings* DebugDrawSettings = Cast<UChaosVDSceneQueriesVisualizationSettings>(VisualizationContext.DebugDrawSettings);
	if (!DebugDrawSettings)
	{
		return;
	}

	PDI->SetHitProxy(new HChaosVDComponentVisProxy(Component, VisualizationContext.DataSelectionHandle));

	const FText DebugText = DebugDrawSettings->bShowText ? FText::FormatOrdered(LOCTEXT("LineTraceDebugDrawText", "Type: Line Trace \n Tag {1} \n Owner Tag {2}"), FText::AsCultureInvariant(SceneQueryData.CollisionQueryParams.TraceTag.ToString()), FText::AsCultureInvariant(SceneQueryData.CollisionQueryParams.OwnerTag.ToString()))
							: FText::GetEmpty();

	FVector EndLocationToDraw;
	if (SceneQueryData.SQVisitData.IsValidIndex(SceneQueryData.CurrentVisitIndex))
	{
		const FChaosVDQueryVisitStep& SQVisitData = SceneQueryData.SQVisitData[SceneQueryData.CurrentVisitIndex];
		EndLocationToDraw = SceneQueryData.StartLocation + SQVisitData.QueryFastData.Dir * SQVisitData.QueryFastData.CurrentLength;
	}
	else
	{
		// Fallback to draw the end position
		EndLocationToDraw = SceneQueryData.EndLocation;
	}

	FChaosVDDebugDrawUtils::DrawArrowVector(PDI, SceneQueryData.StartLocation, EndLocationToDraw, DebugText, VisualizationContext.DebugDrawColor, DebugDrawSettings->DepthPriority);

	PDI->SetHitProxy(nullptr);

	if (EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::DrawHits, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags)))
	{
		DrawHits(Component, SceneQueryData, PDI, VisualizationContext.HitColor, VisualizationContext);
	}
}

void FChaosVDSceneQueryDataComponentVisualizer::DrawOverlapQuery(const UActorComponent* Component, const FChaosVDQueryDataWrapper& SceneQueryData, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDSceneQueriesVisualizationSettings* DebugDrawSettings = Cast<UChaosVDSceneQueriesVisualizationSettings>(VisualizationContext.DebugDrawSettings);
	if (!DebugDrawSettings)
	{
		return;
	}
	
	PDI->SetHitProxy(new HChaosVDComponentVisProxy(Component, VisualizationContext.DataSelectionHandle));

	const Chaos::FConstImplicitObjectPtr InputShapePtr = VisualizationContext.InputGeometry;
	if (ensure(InputShapePtr))
	{
		const FText DebugText = DebugDrawSettings->bShowText ? FText::FormatOrdered(LOCTEXT("OverlapDebugDrawText", "Type: Overlap \n Tag {1} \n Owner Tag {2}"), FText::AsCultureInvariant(SceneQueryData.CollisionQueryParams.TraceTag.ToString()), FText::AsCultureInvariant(SceneQueryData.CollisionQueryParams.OwnerTag.ToString()))
								: FText::GetEmpty();
		FChaosVDDebugDrawUtils::DrawImplicitObject(PDI, VisualizationContext.GeometryGenerator.Pin(), InputShapePtr, FTransform(SceneQueryData.GeometryOrientation, SceneQueryData.StartLocation), VisualizationContext.DebugDrawColor, DebugText, DebugDrawSettings->DepthPriority);
	}

	PDI->SetHitProxy(nullptr);

	if (EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::DrawHits, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags)))
	{
		DrawHits(Component, SceneQueryData, PDI, VisualizationContext.HitColor, VisualizationContext);
	}
}

void FChaosVDSceneQueryDataComponentVisualizer::DrawSweepQuery(const UActorComponent* Component, const FChaosVDQueryDataWrapper& SceneQueryData, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDSceneQueriesVisualizationSettings* DebugDrawSettings = Cast<UChaosVDSceneQueriesVisualizationSettings>(VisualizationContext.DebugDrawSettings);
	if (!DebugDrawSettings)
	{
		return;
	}

	PDI->SetHitProxy(new HChaosVDComponentVisProxy(Component, VisualizationContext.DataSelectionHandle));

	const Chaos::FConstImplicitObjectPtr InputShapePtr = VisualizationContext.InputGeometry;
	if (ensure(InputShapePtr))
	{
		FChaosVDDebugDrawUtils::DrawImplicitObject(PDI, VisualizationContext.GeometryGenerator.Pin(), InputShapePtr, FTransform(SceneQueryData.GeometryOrientation, SceneQueryData.StartLocation), VisualizationContext.DebugDrawColor, FText::GetEmpty(), DebugDrawSettings->DepthPriority);

		FVector EndLocationToDraw;
		if (SceneQueryData.SQVisitData.IsValidIndex(SceneQueryData.CurrentVisitIndex))
		{
			const FChaosVDQueryVisitStep& SQVisitData = SceneQueryData.SQVisitData[SceneQueryData.CurrentVisitIndex];
			EndLocationToDraw = SceneQueryData.StartLocation + SQVisitData.QueryFastData.Dir * SQVisitData.QueryFastData.CurrentLength;
		}
		else
		{
			// Fallback to draw the end position
			EndLocationToDraw = SceneQueryData.EndLocation;
		}

		FChaosVDDebugDrawUtils::DrawImplicitObject(PDI, VisualizationContext.GeometryGenerator.Pin(), InputShapePtr, FTransform(SceneQueryData.GeometryOrientation, EndLocationToDraw), VisualizationContext.DebugDrawColor, FText::GetEmpty(), DebugDrawSettings->DepthPriority);
	}

	PDI->SetHitProxy(nullptr);

	const FText DebugText = DebugDrawSettings->bShowText ? FText::FormatOrdered(LOCTEXT("SweepDebugDrawText", "Type: Sweep \n Tag {1} \n Owner Tag {2}"), FText::AsCultureInvariant(SceneQueryData.CollisionQueryParams.TraceTag.ToString()), FText::AsCultureInvariant(SceneQueryData.CollisionQueryParams.OwnerTag.ToString()))
							: FText::GetEmpty();

	FChaosVDDebugDrawUtils::DrawArrowVector(PDI, SceneQueryData.StartLocation, SceneQueryData.EndLocation, DebugText, VisualizationContext.DebugDrawColor, DebugDrawSettings->DepthPriority);

	if (EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::DrawHits, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags)))
	{
		DrawHits(Component, SceneQueryData, PDI, VisualizationContext.HitColor, VisualizationContext);
	}
}

void FChaosVDSceneQueryDataComponentVisualizer::DrawHits(const UActorComponent* Component, const FChaosVDQueryDataWrapper& SceneQueryData, FPrimitiveDrawInterface* PDI, const FColor& InColor, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext)
{
	const UChaosVDSceneQueriesVisualizationSettings* DebugDrawSettings = Cast<UChaosVDSceneQueriesVisualizationSettings>(VisualizationContext.DebugDrawSettings);
	if (!DebugDrawSettings)
	{
		return;
	}

	for (int32 SQVisitIndex = 0; SQVisitIndex < SceneQueryData.SQVisitData.Num(); ++SQVisitIndex)
	{
		const FChaosVDQueryVisitStep& SQVisitData = SceneQueryData.SQVisitData[SQVisitIndex];
		if (!SQVisitData.HitData.HasValidData())
		{
			continue;
		}
	
		TSharedPtr<FChaosVDSolverDataSelectionHandle> HitSelectionHandle = VisualizationContext.SolverDataSelectionObject->MakeSelectionHandle(VisualizationContext.DataSelectionHandle->GetDataAsShared<FChaosVDQueryDataWrapper>());

		FChaosVDSceneQuerySelectionContext ContextData;
		ContextData.SQVisitIndex = SQVisitIndex;
		HitSelectionHandle->SetHandleContext(MoveTemp(ContextData));

		PDI->SetHitProxy(new HChaosVDComponentVisProxy(Component, HitSelectionHandle));

		const FText HitPointDebugText = DebugDrawSettings->bShowText ? FText::FormatOrdered(LOCTEXT("SceneQueryHitDebugText", "Distance {0} \n Face Index {1} \n "), SQVisitData.HitData.Distance, SQVisitData.HitData.FaceIdx) : FText::GetEmpty();
		static FText HitFaceNormalDebugText = FText::AsCultureInvariant(TEXT("Hit Face Normal"));
		static FText HitWorldNormalDebugText = FText::AsCultureInvariant(TEXT("Hit World Normal"));

		const FMatrix Axes = FRotationMatrix::MakeFromX(SQVisitData.HitData.WorldNormal);
		constexpr float Thickness = 5.0f;
		constexpr float CircleRadius = 5.0f;
		constexpr int32 CircleSegments = 12;
		constexpr float NormalScale = 10.5f;
		FChaosVDDebugDrawUtils::DrawCircle(PDI, SQVisitData.HitData.WorldPosition, CircleRadius, CircleSegments, InColor, Thickness, Axes.GetUnitAxis(EAxis::Y), Axes.GetUnitAxis(EAxis::Z), HitPointDebugText,  DebugDrawSettings->DepthPriority);
		FChaosVDDebugDrawUtils::DrawArrowVector(PDI, SQVisitData.HitData.WorldPosition, SQVisitData.HitData.WorldPosition + SQVisitData.HitData.FaceNormal * NormalScale, DebugDrawSettings->bShowText ? HitFaceNormalDebugText : FText::GetEmpty(), (FLinearColor(InColor) * 0.65f).ToFColorSRGB(), DebugDrawSettings->DepthPriority);

		// Hit Face Normal is not used in line traces
		if (SceneQueryData.Type != EChaosVDSceneQueryType::RayCast)
		{
			FChaosVDDebugDrawUtils::DrawArrowVector(PDI, SQVisitData.HitData.WorldPosition, SQVisitData.HitData.WorldPosition + SQVisitData.HitData.WorldNormal * NormalScale, DebugDrawSettings->bShowText ? HitWorldNormalDebugText : FText::GetEmpty(), InColor, DebugDrawSettings->DepthPriority);
		}

		if (TSharedPtr<FChaosVDSolverDataSelectionHandle> CurrentSelection = VisualizationContext.SolverDataSelectionObject->GetCurrentSelectionHandle())
		{
			if (IsHitSelected(SQVisitIndex, CurrentSelection.ToSharedRef(), HitSelectionHandle.ToSharedRef()))
			{
				// We don't have an easy way to show something is selected with debug draw
				// but 3D box surrounding the hit is better than nothing
				FTransform SelectionBoxTransform;
				SelectionBoxTransform.SetRotation(FRotationMatrix::MakeFromZ(SQVisitData.HitData.WorldNormal).ToQuat());
				SelectionBoxTransform.SetLocation(SQVisitData.HitData.WorldPosition);

				// The Selection box should be a bit bigger than the configured circle radius for the debug draw hit
				constexpr float HitSelectionBoxSize = CircleRadius * 1.2f;

				FVector SelectionBoxExtents(HitSelectionBoxSize,HitSelectionBoxSize,HitSelectionBoxSize);
				FChaosVDDebugDrawUtils::DrawBox(PDI, SelectionBoxExtents, FColor::Yellow, SelectionBoxTransform, FText::GetEmpty(), DebugDrawSettings->DepthPriority);
			}
		}

		PDI->SetHitProxy(nullptr);
	}
}

bool FChaosVDSceneQueryDataComponentVisualizer::HasEndLocation(const FChaosVDQueryDataWrapper& SceneQueryData) const
{
	return SceneQueryData.Type != EChaosVDSceneQueryType::Sweep;
}

bool FChaosVDSceneQueryDataComponentVisualizer::IsHitSelected(int32 SQVisitIndex, const TSharedRef<FChaosVDSolverDataSelectionHandle>& CurrentSelection, const TSharedRef<FChaosVDSolverDataSelectionHandle>& SQVisitSelectionHandle)
{
	if (SQVisitSelectionHandle->IsSelected())
	{
		if (FChaosVDSceneQuerySelectionContext* SelectionContext = CurrentSelection->GetContextData<FChaosVDSceneQuerySelectionContext>())
		{
			return SelectionContext->SQVisitIndex == SQVisitIndex;
		}
	}

	return false;				
}

void FChaosVDSceneQueryDataComponentVisualizer::DrawSceneQuery(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI, const TSharedPtr<FChaosVDScene>& CVDScene, const TSharedPtr<FChaosVDRecording>& CVDRecording, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext, const TSharedPtr<FChaosVDQueryDataWrapper>& Query)
{
	// Reset query Specify context values
	VisualizationContext.InputGeometry = nullptr;
	VisualizationContext.DataSelectionHandle = nullptr;

	if (!Query)
	{
		return;
	}

	const bool bHideEmptyQueries = EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::HideEmptyQueries, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags));
	if (bHideEmptyQueries && Query->SQVisitData.IsEmpty())
	{
		return;
	}

	const bool bHideSubQueries = EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::HideSubQueries, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags));
	if (bHideSubQueries && Query->ParentQueryID != INDEX_NONE)
	{
		return;
	}

	FBox QueryBounds = Chaos::VisualDebugger::Utils::CalculateSceneQueryShapeBounds(Query.ToSharedRef(), CVDRecording.ToSharedRef());
	if (!View->ViewFrustum.IntersectBox(QueryBounds.GetCenter(), QueryBounds.GetExtent()))
	{
		// If this query location is not even visible, just ignore it.
		return;
	}

	if (const Chaos::FConstImplicitObjectPtr* InputShapePtrPtr = CVDRecording->GetGeometryMap().Find(Query->InputGeometryKey))
	{
		VisualizationContext.InputGeometry = *InputShapePtrPtr;
	}

	VisualizationContext.DataSelectionHandle = VisualizationContext.SolverDataSelectionObject->MakeSelectionHandle(Query);
	bool bIsSelected = VisualizationContext.DataSelectionHandle && VisualizationContext.DataSelectionHandle->IsSelected();
	VisualizationContext.GenerateColor(Query->ID, bIsSelected);

	switch (Query->Type)
	{
	case EChaosVDSceneQueryType::RayCast:
		{
			if (EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::DrawLineTraceQueries, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags)))
			{
				DrawLineTraceQuery(Component, *Query, VisualizationContext, View, PDI);
			}
			break;
		}
	case EChaosVDSceneQueryType::Overlap:
		{
			if (EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::DrawOverlapQueries, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags)))
			{
				DrawOverlapQuery(Component, *Query, VisualizationContext, View, PDI);
			}
			break;
		}
	case EChaosVDSceneQueryType::Sweep:
		{
			if (EnumHasAnyFlags(EChaosVDSceneQueryVisualizationFlags::DrawSweepQueries, static_cast<EChaosVDSceneQueryVisualizationFlags>(VisualizationContext.VisualizationFlags)))
			{
				DrawSweepQuery(Component, *Query, VisualizationContext, View, PDI);
			}
			break;
		}
	default:
		break;
	}
}

#undef LOCTEXT_NAMESPACE
