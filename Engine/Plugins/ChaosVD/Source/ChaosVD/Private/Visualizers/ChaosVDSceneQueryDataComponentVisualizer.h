// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Visualizers/ChaosVDComponentVisualizerBase.h"
#include "ComponentVisualizer.h"
#include "Chaos/ImplicitFwd.h"
#include "Chaos/ImplicitObject.h"
#include "Components/ChaosVDSceneQueryDataComponent.h"

#include "ChaosVDSceneQueryDataComponentVisualizer.generated.h"

struct FChaosVDRecording;
class FChaosVDGeometryBuilder;
struct FChaosVDVisualizationContext;
struct FChaosVDQueryDataWrapper;

/** Struct used to pass data about a specific query to other objects */
USTRUCT()
struct FChaosVDSceneQuerySelectionContext : public FChaosVDSelectionContext
{
	GENERATED_BODY()

	int32 SQVisitIndex = INDEX_NONE;
};

/** Visualization context structure specific for Scene Queries visualizations */
struct FChaosVDSceneQueryVisualizationDataContext : public FChaosVDVisualizationContext
{
	TSharedPtr<FChaosVDSolverDataSelectionHandle> DataSelectionHandle = nullptr;

	/** Generates a random color based on the selection state and query ID, which will be used to debug draw the scene query */
	void GenerateColor(int32 QueryID, bool bIsSelected)
	{
		RandomSeededColor = bIsSelected ? FLinearColor::White : FLinearColor::MakeRandomSeededColor(QueryID);
		DebugDrawColor = RandomSeededColor.ToFColorSRGB();
		DebugDrawDarkerColor = (RandomSeededColor * 0.85f).ToFColorSRGB();
		HitColor = (RandomSeededColor * 1.2f).ToFColorSRGB();
	}

	FLinearColor RandomSeededColor = FLinearColor(EForceInit::ForceInit);
	FColor DebugDrawColor = FColor(EForceInit::ForceInit);
	FColor DebugDrawDarkerColor = FColor(EForceInit::ForceInit);
	FColor HitColor = FColor(EForceInit::ForceInit);

	Chaos::FConstImplicitObjectPtr InputGeometry = nullptr;
	TWeakPtr<FChaosVDGeometryBuilder> GeometryGenerator = nullptr;
};

/**
 * Component visualizer in charge of generating debug draw visualizations for scene queries in a ChaosVDSceneQueryDataComponent
 */
class FChaosVDSceneQueryDataComponentVisualizer final : public FChaosVDComponentVisualizerBase
{

public:
	FChaosVDSceneQueryDataComponentVisualizer();

	virtual void RegisterVisualizerMenus() override;

	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;


	virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy) override;

private:

	void DrawLineTraceQuery(const UActorComponent* Component, const FChaosVDQueryDataWrapper& SceneQueryData, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI);
	void DrawOverlapQuery(const UActorComponent* Component, const FChaosVDQueryDataWrapper& SceneQueryData, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI);
	void DrawSweepQuery(const UActorComponent* Component, const FChaosVDQueryDataWrapper& SceneQueryData, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI);
	void DrawHits(const UActorComponent* Component, const FChaosVDQueryDataWrapper& SceneQueryData, FPrimitiveDrawInterface* PDI, const FColor& InColor, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext);

	void DrawSceneQuery(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI, const TSharedPtr<FChaosVDScene>& CVDScene, const TSharedPtr<FChaosVDRecording>& CVDRecording, FChaosVDSceneQueryVisualizationDataContext& VisualizationContext, const TSharedPtr<FChaosVDQueryDataWrapper>& Query);

	bool HasEndLocation(const FChaosVDQueryDataWrapper& SceneQueryData) const;

	bool IsHitSelected(int32 SQVisitIndex, const TSharedRef<FChaosVDSolverDataSelectionHandle>& CurrentSelection, const TSharedRef<FChaosVDSolverDataSelectionHandle>& SQVisitSelectionHandle);
};
