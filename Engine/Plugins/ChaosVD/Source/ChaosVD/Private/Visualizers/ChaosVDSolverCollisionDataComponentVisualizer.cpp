// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDSolverCollisionDataComponentVisualizer.h"

#include "ChaosVDScene.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDTabsIDs.h"
#include "EditorViewportClient.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Components/ChaosVDSolverCollisionDataComponent.h"
#include "Settings/ChaosVDCollisionVisualizationSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "Utils/ChaosVDMenus.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"
#include "Widgets/SChaosVDCollisionDataInspector.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDSolverCollisionDataComponentVisualizer)


#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDSolverCollisionDataComponentVisualizer::FChaosVDSolverCollisionDataComponentVisualizer()
{
	FChaosVDSolverCollisionDataComponentVisualizer::RegisterVisualizerMenus();

	InspectorTabID = FChaosVDTabID::CollisionDataDetails;
}

FChaosVDSolverCollisionDataComponentVisualizer::~FChaosVDSolverCollisionDataComponentVisualizer()
{
}

void FChaosVDSolverCollisionDataComponentVisualizer::RegisterVisualizerMenus()
{
	FName MenuSection("CollisionDataVisualization.Show");
	FText MenuSectionLabel = LOCTEXT("CollisionDataVisualizationShowMenuLabel", "Collision Data Visualization");
	FText FlagsMenuLabel = LOCTEXT("CollisionVisualizationFlagsMenuLabel", "Collision Data Flags");
	FText FlagsMenuTooltip = LOCTEXT("CollisionVisualizationFlagsMenuToolTip", "Set of flags to enable/disable visibility of specific types of collisionData");
	FSlateIcon FlagsMenuIcon = FSlateIcon(FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("ShowFlagsMenu.Collision")));

	FText SettingsMenuLabel = LOCTEXT("CollisionVisualizationSettingsMenuLabel", "Collision Visualization Settings");
	FText SettingsMenuTooltip = LOCTEXT("CollisionVisualizationSettingsMenuToolTip", "Options to change how the recorded collision Data is debug drawn");
	
	CreateGenericVisualizerMenu<UChaosVDCollisionDataVisualizationSettings, EChaosVDCollisionVisualizationFlags>(Chaos::VisualDebugger::Menus::ShowMenuName, MenuSection, MenuSectionLabel, FlagsMenuLabel, FlagsMenuTooltip, FlagsMenuIcon, SettingsMenuLabel, SettingsMenuTooltip);
}

bool FChaosVDSolverCollisionDataComponentVisualizer::ShouldShowForSelectedSubcomponents(const UActorComponent* Component)
{
	return false;
}

void FChaosVDSolverCollisionDataComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDSolverCollisionDataComponent* CollisionDataComponent = Cast<UChaosVDSolverCollisionDataComponent>(Component);
	if (!CollisionDataComponent)
	{
		return;
	}

	AChaosVDSolverInfoActor* SolverInfoContainer = Cast<AChaosVDSolverInfoActor>(CollisionDataComponent->GetOwner());
	if (!SolverInfoContainer)
	{
		return;
	}

	UChaosVDParticleDataComponent* ParticleDataComponent = SolverInfoContainer->GetParticleDataComponent();
	if (!ParticleDataComponent)
	{
		return;
	}

	if (!SolverInfoContainer->IsVisible())
	{
		return;
	}

	FChaosVDVisualizationContext VisualizationContext;
	VisualizationContext.SolverID = SolverInfoContainer->GetSolverID();
	VisualizationContext.CVDScene = SolverInfoContainer->GetScene();
	VisualizationContext.SpaceTransform = SolverInfoContainer->GetSimulationTransform();
	VisualizationContext.VisualizationFlags = static_cast<uint32>(UChaosVDCollisionDataVisualizationSettings::GetDataVisualizationFlags());
	VisualizationContext.DebugDrawSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCollisionDataVisualizationSettings>();

	if (!EnumHasAnyFlags(EChaosVDCollisionVisualizationFlags::EnableDraw, static_cast<EChaosVDCollisionVisualizationFlags>(VisualizationContext.VisualizationFlags)))
	{
		return;
	}

	if (!EnumHasAnyFlags(EChaosVDCollisionVisualizationFlags::DrawDataOnlyForSelectedParticle, static_cast<EChaosVDCollisionVisualizationFlags>(VisualizationContext.VisualizationFlags)))
	{
		for (const TSharedPtr<FChaosVDParticlePairMidPhase>& MidPhase : CollisionDataComponent->GetMidPhases())
		{
			DrawMidPhaseData(Component, MidPhase, VisualizationContext, View, PDI);
		}
	}
	else
	{
		ParticleDataComponent->VisitSelectedParticleData([this, PDI, View, &VisualizationContext, CollisionDataComponent](const TSharedPtr<const FChaosVDParticleDataWrapper>& InParticleDataViewer)
		{
			if (const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>* ParticleMidPhases = CollisionDataComponent->GetMidPhasesForParticle(InParticleDataViewer->ParticleIndex, EChaosVDParticlePairSlot::Any))
			{
				for (const TSharedPtr<FChaosVDParticlePairMidPhase>& MidPhase : *ParticleMidPhases)
				{
					DrawMidPhaseData(CollisionDataComponent, MidPhase, VisualizationContext, View, PDI);
				}
			}
			return true;
		});
	}
}

bool FChaosVDSolverCollisionDataComponentVisualizer::CanHandleClick(const HChaosVDComponentVisProxy& VisProxy)
{
	return VisProxy.DataSelectionHandle && VisProxy.DataSelectionHandle->IsA<FChaosVDParticlePairMidPhase>();
}

void FChaosVDSolverCollisionDataComponentVisualizer::DrawMidPhaseData(const UActorComponent* Component, const TSharedPtr<FChaosVDParticlePairMidPhase>& MidPhase, const FChaosVDVisualizationContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	TSharedPtr<FChaosVDScene> ScenePtr = VisualizationContext.CVDScene.Pin();
	if (!ScenePtr.IsValid())
	{
		return;
	}

	const UChaosVDCollisionDataVisualizationSettings* DebugDrawSettings = Cast<UChaosVDCollisionDataVisualizationSettings>(VisualizationContext.DebugDrawSettings);
	if (!DebugDrawSettings)
	{
		return;
	}
	
	if (MidPhase.IsValid())
	{
		TSharedPtr<FChaosVDSceneParticle> ParticleActor = ScenePtr->GetParticleInstance(VisualizationContext.SolverID, MidPhase->Particle0Idx);
		if (!ParticleActor)
		{
			return;
		}
		
		TSharedPtr<const FChaosVDParticleDataWrapper> ParticleDataViewer = ParticleActor->GetParticleData();
		if (!ensure(ParticleDataViewer))
		{
			return;
		}

		TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin();
		if (!ensure(SolverDataSelectionObject))
		{
			return;
		}

		TSharedPtr<FChaosVDSolverDataSelectionHandle> CurrentSelection = SolverDataSelectionObject->GetCurrentSelectionHandle();
		FChaosVDCollisionDataSelectionContext* CurrentSelectionContext = nullptr;

		if (CurrentSelection && CurrentSelection->IsA<FChaosVDParticlePairMidPhase>())
		{
			CurrentSelectionContext = CurrentSelection->GetContextData<FChaosVDCollisionDataSelectionContext>();
		}

		EChaosVDCollisionVisualizationFlags VisualizationFlags = static_cast<EChaosVDCollisionVisualizationFlags>(VisualizationContext.VisualizationFlags);
		
		for (FChaosVDConstraint& Constraint : MidPhase->Constraints)
		{
			int32 ContactIndex = INDEX_NONE;
			for (const FChaosVDManifoldPoint& ManifoldPoint : Constraint.ManifoldPoints)
			{
				ContactIndex++;

				TSharedPtr<FChaosVDSolverDataSelectionHandle> SelectionHandle = SolverDataSelectionObject->MakeSelectionHandle(MidPhase);

				FChaosVDCollisionDataSelectionContext ContextData;
				ContextData.MidPhase = MidPhase;
				ContextData.ConstraintDataPtr = &Constraint;
				ContextData.ContactDataIndex = ContactIndex;

				SelectionHandle->SetHandleContext(MoveTemp(ContextData));

				const bool bIsProbe = Constraint.bIsProbe;
				const bool bIsActive = ManifoldPoint.bIsValid && (!ManifoldPoint.NetPushOut.IsNearlyZero() || !ManifoldPoint.NetImpulse.IsNearlyZero() || (!Constraint.bUseManifold && !Constraint.AccumulatedImpulse.IsNearlyZero()));
				if (!bIsActive && !EnumHasAnyFlags(VisualizationFlags, EChaosVDCollisionVisualizationFlags::DrawInactiveContacts))
				{
					continue;
				}

				const bool bPruned = ManifoldPoint.bDisabled;

				TSharedPtr<FChaosVDSceneParticle> CVDParticleActor1 = ScenePtr->GetParticleInstance(VisualizationContext.SolverID, Constraint.Particle1Index);
				TSharedPtr<FChaosVDSceneParticle> CVDParticleActor0 = ScenePtr->GetParticleInstance(VisualizationContext.SolverID, Constraint.Particle0Index);
				if (CVDParticleActor1 == nullptr || CVDParticleActor0 == nullptr)
				{
					continue;
				}

				PDI->SetHitProxy(new HChaosVDComponentVisProxy(Component, SelectionHandle));

				TSharedPtr<const FChaosVDParticleDataWrapper> ParticleDataActor1 = CVDParticleActor1->GetParticleData();
				TSharedPtr<const FChaosVDParticleDataWrapper> ParticleDataActor0 = CVDParticleActor0->GetParticleData();
				const FTransform WorldActorTransform1 = ParticleDataActor1 && ParticleDataActor1->ParticlePositionRotation.HasValidData() ? FTransform(ParticleDataActor1->ParticlePositionRotation.MR, ParticleDataActor1->ParticlePositionRotation.MX) : FTransform();
				const FTransform WorldActorTransform0 = ParticleDataActor0 && ParticleDataActor0->ParticlePositionRotation.HasValidData() ? FTransform(ParticleDataActor0->ParticlePositionRotation.MR, ParticleDataActor0->ParticlePositionRotation.MX) : FTransform();

				constexpr int32 ContactPlaneOwner = 1;
				constexpr int32 ContactPointOwner = 1 - ContactPlaneOwner;
				const FTransform& PlaneTransform = Constraint.ImplicitTransforms[1] * WorldActorTransform1;
				const FTransform& PointTransform = Constraint.ImplicitTransforms[0] * WorldActorTransform0;

				const FVector PlaneNormal = PlaneTransform.TransformVectorNoScale(FVector(ManifoldPoint.ContactPoint.ShapeContactNormal));
				const FVector PointLocation = PointTransform.TransformPosition(FVector(ManifoldPoint.ContactPoint.ShapeContactPoints[ContactPointOwner]));
				const FVector PlaneLocation = PlaneTransform.TransformPosition(FVector(ManifoldPoint.ContactPoint.ShapeContactPoints[ContactPlaneOwner]));
				const FVector PointPlaneLocation = PointLocation - FVector::DotProduct(PointLocation - PlaneLocation, PlaneNormal) * PlaneNormal;

				// TODO: Extract these colors to a class or another place where ChaosDebugDraw and Chaos VD Visualizers can access
				// The bulk of this implementation including the following colors come from ChaosDebugDraw Implementation.
				// The implementation itself is likely to drift, but we should try to keep the colors the same between both debugging tools

				// Dynamic friction, restitution = red
				// Static friction, no restitution = green
				// Inactive = gray
				FColor DiscColor = FColor(250, 0, 0);
				FColor PhiDiscColor = FColor(128, 128, 0);
				FColor PlaneNormalColor = FColor(250, 0, 0);
				FColor EdgeNormalColor = FColor(250, 150, 0);
				FColor ImpulseColor = FColor(0, 0, 250);
				FColor PushOutImpulseColor = FColor(0, 250, 250);
				FColor InitialPhiColor = FColor(189, 195, 199);
				float ContactLenScale = 1.0f;

				// TODO: Make this a setting when all CVD debug draw options support changing the line thickness
				constexpr float LinesThickness = 2.0f;
				constexpr int32 CircleSegments = 16;
				
				if (ManifoldPoint.bInsideStaticFrictionCone)
				{
					DiscColor = FColor(150, 200, 0);
					PhiDiscColor = FColor(150, 200, 0);
				}
				if (bIsProbe)
				{
					DiscColor = FColor(50, 180, 180);
					PhiDiscColor = FColor(50, 180, 180);
					PlaneNormalColor = FColor(50, 180, 180);
					EdgeNormalColor = FColor(50, 180, 130);
				}
				else if (!bIsActive)
				{
					DiscColor = FColor(100, 100, 100);
					PhiDiscColor = FColor(150, 150, 150);
					PlaneNormalColor = FColor(100, 0, 0);
					EdgeNormalColor = FColor(100, 80, 0);
					ContactLenScale = 0.75;
				}
				if (bPruned)
				{
					DiscColor = FColor(50, 50, 50);
					PhiDiscColor = FColor(50, 50, 50);
					PlaneNormalColor = FColor(200, 0, 200);
					EdgeNormalColor = FColor(200, 0, 200);
					ContactLenScale = 0.5;
				}
			
				const FVector WorldPointLocation = VisualizationContext.SpaceTransform.TransformPosition(PointLocation);
				const FVector WorldPlaneLocation = VisualizationContext.SpaceTransform.TransformPosition(PlaneLocation);
				const FVector WorldPointPlaneLocation = VisualizationContext.SpaceTransform.TransformPosition(PointPlaneLocation);
				const FVector WorldPlaneNormal = VisualizationContext.SpaceTransform.TransformVectorNoScale(PlaneNormal);

				const FMatrix Axes = FRotationMatrix::MakeFromX(WorldPlaneNormal);

				if (EnumHasAnyFlags(VisualizationFlags, EChaosVDCollisionVisualizationFlags::NetPushOut))
				{
					if (ManifoldPoint.bIsValid && !ManifoldPoint.NetPushOut.IsNearlyZero())
					{
						const FText& DebugText = DebugDrawSettings->bShowDebugText ? UEnum::GetDisplayValueAsText(EChaosVDCollisionVisualizationFlags::NetPushOut) : FText::GetEmpty();
						FChaosVDDebugDrawUtils::DrawArrowVector(PDI, WorldPointPlaneLocation, WorldPointPlaneLocation + VisualizationContext.SpaceTransform.TransformPosition(ManifoldPoint.NetPushOut), DebugText, PushOutImpulseColor, DebugDrawSettings->DepthPriority);
					}
				}

				if (EnumHasAnyFlags(VisualizationFlags, EChaosVDCollisionVisualizationFlags::NetImpulse))
				{
					if (ManifoldPoint.bIsValid && !ManifoldPoint.NetImpulse.IsNearlyZero())
					{
						const FText& DebugText = DebugDrawSettings->bShowDebugText ? UEnum::GetDisplayValueAsText(EChaosVDCollisionVisualizationFlags::NetImpulse) : FText::GetEmpty();
						FChaosVDDebugDrawUtils::DrawArrowVector(PDI, WorldPointPlaneLocation, WorldPointPlaneLocation + VisualizationContext.SpaceTransform.TransformPosition(ManifoldPoint.NetImpulse), DebugText, ImpulseColor, DebugDrawSettings->DepthPriority);
					}
				}

				if (EnumHasAnyFlags(VisualizationFlags, EChaosVDCollisionVisualizationFlags::ContactPoints))
				{
					static FText ContactPointsDebugBaseFText = UEnum::GetDisplayValueAsText(EChaosVDCollisionVisualizationFlags::ContactPoints);
					static FText ManifoldPlaneDebugText = FText::FormatOrdered(LOCTEXT("ManifoldPlaneDebugText", "{0} | Manifold Plane"), ContactPointsDebugBaseFText);
					static FText ManifoldPointDebugText = FText::FormatOrdered(LOCTEXT("ManifoldPointDebugText", "{0} | Manifold Point"), ContactPointsDebugBaseFText);
					static FText ManifoldInitialPhiDebugText = FText::FormatOrdered(LOCTEXT("ManifoldInitialPhiDebugText", "{0} | Manifold Initial Phi"), ContactPointsDebugBaseFText);
	
					FChaosVDDebugDrawUtils::DrawCircle(PDI, WorldPlaneLocation, DebugDrawSettings->ContactCircleRadius, CircleSegments, DiscColor, LinesThickness, Axes.GetUnitAxis(EAxis::Y), Axes.GetUnitAxis(EAxis::Z), DebugDrawSettings->bShowDebugText ? ManifoldPlaneDebugText : FText::GetEmpty(), DebugDrawSettings->DepthPriority);
					FChaosVDDebugDrawUtils::DrawCircle(PDI, WorldPointLocation, 0.5f * DebugDrawSettings->ContactCircleRadius, CircleSegments, DiscColor, LinesThickness, Axes.GetUnitAxis(EAxis::Y), Axes.GetUnitAxis(EAxis::Z),  DebugDrawSettings->bShowDebugText ? ManifoldPointDebugText : FText::GetEmpty(), DebugDrawSettings->DepthPriority);
					if (ManifoldPoint.InitialPhi != 0)
					{
						FChaosVDDebugDrawUtils::DrawCircle(PDI, WorldPlaneLocation + ManifoldPoint.InitialPhi * WorldPlaneNormal, 0.25f * DebugDrawSettings->ContactCircleRadius, CircleSegments, InitialPhiColor, LinesThickness, Axes.GetUnitAxis(EAxis::Y), Axes.GetUnitAxis(EAxis::Z),
							DebugDrawSettings->bShowDebugText ? ManifoldInitialPhiDebugText : FText::GetEmpty(), DebugDrawSettings->DepthPriority);
					}
				}

				if (EnumHasAnyFlags(VisualizationFlags, EChaosVDCollisionVisualizationFlags::ContactNormal))
				{
					FColor NormalColor = ((ManifoldPoint.ContactPoint.ContactType != EChaosVDContactPointType::EdgeEdge) ? PlaneNormalColor : EdgeNormalColor);
					const float Scale = DebugDrawSettings->ContactNormalScale * ContactLenScale;
					const FText& DebugText = DebugDrawSettings->bShowDebugText ? UEnum::GetDisplayValueAsText(EChaosVDCollisionVisualizationFlags::ContactNormal) : FText::GetEmpty();
					FChaosVDDebugDrawUtils::DrawArrowVector(PDI, WorldPlaneLocation, WorldPlaneLocation + WorldPlaneNormal * Scale, DebugText, NormalColor, DebugDrawSettings->DepthPriority);
				}

				if (EnumHasAnyFlags(VisualizationFlags, EChaosVDCollisionVisualizationFlags::AccumulatedImpulse))
				{
					if (!Constraint.AccumulatedImpulse.IsNearlyZero())
					{
						const FText& DebugText = DebugDrawSettings->bShowDebugText ? UEnum::GetDisplayValueAsText(EChaosVDCollisionVisualizationFlags::AccumulatedImpulse) : FText::GetEmpty();
						FChaosVDDebugDrawUtils::DrawArrowVector(PDI, WorldActorTransform0.GetLocation(), WorldActorTransform0.GetLocation() + Constraint.AccumulatedImpulse, DebugText, FColor::White, DebugDrawSettings->DepthPriority);
					}
				}

				if (EnumHasAnyFlags(VisualizationFlags, EChaosVDCollisionVisualizationFlags::ContactInfo))
				{
					FTransform Transform;
					Transform.SetRotation(FRotationMatrix::MakeFromZ(WorldPlaneNormal).ToQuat());
					Transform.SetLocation(WorldPlaneLocation);

					FVector BoxExtents(DebugDrawSettings->ContactCircleRadius, DebugDrawSettings->ContactCircleRadius, 0.01f);
					FColor Color;

					if (Constraint.bWasManifoldRestored)
					{
						Color = FColor::Blue;
						FChaosVDDebugDrawUtils::DrawBox(PDI, BoxExtents, Color, Transform, FText::GetEmpty(), DebugDrawSettings->DepthPriority);
					}
					else if (ManifoldPoint.bWasRestored)
					{
						Color = FColor::Purple;
						FChaosVDDebugDrawUtils::DrawBox(PDI, BoxExtents, Color, Transform, FText::GetEmpty(), DebugDrawSettings->DepthPriority);
					}
					else if (ManifoldPoint.bWasReplaced)
					{
						Color = FColor::Orange;
						FChaosVDDebugDrawUtils::DrawBox(PDI, BoxExtents, Color, Transform, FText::GetEmpty(), DebugDrawSettings->DepthPriority);
					}

					if (MidPhase->bIsSleeping)
					{
						// This box should surround the debug draw box we made before
						FChaosVDDebugDrawUtils::DrawBox(PDI, BoxExtents * 1.1f, FColor::Black, Transform, FText::GetEmpty(), DebugDrawSettings->DepthPriority);
					}
				}

				PDI->SetHitProxy(nullptr);

				if (CurrentSelectionContext && CurrentSelectionContext->ConstraintDataPtr == &Constraint && CurrentSelectionContext->ContactDataIndex == ContactIndex)
				{
					// We don't have an easy way to show a contact is selected with debug draw
					// but 3D box surrounding the contact is better than nothing
					FTransform SelectionBoxTransform;
					SelectionBoxTransform.SetRotation(FRotationMatrix::MakeFromZ(WorldPlaneNormal).ToQuat());
					SelectionBoxTransform.SetLocation(WorldPlaneLocation);

					// The Selection box should be a bit bigger than the configured circle radius for the debug draw contact
					const float ContactSelectionBoxSize = DebugDrawSettings->ContactCircleRadius * 1.5f;

					FVector SelectionBoxExtents(ContactSelectionBoxSize,ContactSelectionBoxSize,ContactSelectionBoxSize);
					FChaosVDDebugDrawUtils::DrawBox(PDI, SelectionBoxExtents, FColor::Yellow, SelectionBoxTransform, FText::GetEmpty(), DebugDrawSettings->DepthPriority);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
