// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Visualizers/ChaosVDComponentVisualizerBase.h"
#include "ChaosVDSolverDataSelection.h"
#include "ComponentVisualizer.h"
#include "HitProxies.h"

#include "ChaosVDSolverCollisionDataComponentVisualizer.generated.h"

class FChaosVDScene;

struct FChaosVDConstraint;
struct FChaosVDParticlePairMidPhase;
struct FChaosVDVisualizationContext;

USTRUCT()
struct FChaosVDCollisionDataSelectionContext : public FChaosVDSelectionContext
{
	GENERATED_BODY()

	TSharedPtr<FChaosVDParticlePairMidPhase> MidPhase;
	FChaosVDConstraint* ConstraintDataPtr = nullptr;
	int32 ContactDataIndex = INDEX_NONE;
};

class FChaosVDSolverCollisionDataComponentVisualizer : public FChaosVDComponentVisualizerBase
{
public:
	
	FChaosVDSolverCollisionDataComponentVisualizer();
	virtual ~FChaosVDSolverCollisionDataComponentVisualizer() override;

	virtual void RegisterVisualizerMenus() override;

	virtual bool ShouldShowForSelectedSubcomponents(const UActorComponent* Component) override;
	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;

	virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy) override;

protected:
	void DrawMidPhaseData(const UActorComponent* Component, const TSharedPtr<FChaosVDParticlePairMidPhase>& MidPhase, const FChaosVDVisualizationContext& VisualizationContext, const FSceneView* View, FPrimitiveDrawInterface* PDI);
};
