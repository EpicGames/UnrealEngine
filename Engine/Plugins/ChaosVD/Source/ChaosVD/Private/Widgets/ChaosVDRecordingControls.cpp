// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/ChaosVDRecordingControls.h"

#include "Chaos/ChaosVDEngineEditorBridge.h"
#include "Chaos/ChaosVDRemoteSessionsManager.h"
#include "ChaosVisualDebugger/ChaosVDOptionalDataChannel.h"
#include "ChaosVDEngine.h"
#include "ChaosVDModule.h"
#include "ChaosVDSettingsManager.h"
#include "ChaosVDStyle.h"
#include "SEnumCombo.h"
#include "Settings/ChaosVDGeneralSettings.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/SChaosVDMainTab.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

class UChaosVDGeneralSettings;

FChaosVDRecordingControls::FChaosVDRecordingControls(const TSharedRef<SChaosVDMainTab>& InMainTab
	, const TSharedRef<UE::TraceBasedDebuggers::FRemoteSessionsManager>& InRemoteSessionManager)
	: FRecordingControls(InMainTab->GetStatusBarName(), InRemoteSessionManager, Chaos::VD::DebuggerGuid, LogChaosVDEditor)
{
#if WITH_TRACE_BASED_DEBUGGERS
	TraceRelayTransportInstanceWeakPtr = FChaosVDEngineEditorBridge::Get().GetTraceRelayTransportInstance();
#endif

	MainTabWeakPtr = InMainTab;
	CurrentLoadingMode = EChaosVDLoadRecordedDataMode::SingleSource;

	StartRecordingBrush = FChaosVDStyle::Get().GetBrush("RecordIcon");
	StopRecordingBrush = FChaosVDStyle::Get().GetBrush("StopIcon");
	RecordingSessionIcon = FSlateIcon(FChaosVDStyle::GetStyleSetName(), FName("RecordIcon"), FName("RecordIcon"));
}

void FChaosVDRecordingControls::AddToMenuInternal(FToolMenuSection& InSection)
{
	FToolMenuEntry& AdvancedRecordingOptionsEntry = InSection.AddEntry(
		FToolMenuEntry::InitWidget(
			"AdvancedRecordingOptions",
			SNew(SBox).Padding(1.0f, 0.0f)
			[
				GenerateAdvancedRecordOptionsButton()
			],
			FText::GetEmpty(),
			true,
			false
		));
	AdvancedRecordingOptionsEntry.InsertPosition.Position = EToolMenuInsertType::Before;
	AdvancedRecordingOptionsEntry.InsertPosition.Name = "RecordingTime";

	FToolMenuEntry& DataChannelsEntry = InSection.AddEntry(
		FToolMenuEntry::InitWidget(
			"DataChannelsButton",
			SNew(SBox).Padding(4.0f, 0.0f)
			[
				GenerateDataChannelsButton()
			],
			FText::GetEmpty(),
			false,
			false
		));

	DataChannelsEntry.InsertPosition.Position = EToolMenuInsertType::Before;
	DataChannelsEntry.InsertPosition.Name = "RecordingTime";
}

void FChaosVDRecordingControls::OnGenerateTargetSessionSelectorInternal(FMenuBuilder& MenuBuilder)
{
	using namespace UE::TraceBasedDebuggers;
	MenuBuilder.BeginSection("CVDRecordingWidgetTargetsMulti", LOCTEXT("CVDRecordingMultiTargetsMenu", "Multi Target"));

	const FText AllRemoteTargetsLabel = LOCTEXT("AllRemoteOption", "All Remote");
	MenuBuilder.AddMenuEntry(
			AllRemoteTargetsLabel,
			LOCTEXT("MultiRemoteTargetTooltip", "Select this to act on all remote targets"),
			GetIconForSession(FRemoteSessionsManager::AllRemoteSessionsWrapperGUID),
			FUIAction(FExecuteAction::CreateSP(this, &FChaosVDRecordingControls::SelectTargetSession, FRemoteSessionsManager::AllRemoteSessionsWrapperGUID),
			FCanExecuteAction::CreateSP(this, &FChaosVDRecordingControls::CanSelectMultiSessionTarget, FRemoteSessionsManager::AllRemoteSessionsWrapperGUID),
			EUIActionRepeatMode::RepeatDisabled));

	const FText AllRemoteServersTargetsLabel = LOCTEXT("AllRemoteServersOption", "All Remote Servers");
	MenuBuilder.AddMenuEntry(
			AllRemoteServersTargetsLabel,
			LOCTEXT("MultiRemoteServerTargetTooltip", "Select this to act on all remote server targets"),
			GetIconForSession(FRemoteSessionsManager::AllRemoteServersWrapperGUID),
			FUIAction(FExecuteAction::CreateSP(this, &FChaosVDRecordingControls::SelectTargetSession, FRemoteSessionsManager::AllRemoteServersWrapperGUID),
			FCanExecuteAction::CreateSP(this, &FChaosVDRecordingControls::CanSelectMultiSessionTarget, FRemoteSessionsManager::AllRemoteServersWrapperGUID),
			EUIActionRepeatMode::RepeatDisabled));

	const FText AllRemoteClientsTargetsLabel = LOCTEXT("AllRemoteClientsOption", "All Remote Clients");
	MenuBuilder.AddMenuEntry(
			AllRemoteClientsTargetsLabel,
			LOCTEXT("MultiRemoteClientTargetTooltip", "Select this to act on all remote client targets"),
			GetIconForSession(FRemoteSessionsManager::AllRemoteClientsWrapperGUID),
			FUIAction(FExecuteAction::CreateSP(this, &FChaosVDRecordingControls::SelectTargetSession, FRemoteSessionsManager::AllRemoteClientsWrapperGUID),
			FCanExecuteAction::CreateSP(this, &FChaosVDRecordingControls::CanSelectMultiSessionTarget, FRemoteSessionsManager::AllRemoteClientsWrapperGUID),
			EUIActionRepeatMode::RepeatDisabled));

	const FText AllTargets = LOCTEXT("AllTargetsOption", "All");
	MenuBuilder.AddMenuEntry(
		AllTargets,
		LOCTEXT("MultiAllTargetTooltip", "Select this to act on all targets, both Local and Remote"),
		GetIconForSession(FRemoteSessionsManager::AllSessionsWrapperGUID),
		FUIAction(FExecuteAction::CreateSP(this, &FChaosVDRecordingControls::SelectTargetSession, FRemoteSessionsManager::AllSessionsWrapperGUID), EUIActionRepeatMode::RepeatDisabled));

	MenuBuilder.AddMenuSeparator();

	const FText CustomTargets = LOCTEXT("CustomTargetsOption", "Custom Selection");
	MenuBuilder.AddSubMenu(CustomTargets,
		LOCTEXT("MultiCustomTargetTooltip", "Select this to act on the specific targets you selected"),
		FNewMenuDelegate::CreateSP(this, &FChaosVDRecordingControls::GenerateCustomTargetsMenu)
		);

	MenuBuilder.EndSection();
}

FString FChaosVDRecordingControls::GetSaveDirPath() const
{
	if (const TSharedPtr<UE::TraceBasedDebuggers::FTraceSessionsManager>& TraceSessionsManager = FChaosVDModule::Get().GetTraceSessionsManager())
	{
		return TraceSessionsManager->GetSaveDirPath();
	}
	return TEXT("");
}

int32 FChaosVDRecordingControls::GetMaxConnectionRetriesInternal() const
{
	if (const UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		return Settings->MaxConnectionRetries;
	}

	const int32 DefaultValue = FRecordingControls::GetMaxConnectionRetriesInternal();
	UE_LOGF(LogChaosVDEditor, Warning, "Failed to obtain setting object. Setting the retries attempts to connect to a session to %d as a fallback.", DefaultValue);
	return DefaultValue;
}

UE::TraceBasedDebuggers::ETraceTransportMode FChaosVDRecordingControls::GetTransportModeOverrideForTargetTypeInternal(const EBuildTargetType InBuildTarget) const
{
	if (const UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		return Settings->GetTransportModeForTargetType(InBuildTarget);
	}

	UE::TraceBasedDebuggers::ETraceTransportMode DefaultValue = FRecordingControls::GetTransportModeOverrideForTargetTypeInternal(InBuildTarget);
	UE_LOGF(LogChaosVDEditor, Warning, "Failed to obtain setting object. Setting the retries attempts to connect to a session to %ls as a fallback.", *UEnum::GetValueAsString(DefaultValue));
	return DefaultValue;
}

bool FChaosVDRecordingControls::ConnectToLiveSession_RelayInternal(const FGuid& InSessionId)
{
	if (const TSharedPtr<SChaosVDMainTab> MainTabSharedPtr = MainTabWeakPtr.Pin())
	{
		return MainTabSharedPtr->ConnectToLiveSession_Relay(InSessionId, CurrentLoadingMode);
	}

	return false;
}

bool FChaosVDRecordingControls::ConnectToLiveSession_DirectInternal(const FGuid& InSessionId)
{
	if (const TSharedPtr<SChaosVDMainTab> MainTabSharedPtr = MainTabWeakPtr.Pin())
	{
		return MainTabSharedPtr->ConnectToLiveSession_Direct(InSessionId, CurrentLoadingMode);
	}

	return false;
}

TNotNull<TUniquePtr<UE::TraceBasedDebuggers::FStartRecordingCommandMessage>> FChaosVDRecordingControls::BuildStartRecordingParamsInternal()
{
	TNotNull<TUniquePtr<FChaosVDStartRecordingCommandMessage>> RecordingParams = MakeUnique<FChaosVDStartRecordingCommandMessage>();

#if WITH_CHAOS_VISUAL_DEBUGGER
	Chaos::VisualDebugger::FChaosVDDataChannelsManager::Get().EnumerateDataChannels([&RecordingParams](const TWeakPtr< Chaos::VisualDebugger::FChaosVDOptionalDataChannel>& Channel)
	{
		if (TSharedPtr< Chaos::VisualDebugger::FChaosVDOptionalDataChannel> LockedChannel = Channel.Pin())
		{
			if (LockedChannel->IsChannelEnabled())
			{
				RecordingParams->DataChannelsEnabledOverrideList.Add(LockedChannel->GetId().ToString());
			}
		}
		return true;
	});
#endif

	return RecordingParams;
}

void FChaosVDRecordingControls::SendStartRecordingCommandInternal(const FMessageAddress& InAddress
	, TUniquePtr<UE::TraceBasedDebuggers::FStartRecordingCommandMessage> InStartRecordingCommand)
{
	if (const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> SessionsManager = RemoteSessionManagerWeakPtr.Pin())
	{
		const FChaosVDStartRecordingCommandMessage* StartRecordingCommand = static_cast<FChaosVDStartRecordingCommandMessage*>(InStartRecordingCommand.Get());
		SessionsManager->SendCommand(InAddress, *StartRecordingCommand);
	}
}

void FChaosVDRecordingControls::SendStopRecordingCommandInternal(const FMessageAddress& InAddress)
{
	if (const TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> SessionsManager = RemoteSessionManagerWeakPtr.Pin())
	{
		SessionsManager->SendCommand(InAddress, FChaosVDStopRecordingCommandMessage{});
	}
}

bool FChaosVDRecordingControls::IsRecordingAvailableForSessionInternal(const TSharedRef<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo) const
{
	// If we are recording, don't show the stop button for the mode that is disabled
	if (EnumHasAnyFlags(InSessionInfo->GetSessionTypeAttributes(), UE::TraceBasedDebuggers::ERemoteSessionAttributes::IsMultiSessionWrapper))
	{
		TSharedPtr<UE::TraceBasedDebuggers::FMultiSessionInfo> AsMultiSessionInfo = StaticCastSharedPtr<UE::TraceBasedDebuggers::FMultiSessionInfo>(InSessionInfo.ToSharedPtr());

		return CanSelectMultiSessionTarget(InSessionInfo);
	}

	return true;
}

void FChaosVDRecordingControls::OnToggleRecordingStateInternal(const TSharedRef<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo)
{
	if (EnumHasAnyFlags(InSessionInfo->GetSessionTypeAttributes(), UE::TraceBasedDebuggers::ERemoteSessionAttributes::IsMultiSessionWrapper))
	{
		ToggleMultiSessionSessionRecordingState(StaticCastSharedPtr<UE::TraceBasedDebuggers::FMultiSessionInfo>(InSessionInfo.ToSharedPtr()).ToSharedRef());
	}
	else
	{
		ToggleSingleSessionRecordingState(InSessionInfo);
	}
}

void FChaosVDRecordingControls::CloseSessionByRemoteSessionIDInternal(const FGuid& InSessionId)
{
	if (const TSharedPtr<SChaosVDMainTab> MainTab = MainTabWeakPtr.Pin())
	{
		MainTab->GetChaosVDEngineInstance()->CloseSessionByRemoteSessionID(InSessionId);
	}
}

const UE::TraceBasedDebuggers::FTraceSessionDescriptor& FChaosVDRecordingControls::GetCurrentSessionDescriptorInternal()
{
	if (const TSharedPtr<SChaosVDMainTab> MainTab = MainTabWeakPtr.Pin())
	{
		return MainTab->GetChaosVDEngineInstance()->GetTraceSessionDescriptors().Last();
	}

	static UE::TraceBasedDebuggers::FTraceSessionDescriptor InvalidDescriptor;
	return InvalidDescriptor;
}

TSharedRef<SWidget> FChaosVDRecordingControls::GenerateAdvancedRecordOptionsButton()
{
	return SNew(SComboButton)
		.ContentPadding(FMargin(6.0f, 0.0f))
		.IsEnabled_Raw(this, &FChaosVDRecordingControls::IsRecordingToggleButtonEnabled) // This button needs to follow the enabled condition of the recording button
		.MenuPlacement(MenuPlacement_AboveAnchor).ComboButtonStyle(&FAppStyle::Get()
		.GetWidgetStyle<FComboButtonStyle>("ComboButton"))
		.OnGetMenuContent(this, &FChaosVDRecordingControls::GenerateAdvancedRecordOptionsMenu)
		.HasDownArrow(true);
}

TSharedRef<SWidget> FChaosVDRecordingControls::GenerateAdvancedRecordOptionsMenu()
{
	using namespace Chaos::VisualDebugger;

	FMenuBuilder MenuBuilder(true, nullptr);

	constexpr bool bSearchable = true;
	constexpr bool bNoIndent = true;

	MenuBuilder.BeginSection("CVDRecordingWidget", LOCTEXT("CVDAdvancedRecordingMenu", "Advanced"));

	{
		FUIAction AutoStartPlaybackUIAction;
		AutoStartPlaybackUIAction.ExecuteAction = FExecuteAction::CreateSP(this, &FChaosVDRecordingControls::ToggleAutoStartPlayback);
		AutoStartPlaybackUIAction.GetActionCheckState = FGetActionCheckState::CreateStatic(&FUIAction::IsActionCheckedPassthrough, FIsActionChecked::CreateSP(this, &FChaosVDRecordingControls::IsAutoStartPlaybackEnabled));
	
		MenuBuilder.AddMenuEntry(
			LOCTEXT("AutoStartPlaybackLabel", "Auto Start Playback"),
			LOCTEXT("AutoStartPlaybackToolTip", "If set to true, playback will start automatically as soon the recording is started and the connection is successfully established"),
			FSlateIcon(),
			AutoStartPlaybackUIAction, NAME_None, EUserInterfaceActionType::ToggleButton);
	}

	{
		FUIAction SaveToDiskUIAction;
		SaveToDiskUIAction.ExecuteAction = FExecuteAction::CreateSP(this, &FChaosVDRecordingControls::ToggleSaveTracesToDisk);
		SaveToDiskUIAction.GetActionCheckState = FGetActionCheckState::CreateStatic(&FUIAction::IsActionCheckedPassthrough, FIsActionChecked::CreateSP(this, &FChaosVDRecordingControls::IsSaveTracesToDiskEnabled));

		MenuBuilder.AddMenuEntry(
		LOCTEXT("AutoSaveToDiskLabel", "Save Recorded Data to Disk"),
		LOCTEXT("AutoSaveToDiskToolTip", "If set to true, any recorded data for the session will be saved automatically as soon it is recorded."),
		FSlateIcon(),
		SaveToDiskUIAction, NAME_None, EUserInterfaceActionType::ToggleButton);
	}

	MenuBuilder.AddSeparator();
	
	MenuBuilder.AddWidget(
			GenerateLoadingModeSelector(),
			LOCTEXT("LoadingModeLabel", "Loading Mode"),
			bSearchable,
			bNoIndent,
			LOCTEXT("LoadingModeLabelTooltip", "Alters if the newly recorded data should unload previous loaded data (Single Source), or just add to it (Multi Source)"));

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> FChaosVDRecordingControls::GenerateDataChannelsButton()
{
	return SNew(SComboButton)
			.ContentPadding(FMargin(6.0f, 0.0f))
			.IsEnabled_Raw(this, &FChaosVDRecordingControls::HasDataChannelsSupport)
			.MenuPlacement(MenuPlacement_AboveAnchor).ComboButtonStyle(&FAppStyle::Get()
			.GetWidgetStyle<FComboButtonStyle>("SimpleComboButton"))
			.OnGetMenuContent(this, &FChaosVDRecordingControls::GenerateDataChannelsMenu)
			.HasDownArrow(true)
			.ButtonContent()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.VAlign(VAlign_Center)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text(LOCTEXT("DataChannelsButton", "Data Channels"))
					.TextStyle(&FAppStyle::Get().GetWidgetStyle<FTextBlockStyle>("DialogButtonText"))
				]
			];
}

TSharedRef<SWidget> FChaosVDRecordingControls::GenerateLoadingModeSelector()
{
	TAttribute<int32> GetCurrentValueAttribute;
	GetCurrentValueAttribute.BindSPLambda(this, [this]()
	{
		return static_cast<int32>(CurrentLoadingMode);
	});

	SEnumComboBox::FOnEnumSelectionChanged EnumValueChangedDelegate = SEnumComboBox::FOnEnumSelectionChanged::CreateSPLambda(this, [this](int32 NewValue, ESelectInfo::Type SelectionType)
	{
		CurrentLoadingMode = static_cast<EChaosVDLoadRecordedDataMode>(NewValue);
	});
	
	return SNew(SEnumComboBox, StaticEnum<EChaosVDLoadRecordedDataMode>())
		.IsEnabled_Raw(this, &FChaosVDRecordingControls::CanChangeLoadingMode)
		.CurrentValue(GetCurrentValueAttribute)
		.OnEnumSelectionChanged(EnumValueChangedDelegate);
}

TSharedRef<SWidget> FChaosVDRecordingControls::GenerateDataChannelsMenu()
{
	using namespace UE::TraceBasedDebuggers;

	FMenuBuilder MenuBuilder(true, nullptr);

	MenuBuilder.BeginSection("CVDRecordingWidget", LOCTEXT("CVDRecordingMenuChannels", "Data Channels"));
	{
		if (const FChaosVDSessionData* SessionData = GetChaosDataForCurrentSession())
		{
			for (const TPair<FString, FChaosVDDataChannelState>& DataChannelStateWithName : SessionData->DataChannelsStatesByName)
			{
				FText ChannelNamesAsText = FText::AsCultureInvariant(DataChannelStateWithName.Key);
				MenuBuilder.AddMenuEntry(
				ChannelNamesAsText,
				FText::Format(LOCTEXT("ChannelDesc", "Enable/disable the {0} channel"), ChannelNamesAsText),
				FSlateIcon(),
				FUIAction(FExecuteAction::CreateSP(this, &FChaosVDRecordingControls::ToggleChannelEnabledState, DataChannelStateWithName.Key),
					FCanExecuteAction::CreateSP(this, &FChaosVDRecordingControls::CanChangeChannelEnabledState, DataChannelStateWithName.Key), FIsActionChecked::CreateSP(this, &FChaosVDRecordingControls::IsChannelEnabled, DataChannelStateWithName.Key)), NAME_None, EUserInterfaceActionType::ToggleButton);
			}
		}
	}

	MenuBuilder.EndSection();

	return MenuBuilder.MakeWidget();
}

void FChaosVDRecordingControls::ToggleChannelEnabledState(FString ChannelName)
{
	using namespace UE::TraceBasedDebuggers;
	const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager();
	const TSharedPtr<FSessionInfo> SessionInfo = GetCurrentSessionInfo();
	if (RemoteSessionManager && SessionInfo)
	{
		if (FChaosVDSessionData* ChaosData = SessionInfo->GetDebuggerData<FChaosVDSessionData>())
		{
			if (FChaosVDDataChannelState* ChannelState = ChaosData->DataChannelsStatesByName.Find(ChannelName))
			{
				ChannelState->bWaitingUpdatedState = true;
				RemoteSessionManager->SendCommand(SessionInfo->Address, FChaosVDChannelStateChangeCommandMessage
					{
						FChaosVDDataChannelState{.ChannelName = ChannelState->ChannelName, .bIsEnabled = !ChannelState->bIsEnabled}
					});
			}
		}
	}
}

bool FChaosVDRecordingControls::IsChannelEnabled(FString ChannelName)
{
	using namespace UE::TraceBasedDebuggers;
	if (const FChaosVDSessionData* ChaosData = GetChaosDataForCurrentSession())
	{
		if (const FChaosVDDataChannelState* ChannelState = ChaosData->DataChannelsStatesByName.Find(ChannelName))
		{
			return ChannelState->bIsEnabled;
		}
	}

	return false;
}

bool FChaosVDRecordingControls::CanChangeChannelEnabledState(FString ChannelName)
{
	using namespace UE::TraceBasedDebuggers;
	if (const FChaosVDSessionData* ChaosData = GetChaosDataForCurrentSession())
	{
		if (const FChaosVDDataChannelState* ChannelState = ChaosData->DataChannelsStatesByName.Find(ChannelName))
		{
			return ChannelState->bCanChangeChannelState && !ChannelState->bWaitingUpdatedState;
		}
	}
	
	return false;
}

void FChaosVDRecordingControls::GenerateCustomTargetsMenu(FMenuBuilder& MenuBuilder)
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		RemoteSessionManager->EnumerateActiveSessions([this, &MenuBuilder](const TSharedRef<FSessionInfo>& InSessionInfoRef)
		{
			if (EnumHasAnyFlags(InSessionInfoRef->GetSessionTypeAttributes(), ERemoteSessionAttributes::IsMultiSessionWrapper))
			{
				return true;
			}

			const FText SessionNameAsText = FText::AsCultureInvariant(InSessionInfoRef->SessionName);
			MenuBuilder.AddMenuEntry(
			SessionNameAsText,
			FText::Format(LOCTEXT("MultiTargetItemTooltip", "Select {0} session as one of the current targets"), SessionNameAsText),
			GetIconForSession(InSessionInfoRef->InstanceId),
			FUIAction(
			FExecuteAction::CreateSP(this, &FChaosVDRecordingControls::ToggleSessionSelectionInCustomTarget, InSessionInfoRef->InstanceId),
			FCanExecuteAction::CreateSP(this, &FChaosVDRecordingControls::CanSelectInCustomTarget, InSessionInfoRef->InstanceId),
			FIsActionChecked::CreateSP(this, &FChaosVDRecordingControls::IsSessionPartOfCustomTargetSelection, InSessionInfoRef->InstanceId)),
			NAME_None, EUserInterfaceActionType::ToggleButton);

			return true;
		});
	}
}

bool FChaosVDRecordingControls::IsSessionPartOfCustomTargetSelection(FGuid SessionGuid)
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		if (TSharedPtr<FMultiSessionInfo> CustomSessionTarget = StaticCastSharedPtr<FMultiSessionInfo>(RemoteSessionManager->GetSessionInfo(FRemoteSessionsManager::CustomSessionsWrapperGUID).Pin()))
		{
			return CustomSessionTarget->InnerSessionsByInstanceID.Find(SessionGuid) != nullptr;
		}
	}

	return false;
}

void FChaosVDRecordingControls::ToggleSessionSelectionInCustomTarget(FGuid SessionGuid)
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		if (TSharedPtr<FMultiSessionInfo> CustomSessionTarget = StaticCastSharedPtr<FMultiSessionInfo>(RemoteSessionManager->GetSessionInfo(FRemoteSessionsManager::CustomSessionsWrapperGUID).Pin()))
		{
			if (CustomSessionTarget->InnerSessionsByInstanceID.Find(SessionGuid) != nullptr)
			{
				CustomSessionTarget->InnerSessionsByInstanceID.Remove(SessionGuid);
			}
			else
			{
				CustomSessionTarget->InnerSessionsByInstanceID.Add(SessionGuid, RemoteSessionManager->GetSessionInfo(SessionGuid));
			}

			SelectTargetSession(FRemoteSessionsManager::CustomSessionsWrapperGUID);
		}
	}
}

bool FChaosVDRecordingControls::CanSelectInCustomTarget(FGuid SessionGuid) const
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		if (const TSharedPtr<FSessionInfo> CustomSessionTarget = RemoteSessionManager->GetSessionInfo(SessionGuid).Pin())
		{
			return CustomSessionTarget->ReadyState == ERemoteSessionReadyState::Ready;
		}
	}

	return false;
}

bool FChaosVDRecordingControls::CanSelectMultiSessionTarget(FGuid SessionGuid) const
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		if (const TSharedPtr<FSessionInfo> CustomSessionTarget = RemoteSessionManager->GetSessionInfo(SessionGuid).Pin())
		{
			return CanSelectMultiSessionTarget(CustomSessionTarget.ToSharedRef());
		}
	}

	return false;
}

bool FChaosVDRecordingControls::CanSelectMultiSessionTarget(const TSharedRef<UE::TraceBasedDebuggers::FSessionInfo>& SessionInfoRef) const
{
	using namespace UE::TraceBasedDebuggers;
	if (EnumHasAnyFlags(SessionInfoRef->GetSessionTypeAttributes(), ERemoteSessionAttributes::IsMultiSessionWrapper))
	{
		const TSharedRef<FMultiSessionInfo> AsMultiSessionInfo = StaticCastSharedRef<FMultiSessionInfo>(SessionInfoRef);
		return !AsMultiSessionInfo->InnerSessionsByInstanceID.IsEmpty();
	}

	return false;
}

void FChaosVDRecordingControls::ToggleAutoStartPlayback()
{
	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		FProperty* AutoStartProperty = FindFieldChecked<FProperty>(UChaosVDGeneralSettings::StaticClass(), "bAutoStartLiveSessionsPlayback");
		Settings->PreEditChange(AutoStartProperty);
		Settings->bAutoStartLiveSessionsPlayback = !Settings->bAutoStartLiveSessionsPlayback;
		Settings->PostEditChange();
	}
}

bool FChaosVDRecordingControls::IsAutoStartPlaybackEnabled() const
{
	const UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>();
	return Settings ? Settings->bAutoStartLiveSessionsPlayback : false;
}

void FChaosVDRecordingControls::ToggleSaveTracesToDisk()
{
	if (UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		FProperty* SaveToDiskProperty = FindFieldChecked<FProperty>(UChaosVDGeneralSettings::StaticClass(), "bSaveRecordingsToDisk");
		Settings->PreEditChange(SaveToDiskProperty);
		Settings->bSaveRecordingsToDisk = !Settings->bSaveRecordingsToDisk;
		Settings->PostEditChange();
	}
}

bool FChaosVDRecordingControls::IsSaveTracesToDiskEnabled()
{
	const UChaosVDGeneralSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>();
	return Settings ? Settings->bSaveRecordingsToDisk : false;
}

FChaosVDSessionData* FChaosVDRecordingControls::GetChaosDataForCurrentSession() const
{
	if (const TSharedPtr<UE::TraceBasedDebuggers::FSessionInfo> SessionInfo = GetCurrentSessionInfo())
	{
		return SessionInfo->GetDebuggerData<FChaosVDSessionData>();
	}
	return nullptr;
}

bool FChaosVDRecordingControls::HasDataChannelsSupport() const
{
	if (const FChaosVDSessionData* SessionData = GetChaosDataForCurrentSession())
	{
		return !SessionData->DataChannelsStatesByName.IsEmpty();
	}
	return false;
}

bool FChaosVDRecordingControls::CanChangeLoadingMode() const
{
	using namespace UE::TraceBasedDebuggers;
	if (const TSharedPtr<FSessionInfo> CurrentSession = GetCurrentSessionInfo())
	{
		// In multi session mode targets, the loading mode is controlled automatically
		if (EnumHasAnyFlags(CurrentSession->GetSessionTypeAttributes(), ERemoteSessionAttributes::IsMultiSessionWrapper))
		{
			return false;
		}
		else if (const TSharedPtr<SChaosVDMainTab> MainTabSharedPtr = MainTabWeakPtr.Pin())
		{
			// If nothing is loaded yet, it does not make sense change the loading mode
			return !MainTabSharedPtr->GetChaosVDEngineInstance()->GetTraceSessionDescriptors().IsEmpty();
		}
	}

	return false;
}

void FChaosVDRecordingControls::ToggleMultiSessionSessionRecordingState(const TSharedRef<UE::TraceBasedDebuggers::FMultiSessionInfo>& InSessionInfoRef)
{
	const TSharedPtr<SChaosVDMainTab> MainTabSharedPtr = MainTabWeakPtr.Pin();
	if (!ensure(MainTabSharedPtr))
	{
		return;
	}

	bool bNewRecordingState = !IsRecording();

	if (bNewRecordingState)
	{
		CurrentLoadingMode = EChaosVDLoadRecordedDataMode::MultiSource;
		MainTabSharedPtr->GetChaosVDEngineInstance()->CloseActiveTraceSessions();
	}

	InSessionInfoRef->EnumerateInnerSessions([this, bNewRecordingState](const TSharedRef<UE::TraceBasedDebuggers::FSessionInfo>& InInnerSessionRef)
	{
		SetSessionRecordingState(bNewRecordingState, InInnerSessionRef);
		return true;
	});
}

#undef LOCTEXT_NAMESPACE
