// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/RecordingControls.h"

namespace UE::TraceBasedDebuggers
{
struct FMultiSessionInfo;
}

namespace Chaos::VisualDebugger
{
struct FChaosVDOptionalDataChannel;
}

enum class EChaosVDLoadRecordedDataMode : uint8;
struct FChaosVDSessionData;
class FMenuBuilder;
class SChaosVDMainTab;

typedef Chaos::VisualDebugger::FChaosVDOptionalDataChannel FCVDDataChannel;

class FChaosVDRecordingControls : public UE::TraceBasedDebuggers::FRecordingControls
{
public:

	FChaosVDRecordingControls(const TSharedRef<SChaosVDMainTab>& InMainTab, const TSharedRef<UE::TraceBasedDebuggers::FRemoteSessionsManager>& InRemoteSessionManager);

protected:
	//~ Begin FRecordingControls interface
	virtual void AddToMenuInternal(FToolMenuSection& InSection) override;
	virtual void OnGenerateTargetSessionSelectorInternal(FMenuBuilder& InMenuBuilder) override;
	virtual FString GetSaveDirPath() const override;

	virtual int32 GetMaxConnectionRetriesInternal() const override;
	virtual UE::TraceBasedDebuggers::ETraceTransportMode GetTransportModeOverrideForTargetTypeInternal(EBuildTargetType InBuildTarget) const override;

	virtual bool ConnectToLiveSession_RelayInternal(const FGuid& InSessionId) override;
	virtual bool ConnectToLiveSession_DirectInternal(const FGuid& InSessionId) override;

	[[nodiscard]] virtual TNotNull<TUniquePtr<UE::TraceBasedDebuggers::FStartRecordingCommandMessage>> BuildStartRecordingParamsInternal() override;
	virtual void SendStartRecordingCommandInternal(const FMessageAddress& InAddress, TUniquePtr<UE::TraceBasedDebuggers::FStartRecordingCommandMessage> InStartRecordingCommand) override;
	virtual void SendStopRecordingCommandInternal(const FMessageAddress& InAddress) override;

	virtual bool IsRecordingAvailableForSessionInternal(const TSharedRef<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo) const override;
	virtual void OnToggleRecordingStateInternal(const TSharedRef<UE::TraceBasedDebuggers::FSessionInfo>& InSessionInfo) override;

	virtual void CloseSessionByRemoteSessionIDInternal(const FGuid& InSessionId) override;

	virtual const UE::TraceBasedDebuggers::FTraceSessionDescriptor& GetCurrentSessionDescriptorInternal() override;
	//~ End FRecordingControls interface

	TSharedRef<SWidget> GenerateDataChannelsMenu();
	TSharedRef<SWidget> GenerateDataChannelsButton();
	TSharedRef<SWidget> GenerateLoadingModeSelector();
	TSharedRef<SWidget> GenerateAdvancedRecordOptionsButton();
	TSharedRef<SWidget> GenerateAdvancedRecordOptionsMenu();

	void ToggleChannelEnabledState(FString ChannelName);
	bool IsChannelEnabled(FString ChannelName);
	bool CanChangeChannelEnabledState(FString ChannelName);

	void GenerateCustomTargetsMenu(FMenuBuilder& MenuBuilder);

	bool IsSessionPartOfCustomTargetSelection(FGuid SessionGuid);
	void ToggleSessionSelectionInCustomTarget(FGuid SessionGuid);
	bool CanSelectInCustomTarget(FGuid SessionGuid) const;

	bool CanSelectMultiSessionTarget(FGuid SessionGuid) const;
	bool CanSelectMultiSessionTarget(const TSharedRef<UE::TraceBasedDebuggers::FSessionInfo>& SessionInfoRef) const;

	void ToggleAutoStartPlayback();
	bool IsAutoStartPlaybackEnabled() const;
	void ToggleSaveTracesToDisk();
	bool IsSaveTracesToDiskEnabled();

	FChaosVDSessionData* GetChaosDataForCurrentSession() const;

	bool HasDataChannelsSupport() const;

	bool CanChangeLoadingMode() const;

	void ToggleMultiSessionSessionRecordingState(const TSharedRef<UE::TraceBasedDebuggers::FMultiSessionInfo>& InSessionInfoRef);

	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr;

	EChaosVDLoadRecordedDataMode CurrentLoadingMode;
};
