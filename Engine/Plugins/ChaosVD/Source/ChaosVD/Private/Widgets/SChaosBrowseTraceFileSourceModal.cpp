// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosBrowseTraceFileSourceModal.h"

#include "ChaosVDModule.h"
#include "Editor.h"
#include "SEnumCombo.h"
#include "GenericPlatform/GenericPlatformMisc.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/SChaosVDNameListPicker.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SButton.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SChaosBrowseTraceFileSourceModal)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosBrowseTraceFileSourceModal::Construct(const FArguments& InArgs)
{
	SWindow::Construct(SWindow::FArguments()
	.Title(LOCTEXT("SChaosVDBrowseFileModal_Title", "Open CVD file"))
	.SupportsMinimize(false)
	.SupportsMaximize(false)
	.UserResizeBorder(0)
	.ClientSize(FVector2D(350, 140))
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(0.0f, 0.0f, 0.0f, 5.0f)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.Padding(5.0f, 10.0f, 5.0f, 5.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("OpenFileModalInnerTitle", "Select a folder and mode"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 12))
					]
				]
				+SVerticalBox::Slot()
				.Padding(5.0f, 0.0f, 5.0f, 5.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("OpenFromFileMode", "Loading Mode"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						GenerateConnectionModeWidget().ToSharedRef()
					]	
				]
				+SVerticalBox::Slot()
				.Padding(5.0f, 0.0f, 5.0f, 5.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock)
						.Text(LOCTEXT("OpenFromFileFolder", "Folder"))
						.Font(FCoreStyle::GetDefaultFontStyle("Regular", 10))
					]
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Right)
					.VAlign(VAlign_Center)
					[
						GenerateSourceFolderWidget().ToSharedRef()
					]
				]
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.HAlign(HAlign_Center)
		.Padding(5)
		[
			SNew(SButton)
			.HAlign(HAlign_Center)
			.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
			.Text(LOCTEXT("OpenSelectedLocation", "Open Folder"))
			.OnClicked(this, &SChaosBrowseTraceFileSourceModal::OnOpenButtonClick)
		]
	]);
}

TSharedPtr<SWidget> SChaosBrowseTraceFileSourceModal::GenerateConnectionModeWidget()
{
	TAttribute<int32> GetCurrentMode;
	GetCurrentMode.BindSPLambda(this, [this]()
	{
		return static_cast<int32>(LoadingMode);
	});

	SEnumComboBox::FOnEnumSelectionChanged ValueChangedDelegate;
	ValueChangedDelegate.BindSPLambda(this, [this](int32 NewValue, ESelectInfo::Type)
	{
		LoadingMode = static_cast<EChaosVDLoadRecordedDataMode>(NewValue);
	});

	return SNew(SEnumComboBox, StaticEnum<EChaosVDLoadRecordedDataMode>())
					.CurrentValue(GetCurrentMode)
					.OnEnumSelectionChanged(ValueChangedDelegate);
}

TSharedPtr<SWidget> SChaosBrowseTraceFileSourceModal::GenerateSourceFolderWidget()
{
	TAttribute<int32> GetCurrentMode;
	GetCurrentMode.BindSPLambda(this, [this]()
	{
		return static_cast<int32>(UserSelectedResponse);
	});

	SEnumComboBox::FOnEnumSelectionChanged ValueChangedDelegate;
	ValueChangedDelegate.BindSPLambda(this, [this](int32 NewValue, ESelectInfo::Type)
	{
		UserSelectedResponse = static_cast<EChaosVDBrowseFileModalResponse>(NewValue);
	});

	return SNew(SEnumComboBox, StaticEnum<EChaosVDBrowseFileModalResponse>())
					.CurrentValue(GetCurrentMode)
					.OnEnumSelectionChanged(ValueChangedDelegate);
}

EChaosVDBrowseFileModalResponse SChaosBrowseTraceFileSourceModal::ShowModal()
{
	GEditor->EditorAddModalWindow(SharedThis(this));
	
	return bUserClickedOpen ? UserSelectedResponse : EChaosVDBrowseFileModalResponse::Cancel;
}

FReply SChaosBrowseTraceFileSourceModal::OnOpenButtonClick()
{
	bUserClickedOpen = true;
	RequestDestroyWindow();
	return FReply::Handled();
}

FReply SChaosBrowseTraceFileSourceModal::OnCancelButtonClick()
{
	UserSelectedResponse = EChaosVDBrowseFileModalResponse::Cancel;
	RequestDestroyWindow();

	return FReply::Handled();
}

#undef LOCTEXT_NAMESPACE

