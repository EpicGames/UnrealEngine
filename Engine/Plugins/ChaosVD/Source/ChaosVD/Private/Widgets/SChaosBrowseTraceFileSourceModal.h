// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDEngine.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

#include "SChaosBrowseTraceFileSourceModal.generated.h"

class SChaosVDNameListPicker;
class FReply;

/**
 * Available default folder locations where CVD files might be located
 */
UENUM()
enum class EChaosVDBrowseFileModalResponse
{
	/** Opens the file picker at the last opened folder */
	LastOpened,
	/** Opens the file picker at the currently configure save folder. The documents folder is the default */
	CurrentSaveFolder,
	/** Opens the file picker at local trace store folder (This is where live recordings are located) */
	LocalTraceStore,
	Cancel UMETA(Hidden)
};

/**
 * Simple modal window that allows user pick a file to load and the mode to be used.
 */
class SChaosBrowseTraceFileSourceModal : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SChaosBrowseTraceFileSourceModal)
		{
		}

	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	EChaosVDBrowseFileModalResponse ShowModal();
	
	/** Returns the default connection mode to be used when loading the data for the selected session */
	EChaosVDLoadRecordedDataMode GetSelectedLoadingMode() const
	{
		return LoadingMode;
	}

protected:

	TSharedPtr<SWidget> GenerateConnectionModeWidget();
	TSharedPtr<SWidget> GenerateSourceFolderWidget();
	
	FReply OnOpenButtonClick();
	FReply OnCancelButtonClick();

	TSharedPtr<SChaosVDNameListPicker> NamePickerWidget;
	
	TMap<FName, EChaosVDBrowseFileModalResponse> LocationNameToResponseID;

	FName CurrentSelectedLocationName;

	EChaosVDBrowseFileModalResponse UserSelectedResponse = EChaosVDBrowseFileModalResponse::LastOpened;

	EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::SingleSource;

	bool bUserClickedOpen = false;
};
