// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDEngine.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SWindow.h"

class FName;
class FText;
class FLiveSessionTracker;
class SChaosVDNameListPicker;
class SEditableTextBox;

namespace EAppReturnType { enum Type : int; }

/** Structure containing info about a Trace Session */
struct FChaosVDTraceSessionInfo
{
	uint32 TraceID = 0;
	uint32 IPAddress = 0;
	uint32 ControlPort = 0;
	bool bIsValid = false;
};

/**
 * Modal window used to find, show and select active Trace Sessions to be used
 * with Chaos Visual Debugger
 */
class SChaosVDBrowseSessionsModal : public SWindow
{
public:
	SLATE_BEGIN_ARGS(SChaosVDBrowseSessionsModal)
		{
		}
	SLATE_END_ARGS()

	virtual ~SChaosVDBrowseSessionsModal() override;

	void Construct(const FArguments& InArgs);

	/** Displays the dialog in a blocking fashion */
	EAppReturnType::Type ShowModal();

	/** Response that triggered the closing of this modal */
	EAppReturnType::Type GetResponse() const
	{
		return UserResponse;
	}

	/** Returns information about the active trace session selected */
	FChaosVDTraceSessionInfo GetSelectedTraceInfo();

	/** Returns the address of the trace store selected while looking for active Trace Sessions */
	FString GetSelectedTraceStoreAddress() const
	{
		return CurrentTraceStoreAddress;
	}

	/** Returns the default connection mode to be used when loading the data for the selected session */
	EChaosVDLoadRecordedDataMode GetSelectedConnectionMode() const
	{
		return ConnectionMode;
	}

	void ModalTick(float InDeltaTime);

protected:

	TSharedPtr<SWidget> GenerateConnectionModeWidget();

	bool CanOpenSession() const;
	
	void UpdateCurrentSessionInfoMap();

	FReply OnButtonClick(EAppReturnType::Type ButtonID);
	
	void HandleSessionNameSelected(TSharedPtr<FName> SelectedName);
	
	void OnTraceStoreAddressUpdated(const FText& InText, ETextCommit::Type CommitType);

	TSharedPtr<SChaosVDNameListPicker> NamePickerWidget;
	
	TSharedPtr<SEditableTextBox> TraceStoreAddressWidget;
	
	TMap<FName, FChaosVDTraceSessionInfo> CurrentSessionInfosMap;

	EAppReturnType::Type UserResponse = EAppReturnType::Cancel;

	FString CurrentTraceStoreAddress;

	FName CurrentTraceSessionSelected;

	EChaosVDLoadRecordedDataMode ConnectionMode = EChaosVDLoadRecordedDataMode::SingleSource;

	FDelegateHandle ModalTickHandle;
	float AccumulatedTimeBetweenTicks = 0.0f;
};
