// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDCollisionDataInspector.h"


#include "ChaosVDScene.h"
#include "IStructureDetailsView.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Visualizers/ChaosVDSolverCollisionDataComponentVisualizer.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/SChaosVDNameListPicker.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosVDCollisionDataInspector::SetCollisionDataListToInspect(TConstArrayView<TSharedPtr<FChaosVDParticlePairMidPhase>> CollisionDataList)
{
	ClearInspector();

	if (CollisionDataList.IsEmpty())
	{
		return;
	}
	
	TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr.IsValid())
	{
		return;
	}
	
	TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin();

	if (!SolverDataSelectionObject)
	{
		return;
	}

	TArray<TSharedPtr<FName>> NewCollisionDataEntriesNameList;

	for (const TSharedPtr<FChaosVDParticlePairMidPhase>& CollisionData : CollisionDataList)
	{
		TSharedPtr<FChaosVDSolverDataSelectionHandle> SelectionHandle = SolverDataSelectionObject->MakeSelectionHandle(CollisionData);
		if (TSharedPtr<FName> CollisionDataEntryName = GenerateNameForCollisionDataItem(SelectionHandle))
		{
			CollisionDataByNameMap.Add(*CollisionDataEntryName, SelectionHandle);
			NewCollisionDataEntriesNameList.Add(CollisionDataEntryName);

			if (!CurrentDataSelectionHandle->IsValid())
			{
				CurrentSelectedName = CollisionDataEntryName;
			}
		}
	}

	CollisionDataAvailableList->UpdateNameList(MoveTemp(NewCollisionDataEntriesNameList));
	CollisionDataAvailableList->SelectName(CurrentSelectedName, ESelectInfo::OnMouseClick);
	bIsUpToDate = true;
}

void SChaosVDCollisionDataInspector::SetConstraintDataToInspect(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle)
{
	ClearInspector();

	if (!InDataSelectionHandle || !InDataSelectionHandle->IsA<FChaosVDParticlePairMidPhase>())
	{
		return;
	}
	
	CurrentDataSelectionHandle = InDataSelectionHandle.ToSharedRef();

	CurrentSelectedName = GenerateNameForCollisionDataItem(InDataSelectionHandle);

	CollisionDataByNameMap.Add(*CurrentSelectedName, InDataSelectionHandle);

	CollisionDataAvailableList->UpdateNameList({ CurrentSelectedName });
	CollisionDataAvailableList->SelectName(CurrentSelectedName, ESelectInfo::OnMouseClick);
}

void SChaosVDCollisionDataInspector::SetupWidgets()
{
	SChaosVDConstraintDataInspector::SetupWidgets();

	SecondaryCollisionDataDetailsPanel = CreateCollisionDataDetailsView();
}

void SChaosVDCollisionDataInspector::HandleSceneUpdated()
{
	if (const TSharedPtr<FChaosVDSolverDataSelectionHandle>& SelectionHandle = GetCurrentDataBeingInspected())
	{
		if (SelectionHandle->IsSelected() && SelectionHandle->GetData<FChaosVDParticlePairMidPhase>())
		{
			bIsUpToDate = false;
		}
		else
		{
			ClearInspector();
		}
	}
}

void SChaosVDCollisionDataInspector::ClearInspector()
{
	CollisionDataByNameMap.Reset();
	CollisionDataAvailableList->UpdateNameList({});

	CurrentSelectedName = nullptr;

	bIsUpToDate = true;

	if (SecondaryCollisionDataDetailsPanel->GetStructureProvider())
	{
		SecondaryCollisionDataDetailsPanel->SetStructureData(nullptr);
	}

	SChaosVDConstraintDataInspector::ClearInspector();
}

TSharedRef<SWidget> SChaosVDCollisionDataInspector::GenerateHeaderWidget(FMargin Margin)
{
	return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.Padding(Margin)
			.AutoHeight()
			[
				SAssignNew(CollisionDataAvailableList, SChaosVDNameListPicker)
				.OnNameSleceted_Raw(this, &SChaosVDCollisionDataInspector::HandleCollisionDataEntryNameSelected)
			];
}

TSharedRef<SWidget> SChaosVDCollisionDataInspector::GenerateDetailsViewWidget(FMargin Margin)
{
	return SNew(SScrollBox)
			+SScrollBox::Slot()
			.Padding(Margin)
			[
				MainDataDetailsView->GetWidget().ToSharedRef()
			]
			+SScrollBox::Slot()
			.Padding(Margin)
			[
				SecondaryCollisionDataDetailsPanel->GetWidget().ToSharedRef()
			];
}

FText SChaosVDCollisionDataInspector::GetParticleName(EChaosVDParticlePairIndex ParticleSlot, const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle) const
{
	int32 OutSolverID = INDEX_NONE;
	int32 OutParticleID = INDEX_NONE;
	GetParticleIDForSelectedData(InSelectionHandle, ParticleSlot, OutSolverID, OutParticleID);

	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		if (TSharedPtr<FChaosVDSceneParticle> ParticleActor = ScenePtr->GetParticleInstance(OutSolverID, OutParticleID))
		{
			if (TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = ParticleActor->GetParticleData())
			{
				return FText::AsCultureInvariant(ParticleData->GetDebugName());
			}
		}
	}

	return FText::GetEmpty();
}


void SChaosVDCollisionDataInspector::GetParticleIDForSelectedData(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle, EChaosVDParticlePairIndex ParticleSlot, int32& OutSolverID, int32& OutParticleID) const
{
	if (!InSelectionHandle)
	{
		return;
	}

	if (const FChaosVDParticlePairMidPhase* MidPhasePtr = InSelectionHandle->GetData<FChaosVDParticlePairMidPhase>())
	{
		bool bHasConstraintData = false;
		if (FChaosVDCollisionDataSelectionContext* SelectionContext = InSelectionHandle->GetContextData<FChaosVDCollisionDataSelectionContext>())
		{
			if (SelectionContext->ConstraintDataPtr)
			{
				bHasConstraintData = true;
				OutParticleID = ParticleSlot == EChaosVDParticlePairIndex::Index_0 ? SelectionContext->ConstraintDataPtr->Particle0Index : SelectionContext->ConstraintDataPtr->Particle1Index;
			}
		}
		
		if (!bHasConstraintData)
		{
			OutParticleID = ParticleSlot == EChaosVDParticlePairIndex::Index_0 ? MidPhasePtr->Particle0Idx : MidPhasePtr->Particle1Idx;
		}

		OutSolverID = MidPhasePtr->SolverID;
	}
}

void SChaosVDCollisionDataInspector::HandleCollisionDataEntryNameSelected(TSharedPtr<FName> SelectedName)
{
	CurrentSelectedName = SelectedName;

	if (!CurrentSelectedName.IsValid())
	{
		return;
	}

	CurrentDataSelectionHandle = GetSelectionNameForName(SelectedName).ToSharedRef();

	if (CurrentDataSelectionHandle->IsValid())
	{
		if (FChaosVDParticlePairMidPhase* MidPhasePtr = CurrentDataSelectionHandle->GetData<FChaosVDParticlePairMidPhase>())
		{
			bool bHasConstraintData = false;
			if (FChaosVDCollisionDataSelectionContext* SelectionContext = CurrentDataSelectionHandle->GetContextData<FChaosVDCollisionDataSelectionContext>())
			{
				if (SelectionContext->ConstraintDataPtr)
				{
					bHasConstraintData = true;

					TSharedPtr<FStructOnScope> ConstraintView = MakeShared<FStructOnScope>(FChaosVDConstraint::StaticStruct(), reinterpret_cast<uint8*>(SelectionContext->ConstraintDataPtr));
					SecondaryCollisionDataDetailsPanel->SetStructureData(ConstraintView); 

					if (SelectionContext->ConstraintDataPtr->ManifoldPoints.IsValidIndex(SelectionContext->ContactDataIndex))
					{
						const FChaosVDManifoldPoint* ContactPointData = &SelectionContext->ConstraintDataPtr->ManifoldPoints[SelectionContext->ContactDataIndex];
						TSharedPtr<FStructOnScope> ContactViewView = MakeShared<FStructOnScope>(FChaosVDManifoldPoint::StaticStruct(), reinterpret_cast<uint8*>(const_cast<FChaosVDManifoldPoint*>(ContactPointData)));
						MainDataDetailsView->SetStructureData(ContactViewView);
					}
					else
					{
						MainDataDetailsView->SetStructureData(nullptr); 
					}
				}
			}

			if (!bHasConstraintData)
			{
				// If we have a recorded midphase with not contact data, show that instead
				const TSharedPtr<FStructOnScope> MidPhaseView = MakeShared<FStructOnScope>(FChaosVDParticlePairMidPhase::StaticStruct(), reinterpret_cast<uint8*>(MidPhasePtr));
				MainDataDetailsView->SetStructureData(MidPhaseView); 
			}
		}
	}
}

TSharedRef<FName> SChaosVDCollisionDataInspector::GenerateNameForCollisionDataItem(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle) const
{
	static TSharedPtr<FName> InvalidName = MakeShared<FName>(NAME_None);
	if (!InDataSelectionHandle)
	{
		return InvalidName.ToSharedRef();
	}

	const FText GeneratedName = FText::Format(LOCTEXT("CollisionItemDataTitle", "Particle Pair | [ {0} ] <-> [ {1} ] "), GetParticleName(EChaosVDParticlePairIndex::Index_0, InDataSelectionHandle), GetParticleName(EChaosVDParticlePairIndex::Index_1, InDataSelectionHandle));
	return MakeShared<FName>(GeneratedName.ToString());
}

TSharedPtr<IStructureDetailsView> SChaosVDCollisionDataInspector::CreateCollisionDataDetailsView()
{
	TSharedPtr<SChaosVDMainTab> MainTabPtr = MainTabWeakPtr.Pin();
	if (!MainTabPtr)
	{
		return nullptr;
	}

	FStructureDetailsViewArgs StructDetailsViewArgs;
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowScrollBar = false;

	return MainTabPtr->CreateStructureDetailsView(DetailsViewArgs,StructDetailsViewArgs, nullptr);
}

EVisibility SChaosVDCollisionDataInspector::GetDetailsSectionVisibility() const
{
	return CurrentSelectedName ? EVisibility::Visible : EVisibility::Collapsed;
}

FReply SChaosVDCollisionDataInspector::SelectParticleForCurrentSelectedData(EChaosVDParticlePairIndex ParticleSlot)
{
	int32 OutParticleID = INDEX_NONE;
	int32 OutSolverID = INDEX_NONE;

	GetParticleIDForSelectedData(GetCurrentDataBeingInspected(), ParticleSlot,OutSolverID,OutParticleID);
	
	SelectParticle(OutSolverID, OutParticleID);

	return FReply::Handled();
}


TSharedPtr<FChaosVDSolverDataSelectionHandle> SChaosVDCollisionDataInspector::GetSelectionNameForName(const TSharedPtr<FName>& InName)
{
	if (!InName.IsValid())
	{
		return MakeShared<FChaosVDSolverDataSelectionHandle>();
	}

	if (const TSharedPtr<FChaosVDSolverDataSelectionHandle>* SelectionHandlePtrPtr = CollisionDataByNameMap.Find(*InName))
	{
		if (const TSharedPtr<FChaosVDSolverDataSelectionHandle>& SelectionHandlePtr = *SelectionHandlePtrPtr)
		{
			return SelectionHandlePtr;
		}
	}

	return MakeShared<FChaosVDSolverDataSelectionHandle>();
}


const TSharedRef<FChaosVDSolverDataSelectionHandle>& SChaosVDCollisionDataInspector::GetCurrentDataBeingInspected() const
{
	return CurrentDataSelectionHandle;
}

#undef LOCTEXT_NAMESPACE
