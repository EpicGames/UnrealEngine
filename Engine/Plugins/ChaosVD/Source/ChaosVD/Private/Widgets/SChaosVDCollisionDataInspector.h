// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SChaosVDConstraintDataInspector.h"
#include "Widgets/SCompoundWidget.h"

struct FChaosVDSolverDataSelectionHandle;
class FChaosVDScene;
class IStructureDetailsView;
class IChaosVDCollisionDataProviderInterface;
class SChaosVDNameListPicker;

struct FChaosVDCollisionDataFinder;
struct FChaosVDParticlePairMidPhase;

class SChaosVDCollisionDataInspector : public SChaosVDConstraintDataInspector
{
public:

	void SetCollisionDataListToInspect(TConstArrayView<TSharedPtr<FChaosVDParticlePairMidPhase>> CollisionDataList);

	virtual void SetConstraintDataToInspect(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle) override;

protected:
	virtual void SetupWidgets() override;
	virtual void HandleSceneUpdated() override;

	virtual void ClearInspector() override;

	virtual TSharedRef<SWidget> GenerateHeaderWidget(FMargin Margin) override;
	virtual TSharedRef<SWidget> GenerateDetailsViewWidget(FMargin Margin) override;

	virtual FText GetParticleName(EChaosVDParticlePairIndex ParticleSlot, const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle) const override;

	void GetParticleIDForSelectedData(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle, EChaosVDParticlePairIndex ParticleSlot, int32& OutSolverID, int32& OutParticleID) const;

	virtual const TSharedRef<FChaosVDSolverDataSelectionHandle>& GetCurrentDataBeingInspected() const override;

	void HandleCollisionDataEntryNameSelected(TSharedPtr<FName> SelectedName);

	TSharedRef<FName> GenerateNameForCollisionDataItem(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle) const;

	TSharedPtr<IStructureDetailsView> CreateCollisionDataDetailsView();

	virtual EVisibility GetDetailsSectionVisibility() const override;

	virtual FReply SelectParticleForCurrentSelectedData(EChaosVDParticlePairIndex ParticleSlot) override;

	TSharedPtr<FChaosVDSolverDataSelectionHandle> GetSelectionNameForName(const TSharedPtr<FName>& InName);

	TSharedPtr<SChaosVDNameListPicker> CollisionDataAvailableList;
	
	TMap<FName, TSharedPtr<FChaosVDSolverDataSelectionHandle>> CollisionDataByNameMap;

	TSharedPtr<FName> CurrentSelectedName;

	TSharedPtr<IStructureDetailsView> SecondaryCollisionDataDetailsPanel;
	
};
