// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDConstraintDataInspector.h"

#include "ChaosVDScene.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Widgets/SChaosVDWarningMessageBox.h"
#include "Modules/ModuleManager.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FReadOnlyCopyStructOnScope::FReadOnlyCopyStructOnScope(const FStructOnScope& StructToCopy)
{
	if (const UStruct* ScriptStructPtr = StructToCopy.GetStruct())
	{
		ScriptStruct = ScriptStructPtr;
		SampleStructMemory = (uint8*)FMemory::Malloc(ScriptStructPtr->GetStructureSize() ? ScriptStructPtr->GetStructureSize() : 1, ScriptStructPtr->GetMinAlignment());
		ScriptStructPtr->InitializeStruct(SampleStructMemory);
		OwnsMemory = true;

		if (ensure(SampleStructMemory))
		{
			UScriptStruct* AsScript = Cast<UScriptStruct>(const_cast<UStruct*>(ScriptStructPtr));
			if (ensure(AsScript))
			{
				AsScript->CopyScriptStruct(SampleStructMemory,StructToCopy.GetStructMemory());
			}
		}
	}
}

void FReadOnlyCopyStructOnScope::UpdateFromOther(const FStructOnScope& StructToCopy)
{
	const UStruct* StructPtr = StructToCopy.GetStruct();
	if (StructPtr && ensure(ScriptStruct == StructToCopy.GetStruct()))
	{
		if (ensure(SampleStructMemory))
		{
			UScriptStruct* AsScript = Cast<UScriptStruct>(const_cast<UStruct*>(StructToCopy.GetStruct()));
			if (ensure(AsScript))
			{
				AsScript->CopyScriptStruct(SampleStructMemory,StructToCopy.GetStructMemory());
			}
		}
	}
}

SChaosVDConstraintDataInspector::SChaosVDConstraintDataInspector() : CurrentDataSelectionHandle(MakeShared<FChaosVDSolverDataSelectionHandle>())
{ 
}

SChaosVDConstraintDataInspector::~SChaosVDConstraintDataInspector()
{
	UnregisterSceneEvents();
}

void SChaosVDConstraintDataInspector::Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene>& InScenePtr, const TSharedRef<SChaosVDMainTab>& InMainTab)
{
	SceneWeakPtr = InScenePtr;
	MainTabWeakPtr = InMainTab;

	RegisterSceneEvents();

	SetupWidgets();

	constexpr float NoPadding = 0.0f;
	constexpr float OuterBoxPadding = 2.0f;
	constexpr float OuterInnerPadding = 5.0f;
	constexpr float TagTitleBoxHorizontalPadding = 10.0f;
	constexpr float TagTitleBoxVerticalPadding = 5.0f;
	constexpr float InnerDetailsPanelsHorizontalPadding = 15.0f;
	constexpr float InnerDetailsPanelsVerticalPadding = 15.0f;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(OuterInnerPadding)
		[
			GenerateHeaderWidget({})
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(OuterInnerPadding)
		[
			SNew(SBox)
			.Visibility_Raw(this, &SChaosVDConstraintDataInspector::GetOutOfDateWarningVisibility)
			.Padding(OuterBoxPadding, OuterBoxPadding,OuterBoxPadding,NoPadding)
			[
				SNew(SChaosVDWarningMessageBox)
				.WarningText(LOCTEXT("ConstraintDataOutOfDate", "Scene change detected!. Selected constraint data is out of date..."))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(TagTitleBoxHorizontalPadding, TagTitleBoxVerticalPadding, TagTitleBoxHorizontalPadding, NoPadding)
		[
			GenerateParticleSelectorButtons()
		]
		+SVerticalBox::Slot()
		.Padding(TagTitleBoxHorizontalPadding, TagTitleBoxVerticalPadding, TagTitleBoxHorizontalPadding, NoPadding)
		.AutoHeight()
		[
			SNew(STextBlock)
			.Visibility_Raw(this, &SChaosVDConstraintDataInspector::GetNothingSelectedMessageVisibility)
			.Justification(ETextJustify::Center)
			.TextStyle(FAppStyle::Get(), "DetailsView.BPMessageTextStyle")
			.Text(LOCTEXT("ConstraintDataNoSelectedMessage", "Select a Constraint in the viewport to see its details..."))
			.AutoWrapText(true)
		]
		+SVerticalBox::Slot()
		.Padding(OuterInnerPadding)
		[
			GenerateDetailsViewWidget({InnerDetailsPanelsHorizontalPadding,NoPadding,InnerDetailsPanelsHorizontalPadding,InnerDetailsPanelsVerticalPadding})
		]
	];
}

void SChaosVDConstraintDataInspector::RegisterSceneEvents()
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->OnSceneUpdated().AddRaw(this, &SChaosVDConstraintDataInspector::HandleSceneUpdated);
		if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			SelectionObject->GetDataSelectionChangedDelegate().AddRaw(this, &SChaosVDConstraintDataInspector::SetConstraintDataToInspect);
		}
	}
}

void SChaosVDConstraintDataInspector::UnregisterSceneEvents() const
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->OnSceneUpdated().RemoveAll(this);

		if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			SelectionObject->GetDataSelectionChangedDelegate().RemoveAll(this);
		}
	}
}

void SChaosVDConstraintDataInspector::SetConstraintDataToInspect(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle)
{
	if (InDataSelectionHandle && InDataSelectionHandle->IsA<FChaosVDConstraintDataWrapperBase>())
	{
		CurrentDataSelectionHandle = InDataSelectionHandle.ToSharedRef();

		if (HasCompatibleStructScopeView(InDataSelectionHandle.ToSharedRef()))
		{
			DataBeingInspectedCopy->UpdateFromOther(*InDataSelectionHandle->GetDataAsStructScope());
		}
		else
		{
			DataBeingInspectedCopy = MakeShared<FReadOnlyCopyStructOnScope>(*InDataSelectionHandle->GetDataAsStructScope());
			MainDataDetailsView->SetStructureData(DataBeingInspectedCopy);
		}
	}
	else
	{
		ClearInspector();
	}

	bIsUpToDate = true;
}

void SChaosVDConstraintDataInspector::SetupWidgets()
{
	MainDataDetailsView = CreateDataDetailsView();
}

TSharedRef<SWidget> SChaosVDConstraintDataInspector::GenerateDetailsViewWidget(FMargin Margin)
{
	return  SNew(SScrollBox)
	.Visibility_Raw(this, &SChaosVDConstraintDataInspector::GetDetailsSectionVisibility)
	+SScrollBox::Slot()
	.Padding(Margin.Left, Margin.Top, Margin.Right,Margin.Bottom)
	[
		MainDataDetailsView->GetWidget().ToSharedRef()
	];
}

TSharedRef<SWidget> SChaosVDConstraintDataInspector::GenerateHeaderWidget(FMargin Margin)
{
	return SNew(SBox)
			.Visibility(EVisibility::Collapsed);
}

FText SChaosVDConstraintDataInspector::GetParticleName(EChaosVDParticlePairIndex ParticleSlot) const
{
	return GetParticleName(ParticleSlot, GetCurrentDataBeingInspected());
}

FText SChaosVDConstraintDataInspector::GetParticleName(const EChaosVDParticlePairIndex ParticleSlot, const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle) const
{
	bool bHasValidSelection = InSelectionHandle && InSelectionHandle->IsValid();
	if (!bHasValidSelection)
	{
		return FText::GetEmpty();
	}

	if (FChaosVDConstraintDataWrapperBase* ConstraintData = GetCurrentDataBeingInspected()->GetData<FChaosVDConstraintDataWrapperBase>())
	{
		return GetParticleName_Internal(ConstraintData->GetSolverID(), ConstraintData->GetParticleIDAtSlot(ParticleSlot));	
	}

	return FText::GetEmpty();
}

TSharedRef<SWidget> SChaosVDConstraintDataInspector::GenerateParticleSelectorButtons()
{
	return SNew(SUniformGridPanel)
			.Visibility_Raw(this, &SChaosVDConstraintDataInspector::GetDetailsSectionVisibility)
			.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
			.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
			.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
			+SUniformGridPanel::Slot(0, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("SelectParticle0", "Select Particle 0"))
				.ToolTipText_Raw(this, &SChaosVDConstraintDataInspector::GetParticleName, EChaosVDParticlePairIndex::Index_0)
				.OnClicked(this, &SChaosVDConstraintDataInspector::SelectParticleForCurrentSelectedData, EChaosVDParticlePairIndex::Index_0)
			]
			+SUniformGridPanel::Slot(1, 0)
			[
				SNew(SButton)
				.HAlign(HAlign_Center)
				.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				.Text(LOCTEXT("SelectParticle1", "Select Particle 1"))
				.ToolTipText_Raw(this, &SChaosVDConstraintDataInspector::GetParticleName, EChaosVDParticlePairIndex::Index_1)
				.OnClicked(this, &SChaosVDConstraintDataInspector::SelectParticleForCurrentSelectedData, EChaosVDParticlePairIndex::Index_1)
			];
}

FReply SChaosVDConstraintDataInspector::SelectParticleForCurrentSelectedData(EChaosVDParticlePairIndex ParticleSlot)
{
	if (FChaosVDConstraintDataWrapperBase* ConstraintData = GetConstraintDataFromSelectionHandle())
	{
		SelectParticle(ConstraintData->GetSolverID(), ConstraintData->GetParticleIDAtSlot(ParticleSlot));
	}
	
	return FReply::Handled();
}

bool SChaosVDConstraintDataInspector::HasCompatibleStructScopeView(const TSharedRef<FChaosVDSolverDataSelectionHandle>& InSelectionHandle) const
{
	return DataBeingInspectedCopy && InSelectionHandle->GetDataAsStructScope() && DataBeingInspectedCopy->GetStruct() == InSelectionHandle->GetDataAsStructScope()->GetStruct();
}

EVisibility SChaosVDConstraintDataInspector::GetOutOfDateWarningVisibility() const
{
	return !bIsUpToDate && GetCurrentDataBeingInspected()->IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SChaosVDConstraintDataInspector::GetDetailsSectionVisibility() const
{
	return GetCurrentDataBeingInspected()->IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SChaosVDConstraintDataInspector::GetNothingSelectedMessageVisibility() const
{
	return GetCurrentDataBeingInspected()->IsValid() ? EVisibility::Collapsed : EVisibility::Visible;
}

void SChaosVDConstraintDataInspector::HandleSceneUpdated()
{
	// TODO: Disabling the "out of date" message system, it is a little bit confusing with joint constraints as the debug draw position will be updated between frames (because the constraint only has particles IDs)
	// but the data itself is not updated.
	// We are clearing out the selection altogether instead for now
	bIsUpToDate = false;

	//ClearInspector();

	// TODO: To Keep a selection up to date we need a persistent ID for the constraint
	// We could hash the pointer for that or add an ID to the Constraint Handle only compiled in when CVD is enabled
}

void SChaosVDConstraintDataInspector::ClearInspector()
{
	DataBeingInspectedCopy = nullptr;

	if (MainDataDetailsView->GetStructureProvider())
	{
		MainDataDetailsView->SetStructureData(nullptr);
	}
	
	CurrentDataSelectionHandle = MakeShared<FChaosVDSolverDataSelectionHandle>();
}

FText SChaosVDConstraintDataInspector::GetParticleName_Internal(int32 SolverID, int32 ParticleID) const
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		if (TSharedPtr<FChaosVDSceneParticle> ParticleActor = ScenePtr->GetParticleInstance(SolverID, ParticleID))
		{
			if (TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = ParticleActor->GetParticleData())
			{
				return FText::AsCultureInvariant(ParticleData->GetDebugName());
			}
		}
	}

	return FText::GetEmpty();
}

void SChaosVDConstraintDataInspector::SelectParticle(int32 SolverID, int32 ParticleID) const
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		if (TSharedPtr<FChaosVDSceneParticle> ParticleActor = ScenePtr->GetParticleInstance(SolverID, ParticleID))
		{
			using namespace Chaos::VD::TypedElementDataUtil;
			ScenePtr->SetSelected(AcquireTypedElementHandleForStruct(ParticleActor.Get(), true));
		}
	}
}

FChaosVDConstraintDataWrapperBase* SChaosVDConstraintDataInspector::GetConstraintDataFromSelectionHandle() const
{
	bool bHasValidSelection = GetCurrentDataBeingInspected()->IsValid();
	if (!bHasValidSelection)
	{
		return nullptr;
	}

	return GetCurrentDataBeingInspected()->GetData<FChaosVDConstraintDataWrapperBase>();
}

const TSharedRef<FChaosVDSolverDataSelectionHandle>& SChaosVDConstraintDataInspector::GetCurrentDataBeingInspected() const
{
	return CurrentDataSelectionHandle;
}

TSharedPtr<IStructureDetailsView> SChaosVDConstraintDataInspector::CreateDataDetailsView() const
{
	TSharedPtr<SChaosVDMainTab> MainTabPtr = MainTabWeakPtr.Pin();
	if (!MainTabPtr)
	{
		return nullptr;
	}

	const FStructureDetailsViewArgs StructDetailsViewArgs;
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bAllowSearch = true;
	DetailsViewArgs.bShowScrollBar = false;

	return MainTabPtr->CreateStructureDetailsView(DetailsViewArgs, StructDetailsViewArgs, nullptr);
}

#undef LOCTEXT_NAMESPACE
