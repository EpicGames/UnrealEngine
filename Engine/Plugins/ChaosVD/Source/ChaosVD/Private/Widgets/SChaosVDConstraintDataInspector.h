// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Components/ChaosVDSolverJointConstraintDataComponent.h"
#include "Widgets/SCompoundWidget.h"

class SChaosVDMainTab;
struct FChaosVDSolverDataSelectionHandle;
struct FChaosVDJointConstraintSelectionHandle;
class FChaosVDScene;
class IStructureDetailsView;
class SChaosVDNameListPicker;

struct FChaosVDConstraintDataWrapperBase;

/** Version of FStructOnScope that will take another FStructOnScope, and copy its data over.
 * This allows us update a details panel without making a full rebuild when we want to inspect another struct that is of the same type
 * As long we don't mind the copy and not being able to edit the source struct (which is 99% of the use cases in CVD)
 */
class FReadOnlyCopyStructOnScope : public FStructOnScope
{
public:
	explicit FReadOnlyCopyStructOnScope(const FStructOnScope& StructToCopy);

	void UpdateFromOther(const FStructOnScope& StructToCopy);
};

class SChaosVDConstraintDataInspector : public SCompoundWidget
{
public:
	SChaosVDConstraintDataInspector();

	SLATE_BEGIN_ARGS(SChaosVDConstraintDataInspector)
		{
	}

	SLATE_END_ARGS()

	virtual ~SChaosVDConstraintDataInspector() override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene>& InScenePtr, const TSharedRef<SChaosVDMainTab>& InMainTab);

	/** Sets a new query data to be inspected */
	virtual void SetConstraintDataToInspect(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle);

protected:

	virtual void SetupWidgets();

	virtual TSharedRef<SWidget> GenerateDetailsViewWidget(FMargin Margin);
	virtual TSharedRef<SWidget> GenerateHeaderWidget(FMargin Margin);

	virtual FText GetParticleName(EChaosVDParticlePairIndex ParticleSlot) const;
	virtual FText GetParticleName(const EChaosVDParticlePairIndex ParticleSlot, const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle) const;

	TSharedPtr<IStructureDetailsView> CreateDataDetailsView() const;

	TSharedRef<SWidget> GenerateParticleSelectorButtons();
	
	virtual FReply SelectParticleForCurrentSelectedData(EChaosVDParticlePairIndex ParticleSlot);

	bool HasCompatibleStructScopeView(const TSharedRef<FChaosVDSolverDataSelectionHandle>& InSelectionHandle) const;

	EVisibility GetOutOfDateWarningVisibility() const;
	virtual EVisibility GetDetailsSectionVisibility() const;
	EVisibility GetNothingSelectedMessageVisibility() const;

	void RegisterSceneEvents();
	void UnregisterSceneEvents() const;

	virtual void HandleSceneUpdated();

	virtual void ClearInspector();

	FText GetParticleName_Internal(int32 SolverID, int32 ParticleID) const;

	void SelectParticle(int32 SolverID, int32 ParticleID) const;

	FChaosVDConstraintDataWrapperBase* GetConstraintDataFromSelectionHandle() const;
	
	virtual const TSharedRef<FChaosVDSolverDataSelectionHandle>& GetCurrentDataBeingInspected() const;

	TSharedPtr<IStructureDetailsView> MainDataDetailsView;
	TSharedPtr<IStructureDetailsView> ConstraintSecondaryDataDetailsView;
	
	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	mutable TSharedRef<FChaosVDSolverDataSelectionHandle> CurrentDataSelectionHandle;

	TSharedPtr<FReadOnlyCopyStructOnScope> DataBeingInspectedCopy;

	bool bIsUpToDate = true;

	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr;
};
