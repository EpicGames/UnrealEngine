// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDDetailsView.h"

#include "PropertyEditorModule.h"
#include "Widgets/SChaosVDMainTab.h"

void SChaosVDDetailsView::Construct(const FArguments& InArgs, const TSharedRef<SChaosVDMainTab>& InMainTab)
{
	MainTabWeakPtr = InMainTab;
	DetailsView = CreateObjectDetailsView();
	StructDetailsView = CreateStructureDataDetailsView();

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(SVerticalBox)
			.Visibility_Raw(this, &SChaosVDDetailsView::GetObjectDetailsVisibility)
			+SVerticalBox::Slot()
			.AutoHeight()
			+SVerticalBox::Slot()
			[
				DetailsView.ToSharedRef()
			]
		]
		+SVerticalBox::Slot()
	    [
			SNew(SVerticalBox)
			.Visibility_Raw(this, &SChaosVDDetailsView::GetStructDetailsVisibility)
			+SVerticalBox::Slot()
			[
				StructDetailsView->GetWidget().ToSharedRef()
			]
	    ]
	];
}

void SChaosVDDetailsView::SetSelectedStruct(const TSharedPtr<FStructOnScope>& NewStruct)
{
	// Clear the object selection view, as now we will have a struct view active
	SetSelectedObject(nullptr);
	
	CurrentStructInView = NewStruct;
	StructDetailsView->SetStructureData(NewStruct);
}

TSharedPtr<IDetailsView> SChaosVDDetailsView::CreateObjectDetailsView()
{
	TSharedPtr<SChaosVDMainTab> MainTabPtr = MainTabWeakPtr.Pin();
	if (!MainTabPtr)
	{
		return nullptr;
	}

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = false;
	DetailsViewArgs.bLockable = true;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::HideNameArea;
	DetailsViewArgs.bCustomFilterAreaLocation = false;
	DetailsViewArgs.bShowSectionSelector = false;
	DetailsViewArgs.bShowScrollBar = true;

	return MainTabPtr->CreateDetailsView(DetailsViewArgs);
}

TSharedPtr<IStructureDetailsView> SChaosVDDetailsView::CreateStructureDataDetailsView() const
{
	TSharedPtr<SChaosVDMainTab> MainTabPtr = MainTabWeakPtr.Pin();
	if (!MainTabPtr)
	{
		return nullptr;
	}

	const FStructureDetailsViewArgs StructDetailsViewArgs;
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowScrollBar = true;

	return MainTabPtr->CreateStructureDetailsView(DetailsViewArgs,StructDetailsViewArgs, nullptr);
}

EVisibility SChaosVDDetailsView::GetStructDetailsVisibility() const
{
	return CurrentStructInView.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SChaosVDDetailsView::GetObjectDetailsVisibility() const
{
	return CurrentObjectInView.IsValid() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SChaosVDDetailsView::SetSelectedObject(UObject* NewObject)
{
	// Even if the object is not valid, clear any active structure view
	StructDetailsView->SetStructureData(nullptr);
	CurrentStructInView = nullptr;

	if (DetailsView->IsLocked())
	{
		return;
	}

	CurrentObjectInView = NewObject;

	DetailsView->SetObject(NewObject, true);
}
