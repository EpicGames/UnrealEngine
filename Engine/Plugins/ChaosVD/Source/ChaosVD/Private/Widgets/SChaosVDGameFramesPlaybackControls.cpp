// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDGameFramesPlaybackControls.h"

void SChaosVDGameFramesPlaybackControls::Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDPlaybackController>& InPlaybackController)
{
	this->ChildSlot
	[
		SAssignNew(FramesTimelineWidget, SChaosVDTimelineWidget)
			.IsEnabled_Raw(this, &SChaosVDGameFramesPlaybackControls::CanPlayback)
			.IsPlaying_Raw(this, &SChaosVDGameFramesPlaybackControls::IsPlaying)
			.ButtonVisibilityFlags(EChaosVDTimelineElementIDFlags::AllPlayback)
			.OnFrameChanged_Raw(this, &SChaosVDGameFramesPlaybackControls::OnFrameSelectionUpdated)
			.OnButtonClicked(this, &SChaosVDGameFramesPlaybackControls::HandleFramePlaybackButtonClicked)
			.MinFrames_Raw(this, &SChaosVDGameFramesPlaybackControls::GetMinFrames)
			.MaxFrames_Raw(this, &SChaosVDGameFramesPlaybackControls::GetMaxFrames)
			.CurrentFrame_Raw(this, &SChaosVDGameFramesPlaybackControls::GetCurrentFrame)
			.ButtonEnabledFlags_Raw(this, &SChaosVDGameFramesPlaybackControls::GetElementEnabledFlags)
			.OnTimelineScrubStart(this, &SChaosVDGameFramesPlaybackControls::HandleTimelineScrubStart)
			.OnTimelineScrubEnd(this, &SChaosVDGameFramesPlaybackControls::HandleTimelineScrubEnd)
	];

	RegisterNewController(InPlaybackController);
}

void SChaosVDGameFramesPlaybackControls::HandleTimelineScrubStart()
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->SetScrubbingTimeline(true);
	}
}

void SChaosVDGameFramesPlaybackControls::HandleTimelineScrubEnd()
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->SetScrubbingTimeline(false);
	}
}

void SChaosVDGameFramesPlaybackControls::OnFrameSelectionUpdated(int32 NewFrameIndex)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		constexpr int32 SolverStage = 0;
		PlaybackControllerPtr->TrySetActiveTrack(GameTrackInfoRef);
		PlaybackControllerPtr->GoToTrackFrameAndSync(GetInstigatorID(), EChaosVDTrackType::Game, FChaosVDPlaybackController::GameTrackID, NewFrameIndex, SolverStage);
	}
}

void SChaosVDGameFramesPlaybackControls::RegisterNewController(TWeakPtr<FChaosVDPlaybackController> NewController)
{
	FChaosVDPlaybackControllerObserver::RegisterNewController(NewController);

	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		if (TSharedPtr<FChaosVDTrackInfo> GameTrackInfoPtr = PlaybackControllerPtr->GetMutableTrackInfo(EChaosVDTrackType::Game, FChaosVDPlaybackController::GameTrackID))
		{
			GameTrackInfoRef = GameTrackInfoPtr.ToSharedRef();
		}
	}
}

void SChaosVDGameFramesPlaybackControls::HandleFramePlaybackButtonClicked(EChaosVDPlaybackButtonsID ButtonID)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->HandleFramePlaybackControlInput(ButtonID, GameTrackInfoRef, GetInstigatorID());
	}
}

void SChaosVDGameFramesPlaybackControls::HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InController)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		if (TSharedPtr<FChaosVDTrackInfo> GameTrackInfoPtr = PlaybackControllerPtr->GetMutableTrackInfo(EChaosVDTrackType::Game, FChaosVDPlaybackController::GameTrackID))
		{
			GameTrackInfoRef = GameTrackInfoPtr.ToSharedRef();
		}
	}
}

bool SChaosVDGameFramesPlaybackControls::CanPlayback() const
{
	bool bCanControlPlayback = false;
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		if (PlaybackControllerPtr->IsRecordingLoaded() && GameTrackInfoRef->TrackType != EChaosVDTrackType::Invalid)
		{
			TSharedRef<FChaosVDTrackInfo> ActivePlaybackTrack = PlaybackControllerPtr->GetActiveTrackInfo();
			bool bActiveTrackIsThisTrack = FChaosVDTrackInfo::AreSameTrack(ActivePlaybackTrack, GameTrackInfoRef);
			bCanControlPlayback = bActiveTrackIsThisTrack || (!bActiveTrackIsThisTrack && !ActivePlaybackTrack->bIsPlaying);
		}
	}

	return bCanControlPlayback;
}

bool SChaosVDGameFramesPlaybackControls::IsPlaying() const
{
	return GameTrackInfoRef->bIsPlaying;
}

int32 SChaosVDGameFramesPlaybackControls::GetCurrentFrame() const
{
	return GameTrackInfoRef->CurrentFrame >= 0 ? GameTrackInfoRef->CurrentFrame : 0 ;
}

int32 SChaosVDGameFramesPlaybackControls::GetMinFrames() const
{
	return 0;
}

int32 SChaosVDGameFramesPlaybackControls::GetMaxFrames() const
{
	return GameTrackInfoRef->MaxFrames > 0 ? GameTrackInfoRef->MaxFrames - 1 : 0;
}

EChaosVDTimelineElementIDFlags SChaosVDGameFramesPlaybackControls::GetElementEnabledFlags() const
{
	EChaosVDTimelineElementIDFlags EnabledButtonFlags = EChaosVDTimelineElementIDFlags::All;
	if (const TSharedPtr<FChaosVDPlaybackController> ControllerSharedPtr = PlaybackController.Pin())
	{
		if (ControllerSharedPtr->IsPlayingLiveSession())
		{
			constexpr EChaosVDTimelineElementIDFlags PlaybackElementDisabledDuringLiveSession = EChaosVDTimelineElementIDFlags::Stop | EChaosVDTimelineElementIDFlags::Next | EChaosVDTimelineElementIDFlags::Prev;
			constexpr EChaosVDTimelineElementIDFlags PlaybackElementDisabledDuringPausedLiveSession = EChaosVDTimelineElementIDFlags::Stop;

			EnumRemoveFlags(EnabledButtonFlags, IsPlaying() ? PlaybackElementDisabledDuringLiveSession : PlaybackElementDisabledDuringPausedLiveSession);
			return EnabledButtonFlags;
		}
		else
		{
			return EnabledButtonFlags;
		}
	}

	return EnabledButtonFlags;
}
