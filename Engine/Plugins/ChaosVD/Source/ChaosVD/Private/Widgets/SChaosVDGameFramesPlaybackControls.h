// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDPlaybackController.h"
#include "ChaosVDPlaybackControllerInstigator.h"
#include "ChaosVDPlaybackControllerObserver.h"
#include "SChaosVDTimelineWidget.h"
#include "Styling/ISlateStyle.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

enum class EChaosVDPlaybackButtonsID : uint8;
struct FChaosVDTrackInfo;
class FChaosVDPlaybackController;
class SChaosVDTimelineWidget;

class SChaosVDGameFramesPlaybackControls : public SCompoundWidget , public FChaosVDPlaybackControllerObserver, public IChaosVDPlaybackControllerInstigator
{
public:
	
	explicit SChaosVDGameFramesPlaybackControls()
	: GameTrackInfoRef(MakeShared<FChaosVDTrackInfo>())
	{
	}
	
	SLATE_BEGIN_ARGS(SChaosVDGameFramesPlaybackControls){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDPlaybackController>& InPlaybackController);

protected:
	void OnFrameSelectionUpdated(int32 NewFrameIndex);

	virtual void RegisterNewController(TWeakPtr<FChaosVDPlaybackController> NewController) override;

	void HandleFramePlaybackButtonClicked(EChaosVDPlaybackButtonsID ButtonID);

	virtual void HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InController) override;

	void HandleTimelineScrubStart();
	void HandleTimelineScrubEnd();

	bool CanPlayback() const;

	bool IsPlaying() const;

	int32 GetCurrentFrame() const;
	int32 GetMinFrames() const;
	int32 GetMaxFrames() const;
	EChaosVDTimelineElementIDFlags GetElementEnabledFlags() const;

	TSharedRef<FChaosVDTrackInfo> GameTrackInfoRef;
	TSharedPtr<SChaosVDTimelineWidget> FramesTimelineWidget;
};
