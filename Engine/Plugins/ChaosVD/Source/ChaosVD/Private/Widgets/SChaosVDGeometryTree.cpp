// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDGeometryTree.h"

#include "SChaosVDGeometryTreeRow.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Actors/ChaosVDSolverInfoActor.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

SChaosVDGeometryTree::FColumnNames SChaosVDGeometryTree::ColumnNames = SChaosVDGeometryTree::FColumnNames();

void SChaosVDGeometryTree::SetDataToInspect(const FChaosVDSceneParticle* Particle)
{
	bool bChangedParticle = SelectedParticle != Particle || !Particle;
	SelectedParticle = Particle;
	RefreshTreeData(bChangedParticle);
}

void SChaosVDGeometryTree::RefreshTreeData(bool bRegenTree)
{
	if (bRegenTree)
	{
		InternalGeometryTreeData.Reset();

		if (SelectedParticle)
		{
			if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
			{
				TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView> Root = SelectedParticle->GetRootImplicitView().Pin();

				TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = SelectedParticle->GetParticleData();
				Chaos::FConstImplicitObjectPtr RootGeometry = ScenePtr->GetUpdatedGeometry(ParticleData->GeometryHash);

				TSharedPtr<FChaosVDGeometryTreeItem> NewTreeItem = MakeShared<FChaosVDGeometryTreeItem>();
				NewTreeItem->Name = FText::FromString(SelectedParticle->GetDisplayName());
				NewTreeItem->Type = Chaos::ImplicitObjectType::Unknown;

				//Implicit Views will not be generated if geometry is not currently streamed in. Traverse ImplicitObjects from scratch as a backup.
				if (Root) {
					GenerateTree(Root.Get(), NewTreeItem->Children);
				}
				else if (RootGeometry) {
					GenerateTree(RootGeometry, NewTreeItem->Children);
				}

				InternalGeometryTreeData.Add(NewTreeItem);
			}
		}

		GeometryTreeWidget->SetTreeItemsSource(&InternalGeometryTreeData);
		ExpandAll(InternalGeometryTreeData);

		if (SelectedParticle && !SelectedParticle->GetSelectedMeshInstance().IsValid() && InternalGeometryTreeData.Num() > 0)
		{
			GeometryTreeWidget->SetSelection(InternalGeometryTreeData[0]);
		}
	}

	GeometryTreeWidget->RebuildList();
	GeometryTreeWidget->RequestTreeRefresh();
}

void SChaosVDGeometryTree::Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene>& InScenePtr)
{
	SceneWeakPtr = InScenePtr;
	GeometryItemSelectedDelegate = InArgs._OnItemSelected;
	GeometryItemFocusedDelegate = InArgs._OnItemFocused;

	constexpr float BottomPadding = 2.0f;
	constexpr float NoPadding = 0.0f;
	constexpr float VerticalPadding = 0.0f;

	static FMargin ColumnHeaderTextMargin(NoPadding, VerticalPadding);

	ChildSlot
		[
			SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(NoPadding, NoPadding, NoPadding, BottomPadding)
				[
					SAssignNew(GeometryTreeWidget, STreeView<TSharedPtr<FChaosVDGeometryTreeItem>>)
						.OnGenerateRow(this, &SChaosVDGeometryTree::GenerateGeometryDataRow)
						.OnSelectionChanged(this, &SChaosVDGeometryTree::GeometryTreeSelectionChanged)
						.OnGetChildren(this, &SChaosVDGeometryTree::OnGetChildrenForGeometryItem)
						.TreeItemsSource(&InternalGeometryTreeData)
						.OnMouseButtonDoubleClick(this, &SChaosVDGeometryTree::HandleFocusRequest)
						.SelectionMode(ESelectionMode::Type::Single)
						.HighlightParentNodesForSelection(true)
						.HeaderRow(
							SNew(SHeaderRow).Style(&FAppStyle::Get().GetWidgetStyle<FHeaderRowStyle>("PropertyTable.HeaderRow"))
							+ SHeaderRow::Column(ColumnNames.Type)
							.SortMode(EColumnSortMode::None)
							.FillWidth(2)
							.HAlignCell(HAlign_Left)
							[
								SNew(STextBlock)
									.Margin(ColumnHeaderTextMargin)
									.Text(LOCTEXT("ImplicitObjectTypeHeader", "Type"))
							]
							+ SHeaderRow::Column(ColumnNames.TraceType)
							.SortMode(EColumnSortMode::None)
							.FillWidth(1.5)
							.HAlignCell(HAlign_Left)
							[
								SNew(STextBlock)
									.Margin(ColumnHeaderTextMargin)
									.Text(LOCTEXT("ImplicitObjectTraceTypeHeader", "Collision Trace Type"))
							]
							+ SHeaderRow::Column(ColumnNames.CollisionEnabled)
							.SortMode(EColumnSortMode::None)
							.FillWidth(1.5)
							.HAlignCell(HAlign_Left)
							[
								SNew(STextBlock)
									.Margin(ColumnHeaderTextMargin)
									.Text(LOCTEXT("ImplicitObjectComplexityCollisionEnabled", "Collision Enabled"))
							]
						)
				]
		];
}

TSharedRef<ITableRow> SChaosVDGeometryTree::GenerateGeometryDataRow(TSharedPtr<FChaosVDGeometryTreeItem> GeometryData, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (!GeometryData)
	{
		return
			SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
			[
				SNew(SVerticalBox)
					+ SVerticalBox::Slot()
					[
						SNew(STextBlock)
							.Text(LOCTEXT("SChaosVDGeometryHierarchyErrorMessage", "Failed to read data for solver."))
					]
			];
	}

	return SNew(SChaosVDGeometryTreeRow, OwnerTable)
		.Item(GeometryData);
}

void SChaosVDGeometryTree::GeometryTreeSelectionChanged(TSharedPtr<FChaosVDGeometryTreeItem> SelectedItem, ESelectInfo::Type Type)
{
	SelectItem(SelectedItem, Type);
	GeometryItemSelectedDelegate.ExecuteIfBound(SelectedItem, Type);
}

void SChaosVDGeometryTree::OnGetChildrenForGeometryItem(TSharedPtr<FChaosVDGeometryTreeItem> GeomEntry, TArray<TSharedPtr<FChaosVDGeometryTreeItem>>& OutGeoms)
{
	OutGeoms.Append(GeomEntry->Children);
}

void SChaosVDGeometryTree::ExpandAll(const TArray<TSharedPtr<FChaosVDGeometryTreeItem>>& Items)
{
	for (const TSharedPtr<FChaosVDGeometryTreeItem>& Item : Items)
	{
		if (SelectedParticle && SelectedParticle->GetSelectedMeshInstance().IsValid())
		{
			if (TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView> View = Item->ItemWeakPtr.Pin())
			{
				if(View->GetMeshData() == SelectedParticle->GetSelectedMeshInstance().Pin())
				{
					GeometryTreeWidget->SetSelection(Item);
				}
			}
		}

		GeometryTreeWidget->SetItemExpansion(Item, true);
		ExpandAll(Item->Children);
	}
}

void SChaosVDGeometryTree::SelectItem(const TSharedPtr<FChaosVDGeometryTreeItem>& ItemToSelect, ESelectInfo::Type Type)
{
	if (!ItemToSelect.IsValid() || Type == ESelectInfo::Direct)
	{
		return;
	}

	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		TSharedPtr<FChaosVDSceneParticle> ParticleInstance = ScenePtr->GetParticleInstance(SelectedParticle->GetParticleData()->SolverID, SelectedParticle->GetParticleData()->ParticleIndex);
		
		if(ParticleInstance)
		{
			if (InternalGeometryTreeData.Num() > 0 && InternalGeometryTreeData[0] == ItemToSelect)
			{
				Chaos::VisualDebugger::SelectParticleWithGeometryInstance(ScenePtr.ToSharedRef(), ParticleInstance.Get(), nullptr);
			}
			else
			{
				if (TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView> View = ItemToSelect->ItemWeakPtr.Pin())
				{
					if (const TSharedPtr<FChaosVDInstancedMeshData> MeshDataHandle = View->GetMeshData())
					{
						Chaos::VisualDebugger::SelectParticleWithGeometryInstance(ScenePtr.ToSharedRef(), ParticleInstance.Get(), MeshDataHandle);
					}
				}
			}
		}
	}
}

void SChaosVDGeometryTree::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (SelectedParticle)
	{
		FChaosVDBaseSceneObject::EStreamingState NewParticleStreamingState = SelectedParticle->GetStreamingState();
		
		// If the particle data has just been streamed in, force refresh the tree.
		if (NewParticleStreamingState == FChaosVDBaseSceneObject::EStreamingState::Visible && ParticleStreamingState == FChaosVDBaseSceneObject::EStreamingState::Hidden)
		{
			RefreshTreeData(true);
		}

		ParticleStreamingState = NewParticleStreamingState;
	}
}

void SChaosVDGeometryTree::HandleFocusRequest(TSharedPtr<FChaosVDGeometryTreeItem> InFocusedItem)
{
	GeometryItemFocusedDelegate.ExecuteIfBound(InFocusedItem);
	SelectItem(InFocusedItem, ESelectInfo::Type::Direct);

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (ScenePtr && SelectedParticle)
	{
		ScenePtr->OnFocusRequest().Broadcast(SelectedParticle->GetBoundingBox());
	}
}

void SChaosVDGeometryTree::GenerateTree(Chaos::VD::FChaosVDImplicitObjectView* InLeafImplicitObject, TArray<TSharedPtr<FChaosVDGeometryTreeItem>>& Nodes)
{
	using namespace Chaos;

	if (!InLeafImplicitObject)
	{
		return;
	}

	TSharedPtr<FChaosVDGeometryTreeItem> NewTreeItem = MakeShared<FChaosVDGeometryTreeItem>();
	NewTreeItem->Name = FText::FromName(FImplicitObject::GetTypeName(InLeafImplicitObject->ImplicitObjectType));
	NewTreeItem->Type = InLeafImplicitObject->ImplicitObjectType;

	NewTreeItem->ItemWeakPtr = InLeafImplicitObject->AsWeak();

	Nodes.Add(NewTreeItem);

	TArray<TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>& Children = *InLeafImplicitObject->GetChildren();

	if (NewTreeItem->Type == Chaos::ImplicitObjectType::Transformed)
	{
		if (Children.Num() == 1 && !(Children[0]->ImplicitObjectType == Chaos::ImplicitObjectType::Transformed))
		{
			NewTreeItem->Name = FText::FromString(FString(NewTreeItem->Name.ToString()).Append(" ").Append(FImplicitObject::GetTypeName(Children[0]->ImplicitObjectType).ToString()));
		}
	}

	if (InLeafImplicitObject->GetMeshData())
	{ 
		FChaosVDShapeCollisionData& Data = InLeafImplicitObject->GetMeshData()->GetGeometryCollisionData();
		NewTreeItem->CollisionEnabled = UEnum::GetDisplayValueAsText(CollisionEnabledFromFlags(Data.bQueryCollision, Data.bSimCollision, Data.bIsProbe));
		NewTreeItem->TraceType =  UEnum::GetDisplayValueAsText(Data.CollisionTraceType);
	}

	for (TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView> Child : Children)
	{
		GenerateTree(Child.Get(), NewTreeItem->Children);
	}
}

void SChaosVDGeometryTree::GenerateTree(const Chaos::FImplicitObject* InLeafImplicitObject, TArray<TSharedPtr<FChaosVDGeometryTreeItem>>& Nodes, int ShapeIndex)
{
	using namespace Chaos;

	if (!InLeafImplicitObject || !SelectedParticle)
	{
		return;
	}

	TSharedPtr<FChaosVDGeometryTreeItem> NewTreeItem = MakeShared<FChaosVDGeometryTreeItem>();
	NewTreeItem->Name = FText::FromName(InLeafImplicitObject->GetTypeName());
	NewTreeItem->Type = GetInnerType(InLeafImplicitObject->GetType());

	Nodes.Add(NewTreeItem);

	const EImplicitObjectType InnerType = GetInnerType(InLeafImplicitObject->GetType());

	if (InnerType == ImplicitObjectType::Union || InnerType == ImplicitObjectType::UnionClustered)
	{
		if (const FImplicitObjectUnion* Union = InLeafImplicitObject->template AsA<FImplicitObjectUnion>())
		{
			const bool bIsRootUnion = ShapeIndex == 0;
			const bool bIsCluster = InnerType == ImplicitObjectType::UnionClustered;

			for (int32 ObjectIndex = 0; ObjectIndex < Union->GetObjects().Num(); ++ObjectIndex)
			{
				const FImplicitObjectPtr UnionImplicit = Union->GetObjects()[ObjectIndex];
				
				if (bIsRootUnion)
				{
					if (bIsCluster)
					{
						// Geometry Collections might break the usual rule of how may shape data instances we have per geometry
						// Sometimes they can create clusters where all particles share a single instance
						if (SelectedParticle->GetParticleData() && SelectedParticle->GetParticleData()->CollisionDataPerShape.Num() == 1)
						{
							ShapeIndex = 0;
						}
					}
					else
					{
						ShapeIndex = ObjectIndex;
					}
				}

				GenerateTree(UnionImplicit, NewTreeItem->Children, ShapeIndex);
			}
		}
	}
	else if (InnerType == ImplicitObjectType::Transformed)
	{
		if (const TImplicitObjectTransformed<FReal, 3>*Transformed = InLeafImplicitObject->template GetObject<TImplicitObjectTransformed<FReal, 3>>())
		{
			if (Transformed->GetTransformedObject()) {
				NewTreeItem->Name = FText::FromString(FString(NewTreeItem->Name.ToString()).Append(" ").Append(Transformed->GetTransformedObject()->GetTypeName().ToString()));
			}
			// For transformed objects, the Instance index is the same so we pass it in without changing it
			GenerateTree(Transformed->GetTransformedObject(), NewTreeItem->Children, ShapeIndex);
		}
	}
	else
	{
		if (SelectedParticle->GetParticleData() && SelectedParticle->GetParticleData()->CollisionDataPerShape.IsValidIndex(ShapeIndex))
		{
			const FChaosVDShapeCollisionData& Data = SelectedParticle->GetParticleData()->CollisionDataPerShape[ShapeIndex];
			NewTreeItem->TraceType = UEnum::GetDisplayValueAsText(Data.CollisionTraceType);
			NewTreeItem->CollisionEnabled = UEnum::GetDisplayValueAsText(CollisionEnabledFromFlags(Data.bQueryCollision, Data.bSimCollision, Data.bIsProbe));
		}
	}
}

#undef LOCTEXT_NAMESPACE
