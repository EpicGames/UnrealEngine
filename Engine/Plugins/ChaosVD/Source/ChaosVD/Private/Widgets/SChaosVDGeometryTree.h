// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "ChaosVDScene.h"
#include "ChaosVDGeometryDataComponent.h"
#include "ChaosVDBaseSceneObject.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectType.h"

class ITableRow;
class FChaosVDScene;
struct FChaosVDSceneParticle;
struct FChaosVDBaseSceneObject;

struct FChaosVDGeometryTreeItem
{
	TWeakPtr<Chaos::VD::FChaosVDImplicitObjectView> ItemWeakPtr;
	TArray<TSharedPtr<FChaosVDGeometryTreeItem>> Children;
	FText Name = FText::GetEmpty();
	Chaos::EImplicitObjectType Type;
	FText CollisionEnabled = FText::GetEmpty();
	FText TraceType = FText::GetEmpty();
};

DECLARE_DELEGATE_TwoParams(FChaosVDGeometryTreeItemSelected, const TSharedPtr<FChaosVDGeometryTreeItem>&, ESelectInfo::Type)
DECLARE_DELEGATE_OneParam(FChaosVDGeometryTreeItemFocused, const TSharedPtr<FChaosVDGeometryTreeItem>&)

/**
 * Tree for geometry hierarchy
 */
class SChaosVDGeometryTree : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SChaosVDGeometryTree) {}
		SLATE_EVENT(FChaosVDGeometryTreeItemSelected, OnItemSelected)
		SLATE_EVENT(FChaosVDGeometryTreeItemFocused, OnItemFocused)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene>& InScenePtr);

	void SelectItem(const TSharedPtr<FChaosVDGeometryTreeItem>& ItemToSelect, ESelectInfo::Type Type);

	void ExpandAll(const TArray<TSharedPtr<FChaosVDGeometryTreeItem>>& Items);

	void SetDataToInspect(const FChaosVDSceneParticle* Particle);

	void RefreshTreeData(bool bRegenTree);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;


protected:
	void HandleFocusRequest(TSharedPtr<FChaosVDGeometryTreeItem> InFocusedItem);

	TSharedRef<ITableRow> GenerateGeometryDataRow(TSharedPtr<FChaosVDGeometryTreeItem> GeometryData, const TSharedRef<STableViewBase>& OwnerTable);

	void GeometryTreeSelectionChanged(TSharedPtr<FChaosVDGeometryTreeItem> SelectedItem, ESelectInfo::Type Type);

	void OnGetChildrenForGeometryItem(TSharedPtr<FChaosVDGeometryTreeItem> GeomEntry, TArray<TSharedPtr<FChaosVDGeometryTreeItem>>& OutGeoms);

	void GenerateTree(Chaos::VD::FChaosVDImplicitObjectView* InLeafImplicitObject, TArray<TSharedPtr<FChaosVDGeometryTreeItem>>& Nodes);

	void GenerateTree(const Chaos::FImplicitObject* InLeafImplicitObject, TArray<TSharedPtr<FChaosVDGeometryTreeItem>>& Nodes, int ShapeIndex = 0);

	TSharedPtr<STreeView<TSharedPtr<FChaosVDGeometryTreeItem>>> GeometryTreeWidget;

	TArray<TSharedPtr<FChaosVDGeometryTreeItem>> InternalGeometryTreeData;

	FChaosVDGeometryTreeItemSelected GeometryItemSelectedDelegate;
	FChaosVDGeometryTreeItemFocused GeometryItemFocusedDelegate;

	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	const FChaosVDSceneParticle* SelectedParticle = nullptr;
	FChaosVDBaseSceneObject::EStreamingState ParticleStreamingState = FChaosVDBaseSceneObject::EStreamingState::Hidden;

public:

	struct FColumnNames
	{
		const FName Type = FName("Type");
		const FName TraceType = FName("Collision Trace Type");
		const FName CollisionEnabled = FName("Collision Enabled");
	};

	static FColumnNames ColumnNames;
};
