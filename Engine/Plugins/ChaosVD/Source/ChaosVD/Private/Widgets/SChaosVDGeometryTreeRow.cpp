// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDGeometryTreeRow.h"

#include "SChaosVDGeometryTree.h"
#include "ChaosVDStyle.h"
#include "SlateOptMacros.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosVDGeometryTreeRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
									.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

	SMultiColumnTableRow<TSharedPtr<const FChaosVDGeometryTreeItem>>::Construct(Args, InOwnerTableView);
}

TSharedRef<SWidget> SChaosVDGeometryTreeRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (!Item)
	{
		return SNullWidget::NullWidget;
	}

	if (ColumnName == SChaosVDGeometryTree::ColumnNames.Type)
	{
		constexpr float NoPadding = 0.0f;
		constexpr float ExpanderLeftPadding = 6.0f;
		constexpr float ExpanderIndentAmount = 12.0f;
		return SNew(SBox)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(ExpanderLeftPadding, NoPadding, NoPadding, NoPadding)
				[
					SNew(SExpanderArrow, SharedThis(this)).IndentAmount(ExpanderIndentAmount)
				]
					+ SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SImage)
							.IsEnabled(false)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(this, &SChaosVDGeometryTreeRow::GetTypeIcon)
					]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					GenerateTextWidgetFromText(Item->Name)
				]
			];
	}

	if (ColumnName == SChaosVDGeometryTree::ColumnNames.TraceType)
	{
		return GenerateTextWidgetFromText(Item->TraceType);
	}

	if (ColumnName == SChaosVDGeometryTree::ColumnNames.CollisionEnabled)
	{
		return GenerateTextWidgetFromText(Item->CollisionEnabled);
	}

	return SNullWidget::NullWidget;
}

const FSlateBrush* SChaosVDGeometryTreeRow::GetTypeIcon() const
{
	if (!Item)
	{
		return nullptr;
	}

	Chaos::EImplicitObjectType Type = Item->Type;

	if (Item->Type == Chaos::ImplicitObjectType::Transformed && Item->Children.Num() == 1)
	{
		Type = Item->Children[0]->Type;
	}

	if (Type == Chaos::ImplicitObjectType::Box)
	{
		return FAppStyle::Get().GetBrush(TEXT("ClassIcon.Cube"));
	}
	else if (Type == Chaos::ImplicitObjectType::Sphere)
	{
		return FAppStyle::Get().GetBrush(TEXT("ClassIcon.Sphere"));
	}
	else if (Type == Chaos::ImplicitObjectType::TriangleMesh)
	{
		return FAppStyle::Get().GetBrush(TEXT("ClassIcon.StaticMesh"));
	}
	else if (Type == Chaos::ImplicitObjectType::HeightField)
	{
		return FAppStyle::Get().GetBrush(TEXT("ClassIcon.Landscape"));
	}
	else if (Type == Chaos::ImplicitObjectType::Capsule)
	{
		return FAppStyle::Get().GetBrush(TEXT("ClassIcon.TriggerCapsule"));
	}
	else if (Type == Chaos::ImplicitObjectType::Convex)
	{
		return FAppStyle::Get().GetBrush(TEXT("ClassIcon.Cone"));
	}
	else if (Type == Chaos::ImplicitObjectType::Union)
	{
		return FAppStyle::Get().GetBrush(TEXT("ClassIcon.GeometryCollection"));
	}
	else if (Type == Chaos::ImplicitObjectType::Unknown)
	{
		return FChaosVDStyle::Get().GetBrush(TEXT("RigidBodyIcon"));
	}

	return FAppStyle::Get().GetBrush(TEXT("ClassIcon.Sphere"));
}

TSharedRef<SWidget> SChaosVDGeometryTreeRow::GenerateTextWidgetFromName(FName Name)
{
	return GenerateTextWidgetFromText(FText::FromName(Name));
}

TSharedRef<SWidget> SChaosVDGeometryTreeRow::GenerateTextWidgetFromText(const FText& Text)
{
	constexpr float MarginLeft = 4.0f;
	constexpr float NoMargin = 0.0f;

	return SNew(STextBlock)
		.Margin(FMargin(MarginLeft, NoMargin, NoMargin, NoMargin))
		.Text(Text)
		.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis);
}

#undef LOCTEXT_NAMESPACE