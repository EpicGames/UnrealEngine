// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

struct FChaosVDQueryDataWrapper;
struct FChaosVDGeometryTreeItem;
class FChaosVDImplicitObjectView;

/**
 * Widget used to represent a row on the geometry hierarchy tree view
 */
class SChaosVDGeometryTreeRow : public SMultiColumnTableRow<TSharedPtr<const FChaosVDGeometryTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SChaosVDGeometryTreeRow)
		{
		}
		SLATE_ARGUMENT(TSharedPtr<FChaosVDGeometryTreeItem>, Item)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	TSharedRef<SWidget> GenerateTextWidgetFromName(FName Name);
	TSharedRef<SWidget> GenerateTextWidgetFromText(const FText& Text);

	const FSlateBrush* GetTypeIcon() const;

	TSharedPtr<FChaosVDGeometryTreeItem> Item;
};
