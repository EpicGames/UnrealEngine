// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDLogBrowserToolbar.h"

#include "SChaosVDRecordedLogBrowser.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "ToolMenuSection.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/SChaosVDEnumFlagsMenu.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SChaosVDLogBrowserToolbar)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosVDLogBrowserToolbar::Construct(const FArguments& InArgs, const TWeakPtr<SChaosVDRecordedLogBrowser>& InLogBrowserWeakPtr)
{
	LogBrowserInstanceWeakPtr = InLogBrowserWeakPtr;

	ChildSlot
	[
		GenerateMainToolbarWidget()
	];
}

void SChaosVDLogBrowserToolbar::RegisterMainToolbarMenu()
{
	const UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered(SChaosVDRecordedLogBrowser::ToolBarName))
	{
		return;
	}

	UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(SChaosVDRecordedLogBrowser::ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);

	FToolMenuSection& MainSection = ToolBar->AddSection(TEXT("LogBrowser.Toolbar.FiltersSection"));

	MainSection.AddDynamicEntry(TEXT("MainSectionEntry"), FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& MenuSection)
	{
		const UChaosVDLogBrowserToolbarMenuContext* Context = MenuSection.FindContext<UChaosVDLogBrowserToolbarMenuContext>();
		TSharedPtr<SChaosVDLogBrowserToolbar> ToolBarInstance = Context ? Context->ToolbarInstanceWeak.Pin() : nullptr;

		MenuSection.AddEntry(FToolMenuEntry::InitWidget("SearchBar", ToolBarInstance->GenerateSearchBarWidget(), FText::GetEmpty()));

		const FText FiltersMenuLabel = LOCTEXT("LogBrowserFiltersMenuLabel", "Filters");
		constexpr bool bInOpenSubMenuOnClick = false;
		MenuSection.AddSubMenu(TEXT("Filters"), FiltersMenuLabel, LOCTEXT("LogBrowserFiltersSubMenuTooltip", "Hide logs based on their category"), FNewToolMenuDelegate::CreateSP(ToolBarInstance.ToSharedRef(), &SChaosVDLogBrowserToolbar::GenerateFiltersSubMenu, FiltersMenuLabel), bInOpenSubMenuOnClick, FSlateIcon(FAppStyle::GetAppStyleSetName(), "Icons.Filter"));
	}));
}

TSharedRef<SWidget> SChaosVDLogBrowserToolbar::GenerateMainToolbarWidget()
{
	RegisterMainToolbarMenu();

	FToolMenuContext MenuContext;

	UChaosVDLogBrowserToolbarMenuContext* CommonContextObject = NewObject<UChaosVDLogBrowserToolbarMenuContext>();
	CommonContextObject->ToolbarInstanceWeak = SharedThis(this);

	MenuContext.AddObject(CommonContextObject);

	return UToolMenus::Get()->GenerateWidget(SChaosVDRecordedLogBrowser::ToolBarName, MenuContext);
}

TSharedRef<SWidget> SChaosVDLogBrowserToolbar::GenerateSearchBarWidget()
{
	TSharedPtr<SWidget> SearchBox =
		SNew(SBox)
		.MinDesiredWidth(400.0f)
		[
			SNew(SSearchBox)
			.HintText(FText::FromString(TEXT("Search by message, category or verbosity...")))
			.OnTextChanged(this, &SChaosVDLogBrowserToolbar::HandleSearchTextChanged)
			.DelayChangeNotificationsWhileTyping(true)
		];

	return SearchBox.ToSharedRef();
}

void SChaosVDLogBrowserToolbar::GenerateCategoriesSubMenu(UToolMenu* Menu)
{
	const UChaosVDLogBrowserToolbarMenuContext* Context = Menu ? Menu->FindContext<UChaosVDLogBrowserToolbarMenuContext>() : nullptr;
	TSharedPtr<SChaosVDLogBrowserToolbar> ToolBarInstance = Context ? Context->ToolbarInstanceWeak.Pin() : nullptr;

	if (TSharedPtr<SChaosVDRecordedLogBrowser> LogBrowserInstance = ToolBarInstance ? ToolBarInstance->LogBrowserInstanceWeakPtr.Pin() : nullptr)
	{
		bool bHasCategories = false;
		LogBrowserInstance->EnumerateNonEmptyCategories([ToolBarInstance, Menu, &bHasCategories](const SChaosVDRecordedLogBrowser::FCategorizedItemsContainer& InCategoryContainer)
		{
			FText CategoryNameAsText = FText::FromName(InCategoryContainer.CategoryName);
			FToolMenuEntry Entry = FToolMenuEntry::InitMenuEntry(
			NAME_None,
			CategoryNameAsText,
			FText::Format(LOCTEXT("LogCategoryTooltip", "Enable/disable the {0} category"), CategoryNameAsText),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(ToolBarInstance.ToSharedRef(), &SChaosVDLogBrowserToolbar::ToggleCategoryEnabledState, InCategoryContainer.CategoryName),
			FCanExecuteAction::CreateLambda([](){ return true; }), FIsActionChecked::CreateSP(ToolBarInstance.ToSharedRef(), &SChaosVDLogBrowserToolbar::IsCategoryEnabled, InCategoryContainer.CategoryName)), EUserInterfaceActionType::ToggleButton);

			Menu->AddMenuEntry(NAME_None, Entry);

			bHasCategories = true;
		});

		if (!bHasCategories)
		{
			FText ErrorLabel = LOCTEXT("EmptyCategoryListLabel", "No Categories Available");
			TSharedRef<SWidget> EmptyCategoriesWidget = SNew(STextBlock)
														.Margin(FMargin(4.0f,0.0f))
														.Text(ErrorLabel);
			Menu->AddMenuEntry(NAME_Name, FToolMenuEntry::InitWidget(NAME_None, EmptyCategoriesWidget, FText::GetEmpty()));
		}
	}
}

void SChaosVDLogBrowserToolbar::GenerateFiltersSubMenu(UToolMenu* Menu, FText FiltersMenuLabel)
{
	const UChaosVDLogBrowserToolbarMenuContext* Context = Menu ? Menu->FindContext<UChaosVDLogBrowserToolbarMenuContext>() : nullptr;
	TSharedPtr<SChaosVDLogBrowserToolbar> ToolBarInstance = Context ? Context->ToolbarInstanceWeak.Pin() : nullptr;

	if (TSharedPtr<SChaosVDRecordedLogBrowser> LogBrowserInstance = ToolBarInstance ? ToolBarInstance->LogBrowserInstanceWeakPtr.Pin() : nullptr)
	{
		FToolMenuSection& Section = Menu->AddSection(TEXT("LogBrowser.Toolbar.Filters"), FiltersMenuLabel);

		Section.AddMenuEntry(
			NAME_None,
			LOCTEXT("LogBrowserShowAllCategories", "Show All"),
			LOCTEXT("LogBrowserShowAllCategories_Tooltip", "Filter the Recorded Output Log to show all categories"),
			FSlateIcon(),
			FUIAction(FExecuteAction::CreateSP(LogBrowserInstance.ToSharedRef(), &SChaosVDRecordedLogBrowser::ToggleShowAllCategories),
				FCanExecuteAction::CreateLambda([] { return true; }),
				FIsActionChecked::CreateSP(LogBrowserInstance.ToSharedRef(), &SChaosVDRecordedLogBrowser::GetShowAllCategories)),
			EUserInterfaceActionType::ToggleButton
		);

		Section.AddSubMenu(TEXT("LogBrowser.Toolbar.Filters.Categories"), LOCTEXT("LogBrowserFiltersSubMenuLabel", "Categories"), FText::GetEmpty(), FNewToolMenuDelegate::CreateSP(ToolBarInstance.ToSharedRef(), &SChaosVDLogBrowserToolbar::GenerateCategoriesSubMenu));

		FToolMenuSection& VerbositySection = Menu->AddSection(TEXT("LogBrowser.Toolbar.Verbosity"), LOCTEXT("LogBrowserMenuVerbosityLabel", "Verbosity"));
		TSharedRef<SWidget> VerbosityFlagsWidget = SNew(SChaosVDEnumFlagsMenu<EChaosVDLogVerbosityFlags>)
								   .CurrentValue_Raw(ToolBarInstance.Get(), &SChaosVDLogBrowserToolbar::GetVerbosityFlags)
								   .OnEnumSelectionChanged_Raw(ToolBarInstance.Get(), &SChaosVDLogBrowserToolbar::SetVerbosityFlags);

		VerbositySection.AddEntry(FToolMenuEntry::InitWidget("VerbosityFlags", VerbosityFlagsWidget, FText::GetEmpty()));
	}
}

void SChaosVDLogBrowserToolbar::ToggleCategoryEnabledState(FName CategoryName)
{
	if (const TSharedPtr<SChaosVDRecordedLogBrowser>& LogBrowserInstance = LogBrowserInstanceWeakPtr.Pin())
    {
		LogBrowserInstance->ToggleCategoryEnabled(CategoryName);
    }
}

bool SChaosVDLogBrowserToolbar::IsCategoryEnabled(FName CategoryName)
{
	if (const TSharedPtr<SChaosVDRecordedLogBrowser>& LogBrowserInstance = LogBrowserInstanceWeakPtr.Pin())
	{
		return LogBrowserInstance->IsCategoryEnabled(CategoryName);
	}

	return false;
}

void SChaosVDLogBrowserToolbar::SetVerbosityFlags(EChaosVDLogVerbosityFlags NewFlags)
{
	if (const TSharedPtr<SChaosVDRecordedLogBrowser>& LogBrowserInstance = LogBrowserInstanceWeakPtr.Pin())
	{
		LogBrowserInstance->SetVerbosityFlags(NewFlags);
	}
}

EChaosVDLogVerbosityFlags SChaosVDLogBrowserToolbar::GetVerbosityFlags() const
{
	if (const TSharedPtr<SChaosVDRecordedLogBrowser>& LogBrowserInstance = LogBrowserInstanceWeakPtr.Pin())
	{
		return LogBrowserInstance->GetVerbosityFlags();
	}

	return EChaosVDLogVerbosityFlags::None;
}

void SChaosVDLogBrowserToolbar::HandleSearchTextChanged(const FText& Text)
{
	if (const TSharedPtr<SChaosVDRecordedLogBrowser>& LogBrowserInstance = LogBrowserInstanceWeakPtr.Pin())
	{
		LogBrowserInstance->HandleSearchTextChanged(Text);
	}
}

#undef LOCTEXT_NAMESPACE
