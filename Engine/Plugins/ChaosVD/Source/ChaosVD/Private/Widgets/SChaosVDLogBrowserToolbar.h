// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

#include "SChaosVDLogBrowserToolbar.generated.h"

enum class EChaosVDLogVerbosityFlags : uint8;
class SChaosVDRecordedLogBrowser;
class UToolMenu;

UCLASS()
class UChaosVDLogBrowserToolbarMenuContext : public UObject
{
	GENERATED_BODY()

public:
	TWeakPtr<class SChaosVDLogBrowserToolbar> ToolbarInstanceWeak;
};

/**
 * Toolbar Widget for the Recorded Log Browser tab
 */
class SChaosVDLogBrowserToolbar : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChaosVDLogBrowserToolbar)
		{
		}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<SChaosVDRecordedLogBrowser>& InLogBrowserWeakPtr);

private:
	void RegisterMainToolbarMenu();

	TSharedRef<SWidget> GenerateMainToolbarWidget();
	TSharedRef<SWidget> GenerateSearchBarWidget();

	void GenerateCategoriesSubMenu(UToolMenu* Menu);
	void GenerateFiltersSubMenu(UToolMenu* Menu, FText FiltersMenuLabel);
	
	void ToggleCategoryEnabledState(FName CategoryName);
	bool IsCategoryEnabled(FName CategoryName);

	void SetVerbosityFlags(EChaosVDLogVerbosityFlags NewFlags);
	EChaosVDLogVerbosityFlags GetVerbosityFlags() const;

	TWeakPtr<SChaosVDRecordedLogBrowser> LogBrowserInstanceWeakPtr;

	void HandleSearchTextChanged(const FText& Text);
};
