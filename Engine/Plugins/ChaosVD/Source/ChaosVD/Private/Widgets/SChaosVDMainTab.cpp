// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDMainTab.h"

#include "ChaosVDEditorModeTools.h"
#include "ChaosVDEngine.h"
#include "ChaosVDModule.h"
#include "ChaosVDObjectDetailsTab.h"
#include "ChaosVDOutputLogTab.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDPlaybackViewportTab.h"
#include "ChaosVDScene.h"
#include "ChaosVDSolversTracksTab.h"
#include "ChaosVDCollisionDataDetailsTab.h"
#include "ChaosVDCommands.h"
#include "ChaosVDConstraintDataInspectorTab.h"
#include "ChaosVDIndependentDetailsPanelManager.h"
#include "ChaosVDSceneParticleCustomization.h"
#include "ChaosVDRecordedLogTab.h"
#include "ChaosVDSceneQueryDataInspectorTab.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "ChaosVDWorldOutlinerTab.h"
#include "Components/ChaosVDSceneQueryDataComponent.h"
#include "DesktopPlatformModule.h"
#include "Editor.h"
#include "IAssetViewport.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/Application/SWindowTitleBar.h"
#include "Framework/Commands/UICommandList.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "HAL/PlatformApplicationMisc.h"
#include "IDesktopPlatform.h"
#include "PropertyEditorModule.h"
#include "SChaosVDPlaybackViewport.h"
#include "Widgets/SConnectionStatusBar.h"
#include "Misc/MessageDialog.h"
#include "StatusBarSubsystem.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "Chaos/ChaosVDEngineEditorBridge.h"
#include "Components/ChaosVDInstancedStaticMeshComponent.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "Components/ChaosVDSolverCharacterGroundConstraintDataComponent.h"
#include "Components/ChaosVDSolverCollisionDataComponent.h"
#include "Components/ChaosVDSolverJointConstraintDataComponent.h"
#include "Components/ChaosVDStaticMeshComponent.h"
#include "DetailsCustomizations/ChaosVDGeometryComponentCustomization.h"
#include "DetailsCustomizations/ChaosVDParticleDataWrapperCustomization.h"
#include "DetailsCustomizations/ChaosVDParticleMetadataCustomization.h"
#include "DetailsCustomizations/ChaosVDQueryDataWrappersCustomizationDetails.h"
#include "DetailsCustomizations/ChaosVDSelectionMultipleViewCustomization.h"
#include "DetailsCustomizations/ChaosVDShapeDataCustomization.h"
#include "ExtensionsSystem/ChaosVDExtensionsManager.h"

#include "Features/IModularFeatures.h"
#include "Framework/Docking/LayoutService.h"
#include "HAL/FileManager.h"
#include "Settings/ChaosVDGeneralSettings.h"
#include "Settings/ChaosVDMiscSettings.h"
#include "Styling/StyleColors.h"
#include "Styling/ToolBarStyle.h"
#include "TabSpawers/ChaosVDSceneQueryBrowserTab.h"
#include "TraceSessionsManager.h"
#include "Visualizers/ChaosVDCharacterGroundConstraintsDataComponentVisualizer.h"
#include "Visualizers/ChaosVDJointConstraintsDataComponentVisualizer.h"
#include "Visualizers/ChaosVDParticleDataComponentVisualizer.h"
#include "Visualizers/ChaosVDSceneQueryDataComponentVisualizer.h"
#include "Visualizers/ChaosVDSolverCollisionDataComponentVisualizer.h"
#include "Widgets/SChaosBrowseTraceFileSourceModal.h"
#include "Widgets/SChaosVDBrowseSessionsModal.h"
#include "Widgets/ChaosVDRecordingControls.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Notifications/SNotificationList.h"
#include "Widgets/SToolTip.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SChaosVDMainTab)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

const FName SChaosVDMainTab::MainToolBarName = FName("ChaosVD.MainToolBar"); 
const FName SChaosVDMainTab::MainMenuName = FName("ChaosVD.MainMenuBar");

void SChaosVDMainTab::Construct(const FArguments& InArgs, TSharedPtr<FChaosVDEngine> InChaosVDEngine)
{
	ChaosVDEngine = InChaosVDEngine;

	EditorModeTools = MakeShared<FChaosVDEditorModeTools>(InChaosVDEngine->GetCurrentScene());

	GlobalCommandList = MakeShared<FUICommandList>();

	BindUICommands(GlobalCommandList.ToSharedRef());
	
	EditorModeTools->SetToolkitHost(StaticCastSharedRef<SChaosVDMainTab>(AsShared()));
	OwnerTab = InArgs._OwnerTab;

	RegisterComponentVisualizer(UChaosVDSolverCollisionDataComponent::StaticClass()->GetFName(), MakeShared<FChaosVDSolverCollisionDataComponentVisualizer>());
	RegisterComponentVisualizer(UChaosVDSceneQueryDataComponent::StaticClass()->GetFName(), MakeShared<FChaosVDSceneQueryDataComponentVisualizer>());
	RegisterComponentVisualizer(UChaosVDParticleDataComponent::StaticClass()->GetFName(), MakeShared<FChaosVDParticleDataComponentVisualizer>());
	RegisterComponentVisualizer(UChaosVDSolverJointConstraintDataComponent::StaticClass()->GetFName(), MakeShared<FChaosVDJointConstraintsDataComponentVisualizer>());
	RegisterComponentVisualizer(UChaosVDSolverCharacterGroundConstraintDataComponent::StaticClass()->GetFName(), MakeShared<FChaosVDCharacterGroundConstraintDataComponentVisualizer>());
	
	FChaosVDExtensionsManager::Get().EnumerateExtensions([this](const TSharedRef<FChaosVDExtension>& Extension)
	{
		Extension->RegisterComponentVisualizers(StaticCastSharedRef<SChaosVDMainTab>(AsShared()));
		return true;
	});

	TabManager = FGlobalTabmanager::Get()->NewTabManager(InArgs._OwnerTab.ToSharedRef()).ToSharedPtr();

	TabManager->SetOnPersistLayout(FTabManager::FOnPersistLayout::CreateSP(this, &SChaosVDMainTab::HandlePersistLayout));

	RegisterTabSpawner<FChaosVDWorldOutlinerTab>(FChaosVDTabID::WorldOutliner);
	RegisterTabSpawner<FChaosVDObjectDetailsTab>(FChaosVDTabID::DetailsPanel);
	RegisterTabSpawner<FChaosVDOutputLogTab>(FChaosVDTabID::OutputLog);
	RegisterTabSpawner<FChaosVDPlaybackViewportTab>(FChaosVDTabID::PlaybackViewport);
	RegisterTabSpawner<FChaosVDSolversTracksTab>(FChaosVDTabID::SolversTrack);
	RegisterTabSpawner<FChaosVDCollisionDataDetailsTab>(FChaosVDTabID::CollisionDataDetails);
	RegisterTabSpawner<FChaosVDSceneQueryDataInspectorTab>(FChaosVDTabID::SceneQueryDataDetails);
	RegisterTabSpawner<FChaosVDConstraintDataInspectorTab>(FChaosVDTabID::ConstraintsInspector);
	RegisterTabSpawner<FChaosVDSceneQueryBrowserTab>(FChaosVDTabID::SceneQueryBrowser);
	RegisterTabSpawner<FChaosVDRecordedLogTab>(FChaosVDTabID::RecordedOutputLog);

	IndependentDetailsPanelManager = MakeShared<FChaosVDIndependentDetailsPanelManager>(StaticCastSharedRef<SChaosVDMainTab>(AsShared()));

	FChaosVDExtensionsManager::Get().EnumerateExtensions([this](const TSharedRef<FChaosVDExtension>& Extension)
	{
		Extension->RegisterCustomTabSpawners(StaticCastSharedRef<SChaosVDMainTab>(AsShared()));
		return true;
	});

	StatusBarID = FName(FChaosVDTabID::StatusBar.ToString() + InChaosVDEngine->GetInstanceGuid().ToString());
	
	TSharedPtr<SWidget> StatusBarWidget;
	
	if (UStatusBarSubsystem* StatusBarSubsystem = GEditor ? GEditor->GetEditorSubsystem<UStatusBarSubsystem>() : nullptr)
	{
		using namespace UE::Editor::Toolbars;
		ECreateStatusBarOptions StatusBarOptions = ECreateStatusBarOptions::HideContentBrowser | ECreateStatusBarOptions::HideOutputLog| ECreateStatusBarOptions::HideSourceControl;

		StatusBarWidget = StatusBarSubsystem->MakeStatusBarWidget(StatusBarID, TabManager->GetOwnerTab().ToSharedRef(), StatusBarOptions);

		ExtendStatusToolbar();
	}
	else
	{
		// TODO: Add a way to try to create the status bar later in case the status bar subsystem was not ready yet.
	
		StatusBarWidget = SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(2.0f)
		[
			SNew(STextBlock)
			.Text(LOCTEXT("MainTabStatusBarError", " There was an issue trying to get the status bar ready. The status bar will not be available"))
		];

		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed to obtain the status bar subsystem - The status bar will not be available", ANSI_TO_TCHAR(__FUNCTION__));		
	}

	GenerateMainWindowMenu();

	ChildSlot
	[
		// Row between the tab and main content 
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			// Create the Main Toolbar
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SNew(SImage)
				.Image(&FAppStyle::Get().GetWidgetStyle<FToolBarStyle>("SlimToolBar").BackgroundBrush)
			]
			+ SOverlay::Slot()
			[
				SNew(SBorder)
				.Padding(0)
				.BorderImage(FAppStyle::Get().GetBrush("NoBorder"))
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.HAlign(HAlign_Left)
					[
						GenerateMainToolbarWidget()
					]
				]
			]
		]
		// Main Visual Debugger Interface content
		+SVerticalBox::Slot()
		.Padding(FMargin(0.0f,5.0f,0.0f,0.0f))
		[
			TabManager->RestoreFrom(FLayoutSaveRestore::LoadFromConfig(GEditorLayoutIni, GenerateDefaultLayout()), TabManager->GetOwnerTab()->GetParentWindow()).ToSharedRef()
		]
		+SVerticalBox::Slot()
		.Padding(0.0f, 2.0f, 0.0f, 0.0f)
		.AutoHeight()
		[
			StatusBarWidget.ToSharedRef()
		]
	];

	FChaosVDExtensionsManager::Get().OnExtensionRegistered().AddSP(this, &SChaosVDMainTab::HandlePostInitializationExtensionRegistered);

	// Make sure these tabs are always focused at the start
	TabManager->TryInvokeTab(FChaosVDTabID::SolversTrack);
	TabManager->TryInvokeTab(FChaosVDTabID::DetailsPanel);

	FChaosVDExtensionsManager::Get().EnumerateExtensions([this](const TSharedRef<FChaosVDExtension>& Extension)
	{
		Extension->PostMainTabInitialization(StaticCastSharedRef<SChaosVDMainTab>(AsShared()));
		return true;
	});

	SetUpDisableCPUThrottlingDelegate();

	if (TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		RemoteSessionManager->StartSessionDiscovery(Chaos::VD::DebuggerGuid);
	}
}

SChaosVDMainTab::~SChaosVDMainTab()
{
	if (TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		RemoteSessionManager->StopSessionDiscovery(Chaos::VD::DebuggerGuid);
	}

	FChaosVDExtensionsManager::Get().OnExtensionRegistered().RemoveAll(this);
	CleanUpDisableCPUThrottlingDelegate();

	if (UToolMenus* ToolMenus = UToolMenus::Get())
	{
		if(ToolMenus->IsMenuRegistered(MainMenuName))
		{
			ToolMenus->RemoveMenu(MainMenuName);
		}
	}
}

void SChaosVDMainTab::AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder, TSharedPtr<IAssetViewport> InViewport)
{
	if (InViewport)
	{
		InViewport->AddOverlayWidget(InOverlaidWidget, ZOrder);
		return;
	}
	
	{
		TSharedPtr<FChaosVDPlaybackViewportTab> PlaybackViewportTab = GetTabSpawnerInstance<FChaosVDPlaybackViewportTab>(FChaosVDTabID::PlaybackViewport).Pin();
		TSharedPtr<SChaosVDPlaybackViewport> PlaybackViewportWidget = PlaybackViewportTab ? PlaybackViewportTab->GetPlaybackViewportWidget().Pin() : nullptr;
		if (PlaybackViewportWidget)
		{
			PlaybackViewportWidget->AddViewportOverlayWidget(InOverlaidWidget, ZOrder);
		}
	}
}

void SChaosVDMainTab::ExtendMainToolbar()
{
	if (!RecordingControls)
	{
		if (TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
		{
			RecordingControls = MakeShared<FChaosVDRecordingControls>(StaticCastSharedRef<SChaosVDMainTab>(AsShared()), RemoteSessionManager.ToSharedRef());
			RecordingControls->Initialize();
		}
	}

	if (RecordingControls)
	{
		RecordingControls->AddToMenu(MainToolBarName);
	}

	AddSettingsMenu();
}

void SChaosVDMainTab::ExtendStatusToolbar()
{
	if (TSharedPtr<UE::TraceBasedDebuggers::FRemoteSessionsManager> RemoteSessionManager = FChaosVDEngineEditorBridge::Get().GetSessionsManager())
	{
		FName StatusBarToolbarID = FName(StatusBarID.ToString() + TEXT(".Toolbar"));
	
		if (UToolMenu* StatusToolBarMenu = UToolMenus::Get()->ExtendMenu(StatusBarToolbarID))
		{
			FToolMenuSection& ConnectionStatusSection = StatusToolBarMenu->AddSection("ConnectionStatus");
	
			constexpr bool bNoIndent = true;
			constexpr bool bSearchable = false;
			ConnectionStatusSection.AddEntry(FToolMenuEntry::InitWidget("ConnectionStatusBar"
				, SNew(UE::TraceBasedDebuggers::SConnectionStatusBar, Chaos::VD::DebuggerGuid, RemoteSessionManager.ToSharedRef())
				, FText::GetEmpty()
				, bNoIndent
				, bSearchable));
		}
	}
}

void SChaosVDMainTab::BindUICommands(const TSharedRef<FUICommandList>& InGlobalUICommandsRef)
{
	const FChaosVDCommands& Commands = FChaosVDCommands::Get();

	FUIAction OpenFileAction;
	OpenFileAction.ExecuteAction.BindSPLambda(SharedThis(this), [this]
	{
		BrowseAndOpenChaosVDRecording();
	});
	InGlobalUICommandsRef->MapAction(Commands.OpenFile, OpenFileAction);

	FUIAction CombineOpenFilesAction;
	CombineOpenFilesAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDMainTab::CombineOpenSessions);
	CombineOpenFilesAction.CanExecuteAction.BindSP(GetChaosVDEngineInstance(), &FChaosVDEngine::CanCombineOpenSessions);
	InGlobalUICommandsRef->MapAction(Commands.CombineOpenFiles, CombineOpenFilesAction);
	
	FUIAction BrowseLiveSessionsAction;
	BrowseLiveSessionsAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDMainTab::BrowseLiveSessionsFromTraceStore);
	InGlobalUICommandsRef->MapAction(Commands.BrowseLiveSessions, BrowseLiveSessionsAction);

	FUIAction DisconnectFromLiveSessionAction;
	DisconnectFromLiveSessionAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDMainTab::HandleDisconnectSessionClicked);
	InGlobalUICommandsRef->MapAction(Commands.DisconnectFromLiveSession, DisconnectFromLiveSessionAction);

	FUIAction OpenSceneQueryBrowserAction;
	OpenSceneQueryBrowserAction.ExecuteAction.BindSPLambda(SharedThis(this), [this]()
	{
		TabManager->TryInvokeTab(FChaosVDTabID::SceneQueryBrowser);
	});

	InGlobalUICommandsRef->MapAction(Commands.OpenSceneQueryBrowser, OpenSceneQueryBrowserAction);
}

void SChaosVDMainTab::BringToFront()
{
	if (TabManager.IsValid())
	{
		if (const TSharedPtr<SDockTab> TabPtr = OwnerTab.Pin())
		{
			TabManager->DrawAttention(TabPtr.ToSharedRef());
		}
	}
}

void SChaosVDMainTab::OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit)
{
}

void SChaosVDMainTab::OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit)
{
}

UWorld* SChaosVDMainTab::GetWorld() const
{
	return GetChaosVDEngineInstance()->GetCurrentScene()->GetUnderlyingWorld();
}

TSharedPtr<FChaosVDScene> SChaosVDMainTab::GetScene()
{
	return GetChaosVDEngineInstance()->GetCurrentScene();
}

FEditorModeTools& SChaosVDMainTab::GetEditorModeManager() const
{
	check(EditorModeTools.IsValid())
	return *EditorModeTools;
}

TSharedPtr<FComponentVisualizer> SChaosVDMainTab::FindComponentVisualizer(UClass* ClassPtr)
{
	TSharedPtr<FComponentVisualizer> Visualizer;
	while (!Visualizer.IsValid() && (ClassPtr != nullptr) && (ClassPtr != UActorComponent::StaticClass()))
	{
		Visualizer = FindComponentVisualizer(ClassPtr->GetFName());
		ClassPtr = ClassPtr->GetSuperClass();
	}

	return Visualizer;
}

TSharedPtr<FComponentVisualizer> SChaosVDMainTab::FindComponentVisualizer(FName ClassName)
{
	TSharedPtr<FComponentVisualizer>* FoundVisualizer = ComponentVisualizersMap.Find(ClassName);

	return FoundVisualizer ? *FoundVisualizer : nullptr;
}

void SChaosVDMainTab::RegisterComponentVisualizer(FName ClassName, const TSharedPtr<FComponentVisualizer>& Visualizer)
{
	if (!ComponentVisualizersMap.Contains(ClassName))
	{
		ComponentVisualizersMap.Add(ClassName, Visualizer);
		ComponentVisualizers.Add(Visualizer);
	}
}

const TSharedPtr<FChaosVDIndependentDetailsPanelManager>& SChaosVDMainTab::GetIndependentDetailsPanelManager()
{
	return IndependentDetailsPanelManager;
}

void SChaosVDMainTab::UpdateTraceProcessingNotificationUpdate()
{
	bool bIsTraceAnalysisInProgress = false;
	TSharedRef<FChaosVDEngine> CVDEngineInstance = GetChaosVDEngineInstance();

	// Live sessions should not show the notification pop up. When CVD is connected in live debugging mode the UI already has feedback to inform the user.
	if (CVDEngineInstance->GetTraceSessionDescriptors().Num() > 0 && !CVDEngineInstance->HasAnyLiveSessionActive())
	{
		FChaosVDModule::Get().GetTraceSessionsManager()->EnumerateActiveSessions([&bIsTraceAnalysisInProgress](const TSharedRef<const TraceServices::IAnalysisSession>& InSession)
		{
			// TODO: We should implement a way to gather the actual progress so we can report that in the UI
			// but the Trace API don't have that data yet.
			if (!InSession->IsAnalysisComplete())
			{
				bIsTraceAnalysisInProgress = true;
				return false;
			}

			return true;
		});
	}

	if (bIsTraceAnalysisInProgress)
	{
		ShowTraceDataProcessingNotification();
	}
	else
	{
		HideTraceDataProcessingNotification();
	}
}

void SChaosVDMainTab::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	UpdateTraceProcessingNotificationUpdate();

	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

TSharedPtr<SNotificationItem> SChaosVDMainTab::CreateProcessingTraceDataNotification()
{
	FNotificationInfo Info(LOCTEXT("ProcessingTraceSessionMessage", "Processing Trace Data ..."));
	Info.bFireAndForget = false;
	Info.FadeOutDuration = 3.0f;
	Info.ExpireDuration = 0.0f;

	{
		TSharedPtr<SDockTab> OwnerTabPtr = OwnerTab.Pin();
		Info.ForWindow = OwnerTabPtr ? OwnerTabPtr->GetParentWindow() : nullptr;
	}

	TSharedPtr<SNotificationItem> ProcessingDataNotification = FSlateNotificationManager::Get().AddNotification(Info);

	if (ProcessingDataNotification.IsValid())
	{
		ProcessingDataNotification->SetCompletionState(SNotificationItem::CS_Pending);
		return ProcessingDataNotification;
	}

	return nullptr;
}

void SChaosVDMainTab::ShowTraceDataProcessingNotification()
{
	if (ActiveProcessingTraceDataNotification)
	{
		if (ActiveProcessingTraceDataNotification->GetCompletionState() != SNotificationItem::CS_Pending)
		{
			// The existing notification already expired, we need to create a new one
			ActiveProcessingTraceDataNotification = CreateProcessingTraceDataNotification();
		}
	}
	else
	{
		ActiveProcessingTraceDataNotification = CreateProcessingTraceDataNotification();
	}

	ActiveProcessingTraceDataNotification->SetCompletionState(SNotificationItem::CS_Pending);
}

void SChaosVDMainTab::HideTraceDataProcessingNotification()
{
	if (ActiveProcessingTraceDataNotification)
	{
		ActiveProcessingTraceDataNotification->SetCompletionState(SNotificationItem::CS_Success);
		ActiveProcessingTraceDataNotification->ExpireAndFadeout();
		ActiveProcessingTraceDataNotification = nullptr;
	}
}

void SChaosVDMainTab::HandlePersistLayout(const TSharedRef<FTabManager::FLayout>& InLayoutToSave)
{
	if (!bCanTabManagerPersistLayout)
	{
		return;
	}

	if (TSharedPtr<FTabManager::FArea> PrimaryArea = InLayoutToSave->GetPrimaryArea().Pin())
	{
		FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, InLayoutToSave);
	}
}

void SChaosVDMainTab::HandlePostInitializationExtensionRegistered(const TSharedRef<FChaosVDExtension>& NewExtension)
{	
	NewExtension->RegisterCustomTabSpawners(StaticCastSharedRef<SChaosVDMainTab>(AsShared()));
	NewExtension->RegisterComponentVisualizers(StaticCastSharedRef<SChaosVDMainTab>(AsShared()));

	for (const TWeakPtr<IDetailsView>& DetailsPanel : CustomizedDetailsPanels)
	{
		if (TSharedPtr<IDetailsView> DetailsPanelPtr = DetailsPanel.Pin())
		{
			NewExtension->SetCustomPropertyLayouts(DetailsPanelPtr.Get(), StaticCastSharedRef<SChaosVDMainTab>(AsShared()));
		}
	}
}

void SChaosVDMainTab::HandleTabSpawned(TSharedRef<SDockTab> Tab, FName TabID)
{
	if (!ActiveTabsByID.Contains(TabID))
	{
		ActiveTabsByID.Add(TabID, Tab);
	}
}

void SChaosVDMainTab::HandleTabDestroyed(TSharedRef<SDockTab> Tab, FName TabID)
{
	ActiveTabsByID.Remove(TabID);
}

TSharedRef<FTabManager::FLayout> SChaosVDMainTab::GenerateDefaultLayout()
{
	const float DPIScaleFactor = FPlatformApplicationMisc::GetDPIScaleFactorAtPoint(10.0f, 10.0f);

	return FTabManager::NewLayout("ChaosVisualDebugger_Layout_V1")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->SetExtensionId("TopLevelArea")
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.8f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.7f)
					->AddTab(FChaosVDTabID::PlaybackViewport, ETabState::OpenedTab)
					->SetHideTabWell(true)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->AddTab(FChaosVDTabID::SolversTrack, ETabState::OpenedTab)
					->AddTab(FChaosVDTabID::RecordedOutputLog, ETabState::OpenedTab)
					->AddTab(FChaosVDTabID::OutputLog, ETabState::OpenedTab)
				)
			)
			->Split
			(
				FTabManager::NewSplitter()
				->SetOrientation(Orient_Vertical)
				->SetSizeCoefficient(0.15f)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.7f)
					->AddTab(FChaosVDTabID::WorldOutliner, ETabState::OpenedTab)
				)
				->Split
				(
					FTabManager::NewStack()
					->SetSizeCoefficient(0.3f)
					->AddTab(FChaosVDTabID::DetailsPanel, ETabState::OpenedTab)
					->AddTab(FChaosVDTabID::CollisionDataDetails, ETabState::OpenedTab)
					->AddTab(FChaosVDTabID::SceneQueryDataDetails, ETabState::OpenedTab)
					->AddTab(FChaosVDTabID::ConstraintsInspector, ETabState::ClosedTab)
				)
			)
		)
	->AddArea(
	FTabManager::NewArea(800.0f * DPIScaleFactor, 600.0f * DPIScaleFactor)
	->SetOrientation(Orient_Vertical)
	->Split
	(
		FTabManager::NewStack()
		->SetSizeCoefficient(1.0f)
		->AddTab(FChaosVDTabID::SceneQueryBrowser, ETabState::ClosedTab)
	))
	->AddArea(
	FTabManager::NewArea(800.0f * DPIScaleFactor, 600.0f * DPIScaleFactor)
	->SetOrientation(Orient_Vertical)
	->Split
	(
		FTabManager::NewStack()
		->SetSizeCoefficient(1.0f)
		->AddTab(FChaosVDTabID::IndependentDetailsPanel1, ETabState::ClosedTab)
		->AddTab(FChaosVDTabID::IndependentDetailsPanel2, ETabState::ClosedTab)
		->AddTab(FChaosVDTabID::IndependentDetailsPanel3, ETabState::ClosedTab)
		->AddTab(FChaosVDTabID::IndependentDetailsPanel4, ETabState::ClosedTab)
	));
}

void SChaosVDMainTab::ResetLayout()
{
	// During a layout reset, we manually stomp the currently saved layout, therefore we don't want the layout
	// to be resaved for the remaining of this tab instance lifespan (which should be not longer than this scope).
	bCanTabManagerPersistLayout = false;
	FLayoutSaveRestore::SaveToConfig(GEditorLayoutIni, GenerateDefaultLayout());

	TabManager->CloseAllAreas();
	FChaosVDModule::Get().ReloadInstanceUI(GetChaosVDEngineInstance()->GetInstanceGuid());
}

void SChaosVDMainTab::CombineOpenSessions()
{
	TArray<FString> OutSelectedFilenames;
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		FString ExtensionStr;
		ExtensionStr += TEXT("CVD Multi Session|*.cvdmulti");
	
		DesktopPlatform->SaveFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("SaveDialogTitle", "Save Combined Chaos Visual Debug File").ToString(),
			*FPaths::ProfilingDir(),
			TEXT(""),
			*ExtensionStr,
			EFileDialogFlags::None,
			OutSelectedFilenames
		);
	}

	if (!OutSelectedFilenames.IsEmpty())
	{
		const FString& TargetFilePath = OutSelectedFilenames[0];
		if (GetChaosVDEngineInstance()->SaveOpenSessionToCombinedFile(TargetFilePath))
		{
			FPlatformProcess::ExploreFolder(*TargetFilePath);
		}
		else
		{
			FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, LOCTEXT("FailedCombineFilesMessage", "Failed to combine open recordings into a single file.\n\n See Logs for mor info."), LOCTEXT("FailedCombineFilesMessageTitle", "Failed combine files"));
		}
	}
}

void SChaosVDMainTab::GenerateMainWindowMenu()
{
	FToolMenuContext MenuContext(GlobalCommandList);

	UToolMenus* ToolMenus = UToolMenus::Get();

	UToolMenu* MenuBar;

	if (!ToolMenus->IsMenuRegistered(MainMenuName))
	{
		MenuBar = ToolMenus->RegisterMenu(MainMenuName, NAME_None, EMultiBoxType::MenuBar);
	}
	else
	{
		MenuBar = ToolMenus->ExtendMenu(MainMenuName);
	}

	MenuBar->StyleName = FName("WindowMenuBar");

	UToolMenu* FileEntry = MenuBar->AddSubMenu(
		"MainMenuBar",
		NAME_None,
		"File",
		LOCTEXT("FileMenuLabel", "File"),
		FText::GetEmpty()
	);

	UToolMenu* RecentSubMenu = FileEntry->AddSubMenu("File", NAME_None, "Recent Files", LOCTEXT("RecentFilesMenuLabel", "Recent Files"), LOCTEXT("RecentFilesMenuLabelToolTip", "Shows a list of recently used CVD Files"));
	RecentSubMenu->AddDynamicSection(NAME_None, FNewSectionConstructChoice(FNewToolMenuDelegateLegacy::CreateSP(this, &SChaosVDMainTab::GenerateRecentFilesMenu)));

	const FChaosVDCommands& Commands = FChaosVDCommands::Get();
	FToolMenuSection& SessionsSection = FileEntry->AddSection("LegacySessionSection", LOCTEXT("LegacySessionSectionLabel", "Legacy Sessions Management"));
	SessionsSection.AddMenuEntry(Commands.BrowseLiveSessions);
	SessionsSection.AddMenuEntry(Commands.DisconnectFromLiveSession);

	UToolMenu* WindowEntry = MenuBar->AddSubMenu(
		"MainMenuBar",
		NAME_None,
		"Window",
		LOCTEXT("WindowMenuLabel", "Window"),
		FText::GetEmpty()
	);

	WindowEntry->AddDynamicSection(NAME_None, FNewSectionConstructChoice(FNewToolMenuDelegateLegacy::CreateSPLambda(this, [this](FMenuBuilder& MenuBuilder, UToolMenu* Menu)
	{
		TabManager->PopulateLocalTabSpawnerMenu(MenuBuilder);
	})));

	TSharedRef<SWidget> MenuBarWidget = UToolMenus::Get()->GenerateWidget(MainMenuName, MenuContext);
	if (MenuBarWidget != SNullWidget::NullWidget)
	{
		TSharedRef<SMultiBoxWidget> MultiBoxWidget = StaticCastSharedRef<SMultiBoxWidget>(MenuBarWidget);

		TabManager->SetAllowWindowMenuBar(true);
		TabManager->SetMenuMultiBox(ConstCastSharedRef<FMultiBox>(MultiBoxWidget->GetMultiBox()), MultiBoxWidget);
	}
}

void SChaosVDMainTab::GenerateRecentFilesMenu(FMenuBuilder& MenuBuilder, UToolMenu* Menu)
{
	if (UChaosVDMiscSettings* MiscSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMiscSettings>())
	{
		MiscSettings->RecentFiles.Sort(FChaosVDRecentFile::FRecentFilesSortPredicate());

		for (const FChaosVDRecentFile& RecentFile : MiscSettings->RecentFiles)
		{
			const FText DisplayName = FText::FromString(FPaths::GetBaseFilename(*RecentFile.FileName));
			const FText Tooltip = FText::FromString(IFileManager::Get().ConvertToAbsolutePathForExternalAppForRead(*RecentFile.FileName) );
			MenuBuilder.AddMenuEntry(DisplayName, Tooltip, FSlateIcon(FChaosVDStyle::Get().GetStyleSetName(), "OpenFileIcon"), FUIAction(FExecuteAction::CreateSPLambda(this, [this, FileNameCopy = RecentFile.FileName](){ LoadCVDFile(FileNameCopy, EChaosVDLoadRecordedDataMode::SingleSource); })),
					NAME_None, EUserInterfaceActionType::Button);
		}
	}
}

FReply SChaosVDMainTab::BrowseAndOpenChaosVDRecording()
{
	const TSharedRef<SChaosBrowseTraceFileSourceModal> SessionBrowserModal = SNew(SChaosBrowseTraceFileSourceModal);

	EChaosVDBrowseFileModalResponse Response = SessionBrowserModal->ShowModal();
	EChaosVDLoadRecordedDataMode LoadingMode = SessionBrowserModal->GetSelectedLoadingMode();
	switch(Response)
	{
		case EChaosVDBrowseFileModalResponse::LastOpened:
			{
				BrowseChaosVDRecordingFromFolder(TEXT(""), LoadingMode);
				break;
			}
		case EChaosVDBrowseFileModalResponse::CurrentSaveFolder:
			{
				// GetSaveDirPath will return the path to use base on the current ChaosVDGeneralSettings (i.e., using DefaultSavePathOverride if set)
				const FString FolderPath = FChaosVDModule::Get().GetTraceSessionsManager()->GetSaveDirPath();
				BrowseChaosVDRecordingFromFolder(*FolderPath, LoadingMode);
				break;
			}
		case EChaosVDBrowseFileModalResponse::LocalTraceStore:
			{
				const FString TraceStorePath = FChaosVDModule::Get().GetTraceSessionsManager()->GetLocalTraceStoreDirPath();
				if (TraceStorePath.IsEmpty())
				{
					UE_LOGF(LogChaosVDEditor, Error, "[%s] Failed to access Trace Store...", __func__);
					FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("OpenTraceStoreFailedMessage", "Failed to access the Trace Store, The default profiling folder will be open. \n Please see the logs for mor details... "));
				}

				BrowseChaosVDRecordingFromFolder(TraceStorePath, LoadingMode);
				break;
			}
		case EChaosVDBrowseFileModalResponse::Cancel:
			break;
		default:
				ensureMsgf(false, TEXT("Invalid response received"));
			break;
	}

	return FReply::Handled();
}

TSharedRef<SButton> SChaosVDMainTab::CreateSimpleButton(TFunction<FText()>&& GetTextDelegate, TFunction<FText()>&& ToolTipTextDelegate, const FSlateBrush* ButtonIcon, const UChaosVDMainToolbarMenuContext* MenuContext, const FOnClicked& InButtonClickedCallback)
{
	const TSharedRef<SChaosVDMainTab> MainTab = MenuContext->MainTab.Pin().ToSharedRef();
	
	TSharedRef<SButton> Button = SNew(SButton)
								.ButtonStyle(FAppStyle::Get(), "SimpleButton")
								.ToolTipText_Lambda(MoveTemp(ToolTipTextDelegate))
								.ContentPadding(FMargin(6.0f, 0.0f))
								.OnClicked(InButtonClickedCallback)
								[
									SNew(SHorizontalBox)
									+ SHorizontalBox::Slot()
									.AutoWidth()
									.HAlign(HAlign_Center)
									.VAlign(VAlign_Center)
									[
										SNew(SImage)
										.Image(ButtonIcon)
										.ColorAndOpacity(FSlateColor::UseForeground())
									]
									+SHorizontalBox::Slot()
									.Padding(FMargin(4, 0, 0, 0))
									.VAlign(VAlign_Center)
									.AutoWidth()
									[
										SNew(STextBlock)
										.TextStyle(FAppStyle::Get(), "NormalText")
										.Text_Lambda(MoveTemp(GetTextDelegate))
									]
								];

	return Button;
}

TSharedRef<SWidget> SChaosVDMainTab::GenerateMainToolbarWidget()
{
	CreateMainToolbarMenu();

	FToolMenuContext MenuContext;
	UChaosVDMainToolbarMenuContext* CommonContextObject = NewObject<UChaosVDMainToolbarMenuContext>();
	CommonContextObject->MainTab = SharedThis(this);
	MenuContext.AddObject(CommonContextObject);

	ExtendMainToolbar();

	return UToolMenus::Get()->GenerateWidget(MainToolBarName, MenuContext);
}

void SChaosVDMainTab::LoadCVDFile(const FString& InFilename, EChaosVDLoadRecordedDataMode LoadingMode)
{
	if (ensure(IsSupportedFile(InFilename)))
	{
		GetChaosVDEngineInstance()->LoadRecording(InFilename, LoadingMode);
	}
	else
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Invalid file extension | Only UTrace files are supported | Filename [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *InFilename)			
	}
}

void SChaosVDMainTab::LoadCVDFiles(TConstArrayView<FString> InFilenames, EChaosVDLoadRecordedDataMode LoadingMode)
{
	// Ideally, we should not need to do this, but the UI to support multi source is not robust yet, The UI improvement task to avoid this scenario is planned as UE-197418
	const bool bHasMultiFileData = InFilenames.Num() > 1 || (InFilenames.IsValidIndex(0) && InFilenames[0].EndsWith(TEXT("cvdmulti")));

	if (bHasMultiFileData && LoadingMode == EChaosVDLoadRecordedDataMode::SingleSource)
	{
		GetChaosVDEngineInstance()->CloseActiveTraceSessions();
		LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource;
	
		UE_LOGF(LogChaosVDEditor, Warning, "[%ls] Single source mode was selected with multiple files. Overriding mode to Multi Source...", ANSI_TO_TCHAR(__FUNCTION__));
	}

	for (const FString& Filename : InFilenames)
	{
		LoadCVDFile(Filename, LoadingMode);
	}
}

TSharedRef<IDetailsView> SChaosVDMainTab::CreateDetailsView(const FDetailsViewArgs& InDetailsViewArgs)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");
	TSharedRef<IDetailsView> DetailsView = PropertyEditorModule.CreateDetailView(InDetailsViewArgs);
	SetCustomPropertyLayouts(&DetailsView.Get());

	return DetailsView;
}

TSharedRef<IStructureDetailsView> SChaosVDMainTab::CreateStructureDetailsView(const FDetailsViewArgs& InDetailsViewArgs, const FStructureDetailsViewArgs& InStructureDetailsViewArgs, const TSharedPtr<FStructOnScope>& InStructData, const FText& CustomName)
{
	FPropertyEditorModule& PropertyEditorModule = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	TSharedRef<IStructureDetailsView> DetailsView = PropertyEditorModule.CreateStructureDetailView(InDetailsViewArgs, InStructureDetailsViewArgs, InStructData, CustomName);
	SetCustomPropertyLayouts(DetailsView->GetDetailsView());

	return DetailsView;
}

void SChaosVDMainTab::ProccessKeyEventForPlaybackTrackSelector(const FKeyEvent& InKeyEvent)
{
	if (TSharedPtr<FChaosVDPlaybackController> PlaybackController = GetChaosVDEngineInstance()->GetPlaybackController())
	{
		uint32 KeyCode = InKeyEvent.GetKeyCode();
	
		int32 TrackSlotIndex = INDEX_NONE;

		constexpr int32 AlphaNumKeyCodeLowerBound = 48;
		constexpr int32 AlphaNumKeyCodeUpperBound = 57;
	
		constexpr int32 NumPadNumberKeyCodeLowerBound = 96;
		constexpr int32 NumPadNumberKeyCodeUpperBound = 105;
		
		if (KeyCode >= AlphaNumKeyCodeLowerBound && KeyCode <= AlphaNumKeyCodeUpperBound)
		{
			TrackSlotIndex = KeyCode - AlphaNumKeyCodeLowerBound;
		}

		if (KeyCode >= NumPadNumberKeyCodeLowerBound && KeyCode <= NumPadNumberKeyCodeUpperBound)
		{
			TrackSlotIndex = KeyCode - NumPadNumberKeyCodeLowerBound;
		}

		if (TrackSlotIndex != INDEX_NONE)
		{
			PlaybackController->TrySetActiveTrack(TrackSlotIndex);
		}
	}
}

FReply SChaosVDMainTab::OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	if (InKeyEvent.IsControlDown())
	{
		bShowTrackSelectorKeyShortcut = true;

		ProccessKeyEventForPlaybackTrackSelector(InKeyEvent);
	}

	if (!GlobalCommandList->ProcessCommandBindings(InKeyEvent))
	{
		return SCompoundWidget::OnKeyDown(MyGeometry, InKeyEvent);
	}

	return FReply::Handled();
}

FReply SChaosVDMainTab::OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent)
{
	bShowTrackSelectorKeyShortcut = false;

	return SCompoundWidget::OnKeyUp(MyGeometry, InKeyEvent);
}

bool SChaosVDMainTab::ShouldShowTracksKeyShortcuts()
{
	return bShowTrackSelectorKeyShortcut;
}

void SChaosVDMainTab::SetCustomPropertyLayouts(IDetailsView* DetailsView)
{
	if (!DetailsView)
	{
		return;
	}

	DetailsView->RegisterInstancedCustomPropertyLayout(FChaosVDSceneParticle::StaticStruct(), FOnGetDetailCustomizationInstance::CreateStatic(&FChaosVDSceneParticleCustomization::MakeInstance, StaticCastWeakPtr<SChaosVDMainTab>(AsWeak())));
	DetailsView->RegisterInstancedCustomPropertyLayout(UChaosVDInstancedStaticMeshComponent::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FChaosVDGeometryComponentCustomization::MakeInstance));
	DetailsView->RegisterInstancedCustomPropertyLayout(UChaosVDStaticMeshComponent::StaticClass(), FOnGetDetailCustomizationInstance::CreateStatic(&FChaosVDGeometryComponentCustomization::MakeInstance));
	DetailsView->RegisterInstancedCustomPropertyLayout(FChaosVDQueryVisitStep::StaticStruct(), FOnGetDetailCustomizationInstance::CreateStatic(&FChaosVDQueryVisitDataCustomization::MakeInstance, StaticCastWeakPtr<SChaosVDMainTab>(AsWeak())));
	DetailsView->RegisterInstancedCustomPropertyLayout(FChaosVDQueryDataWrapper::StaticStruct(), FOnGetDetailCustomizationInstance::CreateStatic(&FChaosVDQueryDataWrapperCustomization::MakeInstance));
	DetailsView->RegisterInstancedCustomPropertyLayout(FChaosVDSelectionMultipleView::StaticStruct(), FOnGetDetailCustomizationInstance::CreateStatic(&FChaosVDSelectionMultipleViewCustomization::MakeInstance));

	//TODO: Rename FChaosVDParticleDataWrapperCustomization to something generic as currently works with any type that wants to hide properties of type FChaosVDWrapperDataBase with invalid data.
	// Or another option is create a new custom layout intended to be generic from the get go
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("ChaosVDQueryDataWrapper"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChaosVDParticleDataWrapperCustomization::MakeInstance));

	DetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("ChaosVDCollisionResponseParams"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChaosVDCollisionResponseParamsCustomization::MakeInstance, StaticCastWeakPtr<SChaosVDMainTab>(AsWeak())));
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("ChaosVDCollisionObjectQueryParams"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChaosVDCollisionObjectParamsCustomization::MakeInstance,StaticCastWeakPtr<SChaosVDMainTab>(AsWeak())));
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("ChaosVDShapeCollisionData"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChaosVDShapeDataCustomization::MakeInstance, StaticCastWeakPtr<SChaosVDMainTab>(AsWeak())));
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("ChaosVDQueryVisitStep"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChaosVDQueryVisitDataPropertyCustomization::MakeInstance, StaticCastWeakPtr<SChaosVDMainTab>(AsWeak())));
	DetailsView->RegisterInstancedCustomPropertyTypeLayout(TEXT("ChaosVDParticleMetadata"), FOnGetPropertyTypeCustomizationInstance::CreateStatic(&FChaosVDParticleMetadataCustomization::MakeInstance));

	// We don't need to validate properties, and trying to do so seems to be costing between 2-4ms per tick!
	DetailsView->SetCustomValidatePropertyNodesFunction(FOnValidateDetailsViewPropertyNodes::CreateLambda([](const FRootPropertyNodeList& Root){ return true;}));
	
	// We need to keep a weak ptr array of any panel we customized so we can apply any customization coming from late initialized extensions
	// (if the details panels are still alive)
	CustomizedDetailsPanels.Add(StaticCastWeakPtr<IDetailsView>(DetailsView->AsWeak()));

	FChaosVDExtensionsManager::Get().EnumerateExtensions([this, DetailsView](const TSharedRef<FChaosVDExtension>& Extension)
	{
		Extension->SetCustomPropertyLayouts(DetailsView, StaticCastSharedRef<SChaosVDMainTab>(AsShared()));
		return true;
	});
}

void SChaosVDMainTab::BrowseChaosVDRecordingFromFolder(FStringView FolderPath, EChaosVDLoadRecordedDataMode LoadingMode)
{
	TArray<FString> OutOpenFilenames;
	if (IDesktopPlatform* DesktopPlatform = FDesktopPlatformModule::Get())
	{
		FString ExtensionStr;
		ExtensionStr += TEXT("Unreal Trace|*.utrace|");
		ExtensionStr += TEXT("CVD Multi Session|*.cvdmulti");
		//TODO: Re-enable this when we add "Clips" support as these will use our own format
		//ExtensionStr += TEXT("Chaos Visual Debugger|*.cvd");
	
		DesktopPlatform->OpenFileDialog(
			FSlateApplication::Get().FindBestParentWindowHandleForDialogs(AsShared()),
			LOCTEXT("OpenDialogTitle", "Open Chaos Visual Debug File").ToString(),
			FolderPath.GetData(),
			TEXT(""),
			*ExtensionStr,
			EFileDialogFlags::Multiple,
			OutOpenFilenames
		);
	}

	LoadCVDFiles(OutOpenFilenames, LoadingMode);
}

bool SChaosVDMainTab::ConnectToLiveSession(int32 SessionID, const FString& InSessionAddress, EChaosVDLoadRecordedDataMode LoadingMode) const
{
	return GetChaosVDEngineInstance()->ConnectToLiveSession(SessionID, InSessionAddress, LoadingMode);
}

bool SChaosVDMainTab::ConnectToLiveSession_Direct(EChaosVDLoadRecordedDataMode LoadingMode) const
{
	UE_LOGF(LogChaosVDEditor, Error, "[%s] This method is no longer supported. Please use the overload that takes a Remote Session ID.", __func__);
	return false;
}

bool SChaosVDMainTab::ConnectToLiveSession_Direct(FGuid RemoteSessionID, EChaosVDLoadRecordedDataMode LoadingMode) const
{
	return GetChaosVDEngineInstance()->ConnectToLiveSession_Direct(RemoteSessionID, LoadingMode);
}

bool SChaosVDMainTab::ConnectToLiveSession_Relay(FGuid RemoteSessionID, EChaosVDLoadRecordedDataMode LoadingMode) const
{
	return GetChaosVDEngineInstance()->ConnectToLiveSession_Relay(RemoteSessionID, LoadingMode);
}

bool SChaosVDMainTab::IsSupportedFile(const FString& InFilename)
{
	return InFilename.EndsWith(TEXT(".utrace")) || InFilename.EndsWith(TEXT(".cvdmulti"));
}

void SChaosVDMainTab::SetUpDisableCPUThrottlingDelegate()
{
	if (GEditor)
	{
		GEditor->ShouldDisableCPUThrottlingDelegates.Add(UEditorEngine::FShouldDisableCPUThrottling::CreateSP(this, &SChaosVDMainTab::ShouldDisableCPUThrottling));
		DisableCPUThrottleHandle = GEditor->ShouldDisableCPUThrottlingDelegates.Last().GetHandle();
	}
}

void SChaosVDMainTab::CleanUpDisableCPUThrottlingDelegate() const
{
	if (GEditor)
	{
		GEditor->ShouldDisableCPUThrottlingDelegates.RemoveAll([this](const UEditorEngine::FShouldDisableCPUThrottling& Delegate)
		{
			return Delegate.GetHandle() == DisableCPUThrottleHandle;
		});
	}
}

void SChaosVDMainTab::CreateMainToolbarMenu()
{
	const UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered(MainToolBarName))
	{
		return;
	}

	UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(MainToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);

	FToolMenuSection& Section = ToolBar->AddSection("LoadRecording");
	Section.AddDynamicEntry("OpenFile", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		const UChaosVDMainToolbarMenuContext* Context = InSection.FindContext<UChaosVDMainToolbarMenuContext>();
		TSharedPtr<SChaosVDMainTab> MainTabPtr = Context->MainTab.Pin();
		if (!MainTabPtr)
		{
			return;
		}

		TSharedRef<SButton> OpenFileButton = MainTabPtr->CreateSimpleButton(
			[](){ return LOCTEXT("OpenFile", "Open File"); },
			[](){ return LOCTEXT("OpenFileDesc", "Click here to open a Chaos Visual Debugger file."); },
			FChaosVDStyle::Get().GetBrush("OpenFileIcon"),
			Context, FOnClicked::CreateLambda([WeakTab = MainTabPtr.ToWeakPtr()]()
			{
				if (TSharedPtr<SChaosVDMainTab> TabPtr = WeakTab.Pin())
				{
					return TabPtr->BrowseAndOpenChaosVDRecording();
				}

				return FReply::Handled();
			}));

		InSection.AddEntry(
			FToolMenuEntry::InitWidget(
				"OpenFileButton",
				OpenFileButton,
				FText::GetEmpty(),
				true,
				false
			));
	}));

	Section.AddDynamicEntry("CombineFiles", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		const UChaosVDMainToolbarMenuContext* Context = InSection.FindContext<UChaosVDMainToolbarMenuContext>();
		TSharedPtr<SChaosVDMainTab> MainTabPtr = Context->MainTab.Pin();
		if (!MainTabPtr)
		{
			return;
		}

		TSharedRef<SButton> CombineFilesButton = MainTabPtr->CreateSimpleButton(
			[](){ return LOCTEXT("CombineFilesLabel", "Combine"); },
			[](){ return LOCTEXT("CombineFilesLabelDesc", "Click here to combine multiple open recordings into a single file to make sharing easier."); },
			FAppStyle::Get().GetBrush("MainFrame.ZipUpProject"),
			Context, FOnClicked::CreateLambda([WeakTab = MainTabPtr.ToWeakPtr()]()
			{
				if (TSharedPtr<SChaosVDMainTab> TabPtr = WeakTab.Pin())
				{
					TabPtr->CombineOpenSessions();
				}

				return FReply::Handled();
			}));

		TAttribute<bool> IsCombineButtonEnabled;
		IsCombineButtonEnabled.BindSPLambda(MainTabPtr.ToSharedRef(), [WeakTab = MainTabPtr.ToWeakPtr()]()
		{
			if (TSharedPtr<SChaosVDMainTab> TabPtr = WeakTab.Pin())
			{
				return TabPtr->GetChaosVDEngineInstance()->CanCombineOpenSessions();
			}
			return false;
		});

		CombineFilesButton->SetEnabled(IsCombineButtonEnabled);

		InSection.AddEntry(
			FToolMenuEntry::InitWidget(
				"CombineFilesLabelButton",
				CombineFilesButton,
				FText::GetEmpty(),
				true,
				false
			));
	}));
	
	Section.AddSeparator(NAME_None);

	//TODO : This button should not be added to the toolbar here. Ideally it should be added from the SceneQueryComponent Visualizer, but we have two issues :
	// 1- The recording control buttons are still implemented as a widget we instantiate alongside the tool bar, that needs to be moved to be a properly
	// registered menu entry that is part of the toolbar.
	// 2- We need to ensure the main toolbar is created and ready to use before we allow other system to register into it.
	// Jira for tracking UE-221454

	Section.AddDynamicEntry("DataBrowsers", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		const UChaosVDMainToolbarMenuContext* Context = InSection.FindContext<UChaosVDMainToolbarMenuContext>();
		TSharedPtr<SChaosVDMainTab> MainTabPtr = Context->MainTab.Pin();
		if (!MainTabPtr)
		{
			return;
		}

		FOnClicked OnClickedDelegate = FOnClicked::CreateLambda([WeakTab = StaticCastWeakPtr<SChaosVDMainTab>(MainTabPtr->AsWeak())]()
		{
			if (TSharedPtr<SChaosVDMainTab> TabPtr = WeakTab.Pin())
			{
				TabPtr->TabManager->TryInvokeTab(FChaosVDTabID::SceneQueryBrowser);
			}

			return FReply::Handled();
		});

		TSharedRef<SButton> ConnectToSessionButton = MainTabPtr->CreateSimpleButton(
														[](){ return LOCTEXT("SceneQueryBrowserButton", "Scene Query Browser"); },
														[](){ return LOCTEXT("SceneQueryBrowserButtonTooltip", "Opens the Scene Query Browser window, which shows all the available scene queries in the current frame."); },
														FChaosVDStyle::Get().GetBrush("SceneQueriesInspectorIcon"),
														Context, MoveTemp(OnClickedDelegate));

		InSection.AddEntry(
			FToolMenuEntry::InitWidget(
				"SceneQueryBrowser",
				ConnectToSessionButton,
				FText::GetEmpty(),
				true,
				false
			));
	}));
}

void SChaosVDMainTab::AddSettingsMenu()
{
	UToolMenu* ToolBar = UToolMenus::Get()->ExtendMenu(MainToolBarName);
	if (!ToolBar)
	{
		return;
	}

	using namespace Chaos::VisualDebugger::Utils;

	FToolMenuSection& Section = ToolBar->AddSection("SettingsMenu");

	FNewToolMenuDelegate MainSettingsMenuBuilder = FNewToolMenuDelegate::CreateStatic([](UToolMenu* Menu)
	{
		FToolMenuSection& GeneralSection = Menu->AddSection("GeneralSettingsMenu", LOCTEXT("CommonSettingsMenuLabel", "General"));
		constexpr bool bOpenSubMenuOnClick = false;
		FNewToolMenuDelegate MainCommonSettingsMenuBuilder = FNewToolMenuDelegate::CreateStatic(&CreateMenuEntryForSettingsObject<UChaosVDGeneralSettings>, EChaosVDSaveSettingsOptions::ShowResetButton);
		GeneralSection.AddSubMenu(TEXT("MainCommonSettingsMenu"), LOCTEXT("MainCommonSettingsMenuLabel", "Common"), LOCTEXT("MainCommonSettingsMenuTip", "Common Settings that controls general behavior of CVD"), MainCommonSettingsMenuBuilder, bOpenSubMenuOnClick,  FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbar.Settings")));

		FNewToolMenuDelegate LayoutSettingsMenuBuilder = FNewToolMenuDelegate::CreateStatic([](UToolMenu* Menu)
		{
			const UChaosVDMainToolbarMenuContext* Context = Menu ? Menu->FindContext<UChaosVDMainToolbarMenuContext>() : nullptr;
			TSharedPtr<SChaosVDMainTab> MainTabPtr = Context ? Context->MainTab.Pin() : nullptr;
			if (!MainTabPtr)
			{
				return;
			}

			FToolMenuEntry ResetMenuEntry = FToolMenuEntry::InitMenuEntry(FName("ResetLayoutMenu"),
				LOCTEXT("ResetLayoutMenuEntryLabel", "Reset Layout"),
				LOCTEXT("ResetLayoutMenuEntryLabelToolTip", "Reset the current layout to the defaults one"),
				FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Mainframe.LoadLayout")),
				FUIAction(FExecuteAction::CreateSP(MainTabPtr.ToSharedRef(), &SChaosVDMainTab::ResetLayout)));

			Menu->AddMenuEntry(NAME_None, ResetMenuEntry);
		});
		
		FToolMenuSection& AppearanceSection = Menu->AddSection("AppearanceSectionMenu", LOCTEXT("AppearanceSectionMenuLabel", "Appearance"));
		AppearanceSection.AddSubMenu(TEXT("MainLayoutSettingsMenu"), LOCTEXT("MainLayoutSettingsMenuLabel", "Layout"), LOCTEXT("MainLayoutSettingsMenuTip", "Set of options to alter CVD's UI layout"), LayoutSettingsMenuBuilder, bOpenSubMenuOnClick,  FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Layout")));
	});
	
	constexpr bool bOpenSubMenuOnClick = true;
	Section.AddSubMenu(TEXT("MainSettingsMenu"), LOCTEXT("MainSettingsMenuLabel", "Settings"), LOCTEXT("MainSettingsMenuTip", "Settings that controls general behavior of CVD"), MainSettingsMenuBuilder, bOpenSubMenuOnClick,  FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbar.Settings")));
}

void SChaosVDMainTab::BrowseLiveSessionsFromTraceStore() const
{
	const TSharedRef<SChaosVDBrowseSessionsModal> SessionBrowserModal = SNew(SChaosVDBrowseSessionsModal);

	if (SessionBrowserModal->ShowModal() != EAppReturnType::Cancel)
	{
		bool bSuccess = false;
		const FChaosVDTraceSessionInfo SessionInfo = SessionBrowserModal->GetSelectedTraceInfo();
		if (SessionInfo.bIsValid)
		{
			const FString SessionAddress = SessionBrowserModal->GetSelectedTraceStoreAddress();
			const EChaosVDLoadRecordedDataMode ConnectionMode = SessionBrowserModal->GetSelectedConnectionMode();

			bSuccess = ConnectToLiveSession(SessionInfo.TraceID, SessionAddress, ConnectionMode);
		}

		if (!bSuccess)
		{
			FMessageDialog::Open(EAppMsgType::Ok, LOCTEXT("FailedToConnectToSessionMessage", "Failed to connect to session"));	
		}
	}
}

bool SChaosVDMainTab::ShouldDisableCPUThrottling() const
{
	// If we are playing a live session, it is likely the editor will be in the background, so we need to disable CPU Throttling
	return ChaosVDEngine && ChaosVDEngine->HasAnyLiveSessionActive();
}

FReply SChaosVDMainTab::HandleSessionConnectionClicked()
{
	BrowseLiveSessionsFromTraceStore();
	return FReply::Handled();
}

void SChaosVDMainTab::HandleDisconnectSessionClicked()
{
	if (TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr =  GetChaosVDEngineInstance()->GetPlaybackController())
	{
		const bool bIsAlreadyInLiveSession = PlaybackControllerPtr->IsPlayingLiveSession();
	
		if (bIsAlreadyInLiveSession)
		{
			using namespace UE::TraceBasedDebuggers;
			const TSharedPtr<FTraceSessionsManager> SessionsManager = FChaosVDModule::Get().GetTraceSessionsManager();
			const TArrayView<FTraceSessionDescriptor> ActiveSessions = GetChaosVDEngineInstance()->GetTraceSessionDescriptors();
			for (FTraceSessionDescriptor& ActiveSession : ActiveSessions)
			{
				SessionsManager->CloseSession(ActiveSession.SessionName);
				ActiveSession.bIsLiveSession = false;
			}

			PlaybackControllerPtr->HandleDisconnectedFromSession();
		}
	}
}

FText SChaosVDMainTab::GetDisconnectButtonText() const
{
	if (GetChaosVDEngineInstance()->HasAnyLiveSessionActive())
	{
		if (GetChaosVDEngineInstance()->GetTraceSessionDescriptors().Num() > 1)
		{
			return LOCTEXT("DisconnectFromMultipleSessions", "Disconnect from all Sessions");
		}
	}

	return LOCTEXT("DisconnectFromSession", "Disconnect from Session");
}

#undef LOCTEXT_NAMESPACE
