// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDNameListPicker.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SBoxPanel.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosVDNameListPicker::Construct(const FArguments& InArgs)
{
	NameSelectedDelegate = InArgs._OnNameSleceted;

	// Dropdown Button
	 SAssignNew(DropDownButtonLabelTextBox, STextBlock)
		.Font(FAppStyle::GetFontStyle(TEXT("PropertyWindow.NormalFont")))
		.Text(this, &SChaosVDNameListPicker::GetCurrentDropDownButtonLabel);

	// Name List
	SAssignNew(NameListWidget, SListView<TSharedPtr<FName>>)
		.ListItemsSource(&CachedNameList)
		.SelectionMode( ESelectionMode::Single )
		.OnMouseButtonClick(this, &SChaosVDNameListPicker::OnClickItem)
		.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
		.OnGenerateRow(this, &SChaosVDNameListPicker::MakeNameItemForList);

	// Dropdown List content
	SAssignNew(NamesListDropdown, SListViewSelectorDropdownMenu< TSharedPtr<FName> >, nullptr, NameListWidget)
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::GetBrush("Menu.Background"))
		.Padding(2)
		[
			SNew(SBox)
			[				
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.MaxHeight(200.0f)
				[
					NameListWidget.ToSharedRef()
				]
			]
		]
	];
	
	ChildSlot
	[
		// Combo button that summons the dropdown menu
		SAssignNew(PickerComboButton, SComboButton)
			.IsEnabled_Raw(this, &SChaosVDNameListPicker::HasElements)
			.ButtonContent()
			[
				DropDownButtonLabelTextBox.ToSharedRef()
			]
			.MenuContent()
			[
				NamesListDropdown.ToSharedRef()
			]
			.IsFocusable(true)
			.ContentPadding(2.0f)
			.OnComboBoxOpened(this, &SChaosVDNameListPicker::OnListOpened)
	];
}

void SChaosVDNameListPicker::UpdateNameList(TArray<TSharedPtr<FName>>&& NewNameList)
{
	if (NewNameList.IsEmpty() && CachedNameList.IsEmpty())
	{
		return;
	}

	// TODO: It might be worth to check is the array is different before doing this, but it might be more expensive
	CachedNameList = MoveTemp(NewNameList);

	if (!HasElements())
	{
		CurrentSelectedName = nullptr;
	}

	NameListWidget->RebuildList();
}

void SChaosVDNameListPicker::SelectName(const TSharedPtr<FName>& NameToSelect, ESelectInfo::Type SelectionInfo)
{
	NameListWidget->SetSelection(NameToSelect, ESelectInfo::OnMouseClick);
	NameListWidget->RequestListRefresh();
	OnClickItem(NameToSelect);
}

bool SChaosVDNameListPicker::HasElements() const
{
	return CachedNameList.Num() > 0;
}

FText SChaosVDNameListPicker::GetCurrentDropDownButtonLabel() const
{
	if (CurrentSelectedName.IsValid())
	{
		return FText::FromName(*CurrentSelectedName);
	}
	else
	{
		return HasElements() ? LOCTEXT("NameList", "Pick an element...") : LOCTEXT("EmptyList", "Empty...");
	}
}

void SChaosVDNameListPicker::OnClickItem(TSharedPtr<FName> Item)
{
	CurrentSelectedName = Item;

	if (PickerComboButton.IsValid())
	{
		PickerComboButton->SetIsOpen(false);
	}
	
	NameSelectedDelegate.ExecuteIfBound(Item);
}

void SChaosVDNameListPicker::OnListOpened()
{
	if (NameListWidget.IsValid())
	{
		NameListWidget->SetSelection(CurrentSelectedName, ESelectInfo::OnKeyPress);
		NameListWidget->RequestScrollIntoView(CurrentSelectedName);
	}
}

TSharedRef<ITableRow> SChaosVDNameListPicker::MakeNameItemForList(TSharedPtr<FName> Name, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<STableRow<TSharedPtr<FString>>> ListTableRow;

	SAssignNew(ListTableRow, STableRow<TSharedPtr<FString>>, OwnerTable)
		[
			SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(20.0f, 1.0f, 20.0f, 1.0f)
				[
					SNew(STextBlock)
					.Text(FText::FromName(Name.IsValid() ? *Name.Get() : FName("Invalid")))
				]
		];

	return ListTableRow.ToSharedRef();
}

#undef LOCTEXT_NAMESPACE
