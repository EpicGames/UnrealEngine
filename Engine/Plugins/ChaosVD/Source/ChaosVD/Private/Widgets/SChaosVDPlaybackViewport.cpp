// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDPlaybackViewport.h"

#include "ChaosVDCommands.h"
#include "ChaosVDEditorMode.h"
#include "ChaosVDEngine.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDPlaybackViewportClient.h"
#include "ChaosVDScene.h"
#include "ChaosVDSceneParticle.h"
#include "EditorModeManager.h"
#include "EditorViewportCommands.h"
#include "SChaosVDGameFramesPlaybackControls.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "ChaosVisualDebugger/ChaosVDDataWrapperUtils.h"
#include "Elements/Actor/ActorElementData.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Framework/Application/SlateApplication.h"
#include "TEDS/ChaosVDSelectionInterface.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "Utils/ChaosVDMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/SChaosVDTimelineWidget.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Widgets/Text/STextBlock.h"
#include "ToolMenus.h"
#include "SStreamingStatus.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

namespace Chaos::VisualDebugger::Cvars
{
	static bool bBroadcastGameFrameUpdateEvenIfNotChanged = false;
	static FAutoConsoleVariableRef CVarChaosVDBroadcastGameFrameUpdateEvenIfNotChanged(
		TEXT("p.Chaos.VD.Tool.BroadcastGameFrameUpdateEvenIfNotChanged"),
		bBroadcastGameFrameUpdateEvenIfNotChanged,
		TEXT("If true, each time we get a controller data updated event, a game frame update will be triggered even if the frame didn't change..."));
}

SChaosVDPlaybackViewport::~SChaosVDPlaybackViewport()
{
	if (ExternalInvalidateHandlerHandle.IsValid())
	{
		ExternalViewportInvalidationRequestHandler.Remove(ExternalInvalidateHandlerHandle);
		ExternalInvalidateHandlerHandle.Reset();
	}

	UnbindFromSceneUpdateEvents();

	PlaybackViewportClient.Reset();
}

void SChaosVDPlaybackViewport::Construct(const FArguments& InArgs, TWeakPtr<FChaosVDScene> InScene, TWeakPtr<FChaosVDPlaybackController> InPlaybackController, TSharedPtr<FEditorModeTools> InEditorModeTools)
{
	Extender = MakeShared<FExtender>();

	EditorModeTools = InEditorModeTools;
	EditorModeTools->SetWidgetMode(UE::Widget::WM_Translate);
	EditorModeTools->SetDefaultMode(UChaosVDEditorMode::EM_ChaosVisualDebugger);
	EditorModeTools->ActivateDefaultMode();

	SEditorViewport::Construct(SEditorViewport::FArguments());

	CVDSceneWeakPtr = InScene;
	TSharedPtr<FChaosVDScene> ScenePtr = InScene.Pin();
	ensure(ScenePtr.IsValid());
	ensure(InPlaybackController.IsValid());

	PlaybackViewportClient = StaticCastSharedPtr<FChaosVDPlaybackViewportClient>(GetViewportClient());

	// TODO: Add a way to gracefully shutdown (close) the tool when a no recoverable situation like this happens (UE-191876)
	check(PlaybackViewportClient.IsValid());

	PlaybackViewportClient->SetScene(InScene);

	TAttribute<EVisibility> KeyShortcutVisibilityAttribute;
	KeyShortcutVisibilityAttribute.BindLambda([WeakThis = AsWeak()]()
	{
		if (TSharedPtr<SChaosVDPlaybackViewport> PlaybackViewportWidget = StaticCastSharedPtr<SChaosVDPlaybackViewport>(WeakThis.Pin()))
		{
			return PlaybackViewportWidget->GetTrackSelectorKeyVisibility();
		}

		return EVisibility::Collapsed;
	});

	TAttribute<ECheckBoxState> GameTrackIsActiveAttribute;
	GameTrackIsActiveAttribute.BindLambda([WeakThis = AsWeak(), WeakPlaybackController = InPlaybackController]()
	{
		TSharedPtr<SChaosVDPlaybackViewport> PlaybackViewportWidget = StaticCastSharedPtr<SChaosVDPlaybackViewport>(WeakThis.Pin());
		TSharedPtr<FChaosVDPlaybackController> PlaybackController = StaticCastSharedPtr<FChaosVDPlaybackController>(WeakPlaybackController.Pin());
		if (!PlaybackViewportWidget || !PlaybackController)
		{
			return ECheckBoxState::Undetermined;
		}
		
		return PlaybackController->GetActiveTrackInfo()->TrackType == EChaosVDTrackType::Game ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	});
	
	if (UChaosVDEditorMode* CVDEdMode = Cast<UChaosVDEditorMode>(EditorModeTools->GetActiveScriptableMode(UChaosVDEditorMode::EM_ChaosVisualDebugger)))
	{
		if (ScenePtr.IsValid())
		{
			CVDEdMode->SetWorld(ScenePtr->GetUnderlyingWorld());
		}
	}
	
	TSharedPtr<SWidget> ViewportToolbar = BuildViewportToolbar();
	
	ChildSlot
	[
		// 3D Viewport
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			ViewportToolbar.IsValid() ? ViewportToolbar.ToSharedRef() : SNullWidget::NullWidget
		]
		+SVerticalBox::Slot()
		.FillHeight(0.9f)
		[
			ViewportWidget.ToSharedRef()
		]
		// Playback controls
		// TODO: Now that the tool is In-Editor, see if we can/is worth use the Sequencer widgets
		// instead of these custom ones
		+SVerticalBox::Slot()
		.Padding(16.0f, 16.0f, 16.0f, 16.0f)
		.FillHeight(0.1f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.HAlign(HAlign_Center)
			.Padding(0.0f, 0.0f, 0.0f, 2.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(2.0f, 0.0f, 8.0f, 0.0f)
				[
					SNew(SCheckBox)
					.IsEnabled(false)
					.Style(FAppStyle::Get(), "Menu.RadioButton")
					.IsChecked(GameTrackIsActiveAttribute)
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.f, 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(LOCTEXT("PlaybackViewportWidgetGameFramesLabel", "Game Frames" ))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(2.0f, 0.0f, 2.0f, 0.0f)
				[
					SNew(SSeparator)
					.Visibility(KeyShortcutVisibilityAttribute)
					.Orientation(Orient_Vertical)
					.Thickness(1.f)
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				.AutoWidth()
				.Padding(0.f, 0.f, 0.f, 0.f)
				[
					SNew(STextBlock)
					.Visibility(KeyShortcutVisibilityAttribute)
					.Text(FText::AsCultureInvariant(TEXT("CTRL + 0")))
				]
			]
			+SVerticalBox::Slot()
			[
				SAssignNew(GameFramesPlaybackControls, SChaosVDGameFramesPlaybackControls, InPlaybackController)
			]
		]
	];

	ExternalInvalidateHandlerHandle = ExternalViewportInvalidationRequestHandler.AddSP(this, &SChaosVDPlaybackViewport::HandleExternalViewportInvalidateRequest);

	RegisterNewController(InPlaybackController);
}

TSharedRef<SEditorViewport> SChaosVDPlaybackViewport::GetViewportWidget()
{
	return StaticCastSharedRef<SEditorViewport>(AsShared());
}

TSharedPtr<FExtender> SChaosVDPlaybackViewport::GetExtenders() const
{
	return Extender;
}

void SChaosVDPlaybackViewport::BindGlobalUICommands()
{
	const FChaosVDCommands& Commands = FChaosVDCommands::Get();

	TSharedPtr<SChaosVDMainTab> CVDToolkitHost = StaticCastSharedPtr<SChaosVDMainTab>(EditorModeTools->GetToolkitHost());
	TSharedPtr<FUICommandList> GlobalUICommandsList = CVDToolkitHost ? CVDToolkitHost->GetGlobalUICommandList() : nullptr;

	if (!GlobalUICommandsList)
	{
		return;
	}

	FUIAction PlayPausePlaybackAction;
	PlayPausePlaybackAction.ExecuteAction.BindSPLambda(SharedThis(this), [WeakThis = AsWeak()]()
	{
		if (TSharedPtr<SChaosVDPlaybackViewport> PlaybackViewportWidget = StaticCastSharedPtr<SChaosVDPlaybackViewport>(WeakThis.Pin()))
		{
			if (TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackViewportWidget->PlaybackController.Pin())
			{
				if (PlaybackControllerPtr->GetActiveTrackInfo()->bIsPlaying)
				{
					PlaybackViewportWidget->HandleFramePlaybackControlInput(EChaosVDPlaybackButtonsID::Pause);
				}
				else
				{
					PlaybackViewportWidget->HandleFramePlaybackControlInput(EChaosVDPlaybackButtonsID::Play);
				}
			}
		}
	});

	GlobalUICommandsList->MapAction(Commands.PlayPauseTrack, PlayPausePlaybackAction);

	FUIAction StopPlaybackAction;
	StopPlaybackAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::HandleFramePlaybackControlInput, EChaosVDPlaybackButtonsID::Stop);
	GlobalUICommandsList->MapAction(Commands.StopTrack, StopPlaybackAction);

	FUIAction NextFramePlaybackAction;
	NextFramePlaybackAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::HandleFramePlaybackControlInput, EChaosVDPlaybackButtonsID::Prev);
	GlobalUICommandsList->MapAction(Commands.PrevFrame, NextFramePlaybackAction);

	FUIAction PrevFramePlaybackAction;
	PrevFramePlaybackAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::HandleFramePlaybackControlInput, EChaosVDPlaybackButtonsID::Next);
	GlobalUICommandsList->MapAction(Commands.NextFrame, PrevFramePlaybackAction);

	FUIAction NexStagePlaybackAction;
	NexStagePlaybackAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::HandleFrameStagePlaybackControlInput, EChaosVDPlaybackButtonsID::Prev);
	GlobalUICommandsList->MapAction(Commands.PrevStage, NexStagePlaybackAction);

	FUIAction PrevStagePlaybackAction;
	PrevStagePlaybackAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::HandleFrameStagePlaybackControlInput, EChaosVDPlaybackButtonsID::Next);
	GlobalUICommandsList->MapAction(Commands.NextStage, PrevStagePlaybackAction);

	FUIAction DeselectAllAction;
	DeselectAllAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::DeselectAll);
	GlobalUICommandsList->MapAction(Commands.DeselectAll, DeselectAllAction);

	FUIAction HideSelectedAction;
	HideSelectedAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::HideSelected);
	GlobalUICommandsList->MapAction(Commands.HideSelected, HideSelectedAction);

	FUIAction ShowAllAction;
	ShowAllAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::ShowAll);
	GlobalUICommandsList->MapAction(Commands.ShowAll, ShowAllAction);
}

void SChaosVDPlaybackViewport::UnBindEditorViewportUnsupportedCommands()
{
	const FEditorViewportCommands& DefaultViewportCommands = FEditorViewportCommands::Get();

	if (!CommandList)
	{
		return;
	}

	CommandList->UnmapAction(DefaultViewportCommands.ToggleRealTime);
	CommandList->UnmapAction(DefaultViewportCommands.ToggleStats);
	CommandList->UnmapAction(DefaultViewportCommands.ToggleFPS);
	CommandList->UnmapAction(DefaultViewportCommands.ScreenCaptureForProjectThumbnail);
	CommandList->UnmapAction(DefaultViewportCommands.RelativeCoordinateSystem_World);
	CommandList->UnmapAction(DefaultViewportCommands.RelativeCoordinateSystem_Local);
	CommandList->UnmapAction(DefaultViewportCommands.CycleTransformGizmoCoordSystem);
	CommandList->UnmapAction(DefaultViewportCommands.ToggleInGameExposure);
	CommandList->UnmapAction(DefaultViewportCommands.ToggleAutoExposure);
	CommandList->UnmapAction(DefaultViewportCommands.ToggleInViewportContextMenu);
	CommandList->UnmapAction(DefaultViewportCommands.ToggleOverrideViewportScreenPercentage);
	CommandList->UnmapAction(DefaultViewportCommands.OpenEditorPerformanceProjectSettings);
	CommandList->UnmapAction(DefaultViewportCommands.OpenEditorPerformanceEditorPreferences);
	CommandList->UnmapAction(DefaultViewportCommands.DetailLightingMode);
	CommandList->UnmapAction(DefaultViewportCommands.LightingOnlyMode);
	CommandList->UnmapAction(DefaultViewportCommands.LightComplexityMode);
	CommandList->UnmapAction(DefaultViewportCommands.ShaderComplexityMode);
	CommandList->UnmapAction(DefaultViewportCommands.QuadOverdrawMode);
	CommandList->UnmapAction(DefaultViewportCommands.LightmapDensityMode);
}

void SChaosVDPlaybackViewport::BindCommands()
{
	SEditorViewport::BindCommands();

	UnBindEditorViewportUnsupportedCommands();

	const FChaosVDCommands& Commands = FChaosVDCommands::Get();

	if (ensure(Client))
	{
		const TSharedRef<FChaosVDPlaybackViewportClient> ViewportClientRef = StaticCastSharedRef<FChaosVDPlaybackViewportClient>(Client.ToSharedRef());
		FUIAction ToggleObjectTrackingAction;
		ToggleObjectTrackingAction.ExecuteAction.BindSP(ViewportClientRef, &FChaosVDPlaybackViewportClient::ToggleObjectTrackingIfSelected);
		ToggleObjectTrackingAction.GetActionCheckState.BindLambda([WeakViewportClient = ViewportClientRef.ToWeakPtr()]()
		{
			TSharedPtr<const FChaosVDPlaybackViewportClient> ViewportPtr = WeakViewportClient.Pin();
			return ViewportPtr.IsValid() && ViewportPtr->IsAutoTrackingSelectedObject() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
		CommandList->MapAction(Commands.ToggleFollowSelectedObject, ToggleObjectTrackingAction);

		FUIAction ToggleOverrideFrameRateAction;
		ToggleOverrideFrameRateAction.ExecuteAction.BindSP(SharedThis(this), &SChaosVDPlaybackViewport::ToggleUseFrameRateOverride);
		ToggleOverrideFrameRateAction.GetActionCheckState.BindLambda([WeakThis = AsWeak()]()
		{
			TSharedPtr<const SChaosVDPlaybackViewport> ViewportPtr = StaticCastSharedPtr<const SChaosVDPlaybackViewport>(WeakThis.Pin());
			return ViewportPtr.IsValid() && ViewportPtr->IsUsingFrameRateOverride() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
		CommandList->MapAction(Commands.OverridePlaybackFrameRate, ToggleOverrideFrameRateAction);

		FUIAction ToggleTranslucentGeometrySelectionAction;
		ToggleTranslucentGeometrySelectionAction.ExecuteAction.BindSP(ViewportClientRef, &FChaosVDPlaybackViewportClient::ToggleCanSelectTranslucentGeometry);
		ToggleTranslucentGeometrySelectionAction.GetActionCheckState.BindLambda([WeakViewportClient = ViewportClientRef.ToWeakPtr()]()
		{
			TSharedPtr<const FChaosVDPlaybackViewportClient> ViewportPtr = WeakViewportClient.Pin();
			return ViewportPtr.IsValid() && ViewportPtr->GetCanSelectTranslucentGeometry() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		});
		CommandList->MapAction(Commands.AllowTranslucentSelection, ToggleTranslucentGeometrySelectionAction);
	}

	BindGlobalUICommands();
}

EVisibility SChaosVDPlaybackViewport::GetTransformToolbarVisibility() const
{
	// We want to always show the transform tool bar. We disable each action that is not supported for a selected actor individually.
	// Without doing this, if you select an unsupported mode, the entire toolbar disappears
	return EVisibility::Visible;
}

void SChaosVDPlaybackViewport::GoToLocation(const FVector& InLocation) const
{
	if (PlaybackViewportClient)
	{
		PlaybackViewportClient->GoToLocation(InLocation);
	}
}

void SChaosVDPlaybackViewport::ToggleUseFrameRateOverride()
{
	if (TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->ToggleUseFrameRateOverride();
	}
}

bool SChaosVDPlaybackViewport::IsUsingFrameRateOverride() const
{
	if (TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		return PlaybackControllerPtr->IsUsingFrameRateOverride();
	}

	return false;
}

int32 SChaosVDPlaybackViewport::GetCurrentTargetFrameRateOverride() const
{
	TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin();
	return PlaybackControllerPtr ? PlaybackControllerPtr->GetFrameRateOverride() : FChaosVDPlaybackController::InvalidFrameRateOverride;
}

void SChaosVDPlaybackViewport::SetCurrentTargetFrameRateOverride(int32 NewTarget)
{
	if (TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->SetFrameRateOverride(static_cast<float>(NewTarget));
	}
}

void SChaosVDPlaybackViewport::ExecuteExternalViewportInvalidateRequest()
{
	ExternalViewportInvalidationRequestHandler.Broadcast();
}

void SChaosVDPlaybackViewport::OnFocusViewportToSelection()
{
	PlaybackViewportClient->FocusOnSelectedObject();
}

void SChaosVDPlaybackViewport::AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder)
{
	if (ViewportOverlay)
	{
		ViewportOverlay->AddSlot(ZOrder)
		[
			InOverlaidWidget
		];
	}
}

void SChaosVDPlaybackViewport::PopulateViewportOverlays(TSharedRef<SOverlay> Overlay)
{
	SEditorViewport::PopulateViewportOverlays(Overlay);

	Overlay->AddSlot(0)
	[
		SNew(Chaos::VD::SStreamingStatus)
	];
}

FReply SChaosVDPlaybackViewport::OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<SChaosVDMainTab> AsCVDToolkitHost = StaticCastSharedPtr<SChaosVDMainTab>(EditorModeTools->GetToolkitHost());
	if (!AsCVDToolkitHost)
	{
		return SEditorViewport::OnDragOver(MyGeometry, DragDropEvent);
	}
	
	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (const bool bValidOperation = Operation.IsValid() && Operation->IsOfType<FExternalDragOperation>())
	{
		const TSharedPtr<FExternalDragOperation> AsExternalDragOperation = StaticCastSharedPtr<FExternalDragOperation>(Operation);
		if (AsExternalDragOperation->HasFiles())
		{
			for (const FString& DraggedFile : AsExternalDragOperation->GetFiles())
			{
				if (!AsCVDToolkitHost->IsSupportedFile(DraggedFile))
				{
					return SEditorViewport::OnDragOver(MyGeometry, DragDropEvent);
				}
			}

			return FReply::Handled();
		}
	}

	return SEditorViewport::OnDragOver(MyGeometry, DragDropEvent);
}

FReply SChaosVDPlaybackViewport::OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent)
{
	TSharedPtr<SChaosVDMainTab> AsCVDToolkitHost = StaticCastSharedPtr<SChaosVDMainTab>(EditorModeTools->GetToolkitHost());
	if (!AsCVDToolkitHost)
	{
		return SEditorViewport::OnDrop(MyGeometry, DragDropEvent);
	}

	TSharedPtr<FDragDropOperation> Operation = DragDropEvent.GetOperation();
	if (const bool bValidOperation = Operation.IsValid() && Operation->IsOfType<FExternalDragOperation>())
	{
		const TSharedPtr<FExternalDragOperation> AsExternalDragOperation = StaticCastSharedPtr<FExternalDragOperation>(Operation);
		if (AsExternalDragOperation->HasFiles())
		{
			AsCVDToolkitHost->LoadCVDFiles(AsExternalDragOperation->GetFiles(), EChaosVDLoadRecordedDataMode::SingleSource);
			return FReply::Handled();
		}
	}

	return SEditorViewport::OnDrop(MyGeometry, DragDropEvent);
}

EVisibility SChaosVDPlaybackViewport::GetTrackSelectorKeyVisibility() const
{
	if (TSharedPtr<SChaosVDMainTab> CVDToolkitHost = StaticCastSharedPtr<SChaosVDMainTab>(EditorModeTools->GetToolkitHost()))
	{
		return CVDToolkitHost->ShouldShowTracksKeyShortcuts() ? EVisibility::Visible : EVisibility::Collapsed;
	}

	return EVisibility::Collapsed;
}

TSharedRef<FEditorViewportClient> SChaosVDPlaybackViewport::MakeEditorViewportClient()
{
	TSharedPtr<FChaosVDPlaybackViewportClient> NewViewport = MakeShared<FChaosVDPlaybackViewportClient>(EditorModeTools, GetViewportWidget());

	NewViewport->SetAllowCinematicControl(false);
	
	NewViewport->bSetListenerPosition = false;
	NewViewport->EngineShowFlags = FEngineShowFlags(ESFIM_Editor);
	NewViewport->LastEngineShowFlags = FEngineShowFlags(ESFIM_Editor);
	NewViewport->ViewportType = LVT_Perspective;
	NewViewport->bDrawAxes = true;
	NewViewport->bDisableInput = false;
	NewViewport->VisibilityDelegate.BindLambda([] {return true; });

	NewViewport->EngineShowFlags.DisableAdvancedFeatures();
	NewViewport->EngineShowFlags.SetSelectionOutline(true);
	NewViewport->EngineShowFlags.SetSnap(false);
	NewViewport->EngineShowFlags.SetBillboardSprites(true);

	return StaticCastSharedRef<FEditorViewportClient>(NewViewport.ToSharedRef());
}

TSharedPtr<SWidget> SChaosVDPlaybackViewport::BuildViewportToolbar()
{
	const FName ToolbarName = "ChaosVDViewportToolbarBase";
	
	if (!UToolMenus::Get()->IsMenuRegistered(ToolbarName))
	{
		UToolMenu* Toolbar = UToolMenus::Get()->RegisterMenu(ToolbarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
		Toolbar->StyleName = "ViewportToolbar";
		
		{
			FToolMenuSection& LeftSection = Toolbar->AddSection("Left");
			LeftSection.AddEntry(Chaos::VisualDebugger::Menus::CreateSettingsSubmenu());
		}
		
		{
			FToolMenuSection& RightSection = Toolbar->AddSection("Right");
			RightSection.Alignment = EToolMenuSectionAlign::Last;
			
			RightSection.AddEntry(UE::UnrealEd::CreateCameraSubmenu(UE::UnrealEd::FViewportCameraMenuOptions().ShowAll()));
			RightSection.AddEntry(UE::UnrealEd::CreateViewModesSubmenu());
			RightSection.AddEntry(Chaos::VisualDebugger::Menus::CreateShowSubmenu());
		}
	}

	FToolMenuContext Context;
	{
		UUnrealEdViewportToolbarContext* ViewportContext = UE::UnrealEd::CreateViewportToolbarDefaultContext(SharedThis(this));
		ViewportContext->IsViewModeSupported.BindLambda([](EViewModeIndex ViewMode)
		{
			switch (ViewMode)
			{
			case VMI_Lit:
			case VMI_Unlit:
			case VMI_BrushWireframe:
				return true;
			default:
				return false;
			}
		});
		
		Context.AddObject(ViewportContext);
		Context.AppendCommandList(GetCommandList());
	}

	return UToolMenus::Get()->GenerateWidget(ToolbarName, Context);
}

void SChaosVDPlaybackViewport::HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InController)
{
	if (PlaybackController != InController)
	{
		RegisterNewController(InController);
	}

	PlaybackViewportClient->bNeedsRedraw = true;
}

void SChaosVDPlaybackViewport::HandlePostSelectionChange(const UTypedElementSelectionSet* ChangesSelectionSet)
{
	PlaybackViewportClient->bNeedsRedraw = true;
}

void SChaosVDPlaybackViewport::HandleFramePlaybackControlInput(EChaosVDPlaybackButtonsID ButtonID)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->HandleFramePlaybackControlInput(ButtonID, PlaybackControllerPtr->GetActiveTrackInfo(), GetInstigatorID());
	}
}

void SChaosVDPlaybackViewport::HandleFrameStagePlaybackControlInput(EChaosVDPlaybackButtonsID ButtonID)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		if (PlaybackControllerPtr->GetActiveTrackInfo()->TrackType == EChaosVDTrackType::Solver)
		{
			PlaybackControllerPtr->HandleFrameStagePlaybackControlInput(ButtonID, PlaybackControllerPtr->GetActiveTrackInfo(), GetInstigatorID());
		}	
	}
}

void SChaosVDPlaybackViewport::DeselectAll() const
{
	const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin();
	const TSharedPtr<FChaosVDScene> ScenePtr = PlaybackControllerPtr ? PlaybackControllerPtr->GetControllerScene().Pin() : nullptr;
	if (!ScenePtr)
	{
		return;
	}

	if (UTypedElementSelectionSet* SelectionSet = ScenePtr->GetElementSelectionSet())
	{
		SelectionSet->ClearSelection(FTypedElementSelectionOptions());
	}
	
	if (const TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelection = ScenePtr->GetSolverDataSelectionObject().Pin())
	{
		SolverDataSelection->SelectData(nullptr);
	}
}

void SChaosVDPlaybackViewport::HideSelected() const
{
	if (UTypedElementSelectionSet* SelectionSet = EditorModeTools->GetEditorSelectionSet())
	{
		//TODO: Update this if we add multi selection support
		constexpr int32 MaxElements = 1;
		TArray<FTypedElementHandle, TInlineAllocator<MaxElements>> TypedElementHandles;
		SelectionSet->GetSelectedElementHandles(TypedElementHandles, UChaosVDSelectionInterface::StaticClass());

		if (TypedElementHandles.Num() > 0)
		{
			FTypedElementHandle& SelectionHandle = TypedElementHandles[0];
			using namespace Chaos::VD::TypedElementDataUtil;
			if (FChaosVDSceneParticle* Particle = GetStructDataFromTypedElementHandle<FChaosVDSceneParticle>(SelectionHandle))
			{
				Particle->HideImmediate(EChaosVDHideParticleFlags::HiddenBySceneOutliner);
			}
			else if (AChaosVDSolverInfoActor* SolverInfoActor = Cast<AChaosVDSolverInfoActor>(ActorElementDataUtil::GetActorFromHandle(SelectionHandle)))
			{
				SolverInfoActor->SetIsTemporarilyHiddenInEditor(true);
			}
		}
	}
}

void SChaosVDPlaybackViewport::ShowAll() const
{
	const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin();
	const TSharedPtr<FChaosVDScene> ScenePtr = PlaybackControllerPtr ? PlaybackControllerPtr->GetControllerScene().Pin() : nullptr;
	if (!ScenePtr)
	{
		return;
	}

	const FChaosVDSolverInfoByIDMap& SolverInfoActorsByID = ScenePtr->GetSolverInfoActorsMap();
	for (const TPair<int32, AChaosVDSolverInfoActor*>& SolverInfoWithID : SolverInfoActorsByID)
	{
		if (SolverInfoWithID.Value)
		{
			// Note : Depending on the scene, this will be slow. If that is the case, we should expose the Batch Visibility update for particles
			// in a way other systems can use it.
			// Currently, we process the data changes and apply them to the mesh components at the end of the frame, and de-duplicate any operations
			// that nullify themselves and the final visibility state doesn't actually change, therefore the perf hit of doing this is mostly mitigated
			SolverInfoWithID.Value->SetIsTemporarilyHiddenInEditor(true);
			SolverInfoWithID.Value->SetIsTemporarilyHiddenInEditor(false);
		}
	}
}

void SChaosVDPlaybackViewport::OnPlaybackSceneUpdated()
{
	PlaybackViewportClient->HandleCVDSceneUpdated();
}

void SChaosVDPlaybackViewport::OnSolverVisibilityUpdated(int32 SolverID, bool bNewVisibility)
{
	PlaybackViewportClient->HandleCVDSceneUpdated();	
}

void SChaosVDPlaybackViewport::BindToSceneUpdateEvents()
{	
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		if (TSharedPtr<FChaosVDScene> ScenePtr = PlaybackControllerPtr->GetControllerScene().Pin())
		{
			ScenePtr->OnSceneUpdated().AddSP(this, &SChaosVDPlaybackViewport::OnPlaybackSceneUpdated);
			ScenePtr->OnSolverVisibilityUpdated().AddSP(this, &SChaosVDPlaybackViewport::OnSolverVisibilityUpdated);
		}
	}
}

void SChaosVDPlaybackViewport::UnbindFromSceneUpdateEvents()
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		if (TSharedPtr<FChaosVDScene> ScenePtr = PlaybackControllerPtr->GetControllerScene().Pin())
		{
			ScenePtr->OnSceneUpdated().RemoveAll(this);
			ScenePtr->OnSolverVisibilityUpdated().RemoveAll(this);
		}
	}
}

void SChaosVDPlaybackViewport::RegisterNewController(TWeakPtr<FChaosVDPlaybackController> NewController)
{
	if (PlaybackController != NewController)
	{
		UnbindFromSceneUpdateEvents();	

		FChaosVDPlaybackControllerObserver::RegisterNewController(NewController);

		BindToSceneUpdateEvents();	
	}
}

void SChaosVDPlaybackViewport::HandleExternalViewportInvalidateRequest()
{
	if (PlaybackViewportClient)
	{
		PlaybackViewportClient->Invalidate();
	}
}

#undef LOCTEXT_NAMESPACE
