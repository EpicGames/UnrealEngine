// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDCommands.h"
#include "ChaosVDPlaybackControllerInstigator.h"
#include "ChaosVDPlaybackControllerObserver.h"
#include "SCommonEditorViewportToolbarBase.h"
#include "SEditorViewport.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

class SChaosVDGameFramesPlaybackControls;
enum class EChaosVDPlaybackButtonsID : uint8;
class FChaosVDEditorModeTools;
enum class EChaosVDActorTrackingMode;
class FChaosVDPlaybackViewportClient;
class SChaosVDSolverPlaybackControls;
class FChaosVDPlaybackController;
class SChaosVDTimelineWidget;
struct FChaosVDRecording;
class FChaosVDScene;
class FLevelEditorViewportClient;
class FSceneViewport;
class SViewport;

/* Widget that contains the 3D viewport and playback controls */
class SChaosVDPlaybackViewport : public SEditorViewport, public FChaosVDPlaybackControllerObserver, public IChaosVDPlaybackControllerInstigator, public ICommonEditorViewportToolbarInfoProvider
{
public:

	SLATE_BEGIN_ARGS(SChaosVDPlaybackViewport) {}
	SLATE_END_ARGS()

	virtual ~SChaosVDPlaybackViewport() override;

	void Construct(const FArguments& InArgs, TWeakPtr<FChaosVDScene> InScene, TWeakPtr<FChaosVDPlaybackController> InPlaybackController, TSharedPtr<FEditorModeTools> InEditorModeTools);

	// BEING ICommonEditorViewportToolbarInfoProvider interface
	virtual TSharedRef<SEditorViewport> GetViewportWidget() override;
	virtual TSharedPtr<FExtender> GetExtenders() const override;
	virtual void OnFloatingButtonClicked() override {};
	// END ICommonEditorViewportToolbarInfoProvider interface

	virtual void BindCommands() override;

	void BindGlobalUICommands();

	void UnBindEditorViewportUnsupportedCommands();

	virtual EVisibility GetTransformToolbarVisibility() const override;

	void GoToLocation(const FVector& InLocation) const;

	void ToggleUseFrameRateOverride();
	bool IsUsingFrameRateOverride() const;

	int32 GetCurrentTargetFrameRateOverride() const;

	void SetCurrentTargetFrameRateOverride(int32 NewTarget);

	TWeakPtr<FChaosVDScene> GetCVDScene() { return CVDSceneWeakPtr; }
	
	static void ExecuteExternalViewportInvalidateRequest();

	virtual void OnFocusViewportToSelection() override;

	void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder);

protected:

	virtual void PopulateViewportOverlays(TSharedRef<SOverlay> Overlay) override;
	
	virtual FReply OnDragOver(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	virtual FReply OnDrop(const FGeometry& MyGeometry, const FDragDropEvent& DragDropEvent) override;
	
	EVisibility GetTrackSelectorKeyVisibility() const;

	virtual TSharedRef<FEditorViewportClient> MakeEditorViewportClient() override;
	
	virtual TSharedPtr<SWidget> BuildViewportToolbar() override;

	virtual void RegisterNewController(TWeakPtr<FChaosVDPlaybackController> NewController )override;
	virtual void HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InController) override;
	virtual void HandlePostSelectionChange(const UTypedElementSelectionSet* ChangesSelectionSet) override;

	void HandleFramePlaybackControlInput(EChaosVDPlaybackButtonsID ButtonID);
	void HandleFrameStagePlaybackControlInput(EChaosVDPlaybackButtonsID ButtonID);

	void DeselectAll() const;
	void HideSelected() const;
	void ShowAll() const;

	void OnPlaybackSceneUpdated();
	void OnSolverVisibilityUpdated(int32 SolverID, bool bNewVisibility);
	void BindToSceneUpdateEvents();
	void UnbindFromSceneUpdateEvents();

	void HandleExternalViewportInvalidateRequest();

	TSharedPtr<FChaosVDTrackInfo> CurrentGameTrackInfo;

	TSharedPtr<SChaosVDGameFramesPlaybackControls> GameFramesPlaybackControls;

	TSharedPtr<FChaosVDPlaybackViewportClient> PlaybackViewportClient;
	
	TWeakPtr<FChaosVDScene> CVDSceneWeakPtr;

	TSharedPtr<FExtender> Extender;

	TSharedPtr<FEditorModeTools> EditorModeTools;
	
	DECLARE_MULTICAST_DELEGATE(FChaosVDViewportInvalidationRequestHandler)
	static inline FChaosVDViewportInvalidationRequestHandler ExternalViewportInvalidationRequestHandler = FChaosVDViewportInvalidationRequestHandler();
	
	FDelegateHandle ExternalInvalidateHandlerHandle;

	bool bIsPlaying = false;
};
