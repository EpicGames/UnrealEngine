// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDRecordedLogBrowser.h"

#include "ChaosVDEngine.h"
#include "ChaosVDModule.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDScene.h"
#include "Components/VerticalBox.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "SChaosVDLogBrowserToolbar.h"
#include "SChaosVDRecordedLogView.h"
#include "ToolMenus.h"
#include "TraceSessionsManager.h"
#include "TraceServices/Model/AnalysisSession.h"
#include "TraceServices/Model/Log.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SChaosVDRecordedLogBrowser)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

const FName SChaosVDRecordedLogBrowser::ToolBarName = FName(TEXT("ChaosVD.RecordedLogBrowser.ToolBar"));

bool FChaosVDBasicLogFilterExpressionContext::TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const
{
	return TextFilterUtils::TestBasicStringExpression(LogEntry->Message, InValue, InTextComparisonMode) ||
			TextFilterUtils::TestBasicStringExpression(LogEntry->Category, InValue, InTextComparisonMode) ||
			TextFilterUtils::TestBasicStringExpression(::ToString(LogEntry->Verbosity), InValue, InTextComparisonMode);
}

void FChaosVDLogFilteringAsyncTask::DoWork()
{
	if (TSharedPtr<SChaosVDRecordedLogBrowser> LogBrowserPtr = LogBrowser.Pin())
	{
		FWriteScopeLock WriteScopeLock(LogBrowserPtr->SourceDataCacheLock);

		LogBrowserPtr->FilteredCachedLogItems->Reset();

		for (const TPair<FName, SChaosVDRecordedLogBrowser::FCategorizedItemsContainer>& CategorisedItemsContainer : LogBrowserPtr->CachedLogItemsByCategory)
		{
			if (CategorisedItemsContainer.Value.bIsEnabled)
			{
				LogBrowserPtr->ApplyFilterToData_AssumesLocked(CategorisedItemsContainer.Value.Items, LogBrowserPtr->FilteredCachedLogItems.ToSharedRef());
			}
		}
		
		LogBrowserPtr->FilteredCachedLogItems->Sort([](const TSharedPtr<FChaosVDLogViewListItem>& ItemA, const TSharedPtr<FChaosVDLogViewListItem>& ItemB)
		{
			return ItemA && ItemB ? ItemA->EntryIndex < ItemB->EntryIndex : false;
		});

		LogBrowserPtr->SetDirtyFlag(EChaosVDLogBrowserDirtyFlags::Filtering);
	}
}

void SChaosVDRecordedLogBrowser::Construct(const FArguments& InArgs, const TSharedRef<FChaosVDEngine>& InEngineInstance)
{
	EngineInstanceWeakPtr = InEngineInstance;
	
	FilteredCachedLogItems = MakeShared<TArray<TSharedPtr<FChaosVDLogViewListItem>>>();
	
	FilterEvaluator = MakeShared<FTextFilterExpressionEvaluator>(ETextFilterExpressionEvaluatorMode::BasicString);

	RegisterNewController(InEngineInstance->GetPlaybackController());
	
	constexpr float NoPadding = 0.0f;
	constexpr float MainContentBoxHorizontalPadding = 2.0f;
	constexpr float MainContentBoxVerticalPadding = 5.0f;
	constexpr float StatusBarSlotVerticalPadding = 1.0f;
	constexpr float StatusBarInnerVerticalPadding = 9.0f;
	constexpr float StatusBarInnerHorizontalPadding = 14.0f;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(NoPadding)
		[
			SAssignNew(ToolbarPtr, SChaosVDLogBrowserToolbar, StaticCastWeakPtr<SChaosVDRecordedLogBrowser>(AsWeak()))
		]
		+SVerticalBox::Slot()
		.Padding(MainContentBoxHorizontalPadding, MainContentBoxVerticalPadding, MainContentBoxHorizontalPadding, NoPadding)
		.FillHeight(1.0f)
		[
			SAssignNew(LogViewWidget, SChaosVDRecordedLogView)
			.OnItemSelected(this, &SChaosVDRecordedLogBrowser::HandleItemSelected)
			.OnItemFocused(this, &SChaosVDRecordedLogBrowser::HandleItemFocused)
		]
		+SVerticalBox::Slot()
		.Padding(NoPadding, StatusBarSlotVerticalPadding, NoPadding, StatusBarSlotVerticalPadding)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header") )
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(StatusBarInnerHorizontalPadding, StatusBarInnerVerticalPadding))
			[
				SNew(STextBlock)
				.Text(this, &SChaosVDRecordedLogBrowser::GetFilterStatusText)
				.ColorAndOpacity(this, &SChaosVDRecordedLogBrowser::GetFilterStatusTextColor)
			]
		]
	];
}

SChaosVDRecordedLogBrowser::~SChaosVDRecordedLogBrowser()
{
	for (const TSharedRef<FChaosVDLogInFlightFilteringTaskWrapper>& FilterTask : FilteringTasksBeingCancelled)
	{
		if (!FilterTask->GetAsyncTaskRef().IsIdle())
		{
			FilterTask->GetAsyncTaskRef().EnsureCompletion();
		}
	}

	if (CurrentFilteringTask)
	{
		CurrentFilteringTask->GetAsyncTaskRef().EnsureCompletion();
	}
}

void SChaosVDRecordedLogBrowser::HandleSearchTextChanged(const FText& NewText)
{
	FilterEvaluator->SetFilterText(NewText);
	ApplyFiltersAsync();
}

void SChaosVDRecordedLogBrowser::UpdateLogLineSelectionFromGameTrack()
{
	TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin();
	if (TSharedPtr<FChaosVDRecording> RecordingData = PlaybackControllerPtr ? PlaybackControllerPtr->GetCurrentRecording().Pin() : nullptr)
	{
		if (TSharedPtr<const FChaosVDTrackInfo> GameTrackInfo = PlaybackControllerPtr->GetTrackInfo(EChaosVDTrackType::Game, FChaosVDPlaybackController::GameTrackID))
		{
			PendingTrackUpdatesToProcess.Enqueue(GameTrackInfo);
		}
	}
}

void SChaosVDRecordedLogBrowser::SetSessionName(const FString& NewSessionName)
{
	CurrentSessionName = NewSessionName;
	UpdateBrowserContents();
}

void SChaosVDRecordedLogBrowser::UpdateBrowserContents()
{
	TSharedPtr<FChaosVDEngine> CVDEngineInstance = EngineInstanceWeakPtr.Pin();
	if (!CVDEngineInstance)
	{
		return;	
	}

	TSharedPtr<const TraceServices::IAnalysisSession> Session = nullptr;

	if (!CurrentSessionName.IsEmpty())
	{
		Session = FChaosVDModule::Get().GetTraceSessionsManager()->GetSession(CurrentSessionName);
	}
	
	if (!Session)
	{
		return;
	}
	
	TSharedPtr<SChaosVDRecordedLogView> LogViewList = LogViewWidget.Pin();
	if(!LogViewList)
	{
		return;
	}

	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
	const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

	uint64 LogProviderNumMessages = LogProvider.GetMessageCount();

	bool bHasNewLogData = false;
	bool bNeedsReset = false;
	bool bFirstUpdate = false;

	{
		FReadScopeLock ReadScopeLock(SourceDataCacheLock);
		int32 CurrentCachedItemsNum = UnfilteredCachedLogItems.Num();
		bHasNewLogData =  CurrentCachedItemsNum != LogProviderNumMessages;
		bNeedsReset = CurrentCachedItemsNum > LogProviderNumMessages;
		bFirstUpdate = CurrentCachedItemsNum == 0;
	}

	if (bNeedsReset)
	{
		Reset();
	}

	if (bHasNewLogData)
	{
		FWriteScopeLock WriteScopeLock(SourceDataCacheLock);

		for (uint64 Index = UnfilteredCachedLogItems.Num(); Index < LogProviderNumMessages; Index++)
		{
			TSharedRef<FChaosVDCachedLogItemEntry> LogEntryData = MakeShared<FChaosVDCachedLogItemEntry>();
			LogProvider.ReadMessage(Index, [LogEntryData](const TraceServices::FLogMessageInfo& MessageInfo)
			{
				LogEntryData->Category = MessageInfo.Category->Name;
				LogEntryData->Verbosity = MessageInfo.Verbosity;
				LogEntryData->Message = MessageInfo.Message;
				LogEntryData->Index = MessageInfo.Index;
				LogEntryData->Time = MessageInfo.Time;
			});

			CachedReadLogData.Add(LogEntryData);

			UnfilteredCachedLogItems.Add(MakeLogItem_AssumesLocked(LogEntryData));
		}
	}

	if (bFirstUpdate || bNeedsReset)
	{
		UpdateLogLineSelectionFromGameTrack();
	}
}

FText SChaosVDRecordedLogBrowser::GetFilterStatusText() const
{
	FReadScopeLock ReadScopeLock(SourceDataCacheLock);

	const int32 FilteredItemsNum = FilteredCachedLogItems->Num();
	const int32 UnFilteredItemsNum = UnfilteredCachedLogItems.Num();

	return FText::FormatOrdered(LOCTEXT("LogBrowserFilterStatusMessage","Showing {0} Log Entries | {1} entries are hidden by filter."), FilteredItemsNum, UnFilteredItemsNum - FilteredItemsNum);
}

FSlateColor SChaosVDRecordedLogBrowser::GetFilterStatusTextColor() const
{
	FReadScopeLock ReadScopeLock(SourceDataCacheLock);

	const int32 FilteredItemsNum = FilteredCachedLogItems->Num();
	const int32 UnFilteredItemsNum = UnfilteredCachedLogItems.Num();
	
	if (FilterEvaluator->GetFilterText().IsEmpty())
	{
		return FSlateColor::UseForeground();
	}
	else if (FilteredItemsNum == 0 && UnFilteredItemsNum > 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	else
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
	}
}

void SChaosVDRecordedLogBrowser::CleanPendingCanceledTaskList()
{
	bool bAllPendingTasksFullyCanceled = true;
	for (const TSharedRef<FChaosVDLogInFlightFilteringTaskWrapper>& FilterTask : FilteringTasksBeingCancelled)
	{
		bAllPendingTasksFullyCanceled &= FilterTask->GetAsyncTaskRef().IsIdle();
	}

	// We can only clean the pending tasks list if they are fully cancelled and Idle
	if (bAllPendingTasksFullyCanceled)
	{
		FilteringTasksBeingCancelled.Reset();
	}
}

void SChaosVDRecordedLogBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (GetDirtyFlags() != EChaosVDLogBrowserDirtyFlags::None)
	{
		UpdateBrowserContents();
		ApplyFiltersAsync();
		RefreshLogListView();
		ProcessPendingTrackUpdates();
		DirtyFlags = EChaosVDLogBrowserDirtyFlags::None;
	}

	CleanPendingCanceledTaskList();
}

void SChaosVDRecordedLogBrowser::ToggleShowAllCategories()
{
    bShowAllCategories = !bShowAllCategories;

	{
		FWriteScopeLock WriteScopeLock(SourceDataCacheLock);
    	// This is to mimic the filtering widget of the output log, where this setting stomps the enabled value of any category
    	for (TPair<FName, FCategorizedItemsContainer>& Container : CachedLogItemsByCategory)
    	{
    		Container.Value.bIsEnabled = bShowAllCategories;
    	}
	}
	
	SetDirtyFlag(EChaosVDLogBrowserDirtyFlags::Categories);
}

bool SChaosVDRecordedLogBrowser::IsCategoryEnabled(FName CategoryName)
{
	FReadScopeLock ReadScopeLock(SourceDataCacheLock);

	if (FCategorizedItemsContainer* Container = CachedLogItemsByCategory.Find(CategoryName))
	{
		return Container->bIsEnabled;
	}

	return false;
}

void SChaosVDRecordedLogBrowser::ToggleCategoryEnabled(FName CategoryName)
{
	FWriteScopeLock WriteScopeLock(SourceDataCacheLock);

	if (FCategorizedItemsContainer* Container = CachedLogItemsByCategory.Find(CategoryName))
	{
		Container->bIsEnabled = !Container->bIsEnabled;

		SetDirtyFlag(EChaosVDLogBrowserDirtyFlags::Categories);
	}
}

void SChaosVDRecordedLogBrowser::SetVerbosityFlags(EChaosVDLogVerbosityFlags NewFlags)
{
	{
		FWriteScopeLock WriteScopeLock(VerbosityFlagsLock);
		VerbosityFlags = NewFlags;
	}

	SetDirtyFlag(EChaosVDLogBrowserDirtyFlags::Verbosity);
}

TSharedPtr<FChaosVDLogViewListItem> SChaosVDRecordedLogBrowser::MakeLogItem_AssumesLocked(const TSharedPtr<FChaosVDCachedLogItemEntry>& InLogData)
{
	if (!InLogData)
	{
		return nullptr;
	}

	if (TSharedPtr<FChaosVDLogViewListItem>* FoundItemPtr = CachedLogItemsByID.Find(InLogData->Index))
	{
		return *FoundItemPtr;
	}

	TSharedPtr<FChaosVDLogViewListItem> NewLogItem = MakeShared<FChaosVDLogViewListItem>();
	NewLogItem->ItemWeakPtr = InLogData;
	NewLogItem->EntryIndex = InLogData->Index;

	CachedLogItemsByID.Add(InLogData->Index, NewLogItem);
	FindOrAddCategorizedItemsContainer_AssumesLocked(InLogData->Category).Items.Add(NewLogItem);

	return NewLogItem;
}

void SChaosVDRecordedLogBrowser::ApplyFiltersAsync()
{
	FReadScopeLock ReadScopeLock(SourceDataCacheLock);

	EChaosVDLogBrowserDirtyFlags FlagsForFilteringContext = GetDirtyFlags();
	// We can ignore playback because that is only related to selection events, no need to re-filter
	EnumRemoveFlags(FlagsForFilteringContext, EChaosVDLogBrowserDirtyFlags::Playback);

	// We can ignore filtering because this just indicates that a filter was applied, it does not mean it is out of date
	EnumRemoveFlags(FlagsForFilteringContext, EChaosVDLogBrowserDirtyFlags::Filtering);

	FChaosVDLogInFlightFilteringTaskWrapper::FContext FilteringContext(UnfilteredCachedLogItems.Num(), FilterEvaluator->GetFilterText(), FlagsForFilteringContext, VerbosityFlags);
	if (CurrentFilteringTask)
	{
		FAsyncTask<FChaosVDLogFilteringAsyncTask>& AsyncTask = CurrentFilteringTask->GetAsyncTaskRef();

		if (!AsyncTask.IsIdle())
		{
			// If we have a valid task that is in progress or is waiting to start, and the context (filter and source data) is the same as before,
			// we can just early out
			if (CurrentFilteringTask->HasSameContext(FilteringContext))
			{
				return;
			}
			else
			{
				AsyncTask.Cancel();
	
				// We need to keep the task alive until it is processed for cancellation
				FilteringTasksBeingCancelled.Add(CurrentFilteringTask.ToSharedRef());
			}
		}
	}
	
	CurrentFilteringTask = MakeShared<FChaosVDLogInFlightFilteringTaskWrapper>(FilteringContext, SharedThis(this));
	CurrentFilteringTask->GetAsyncTaskRef().StartBackgroundTask();
}

void SChaosVDRecordedLogBrowser::ApplyFilterToData_AssumesLocked(TConstArrayView<TSharedPtr<FChaosVDLogViewListItem>> InDataSource, const TSharedRef<TArray<TSharedPtr<FChaosVDLogViewListItem>>>& OutFilteredData)
{
	for (const TSharedPtr<FChaosVDLogViewListItem>& LogListItem : InDataSource)
	{
		TSharedPtr<FChaosVDCachedLogItemEntry> LogEntry = LogListItem ? LogListItem->ItemWeakPtr.Pin() : nullptr;
		if (!LogEntry)
		{
			continue;
		}

		if (IsVerbosityEnabled(LogEntry->Verbosity) && FilterEvaluator->TestTextFilter(FChaosVDBasicLogFilterExpressionContext(LogEntry.ToSharedRef())))
		{
			OutFilteredData->Emplace(LogListItem);
		}
	}
}

void SChaosVDRecordedLogBrowser::RefreshLogListView()
{
	TSharedPtr<SChaosVDRecordedLogView> LogViewList = LogViewWidget.Pin();
	if (!LogViewList)
	{
		return;
	}

	FReadScopeLock ReadScopeLock(SourceDataCacheLock);
	static TArray<TSharedPtr<FChaosVDLogViewListItem>> CurrentSelection;
	CurrentSelection.Reset();
	
	LogViewList->GetSelectedItems(CurrentSelection);

	LogViewList->SetSourceList(FilteredCachedLogItems);

	LogViewList->SelectItems(CurrentSelection, ESelectInfo::Direct);
}

void SChaosVDRecordedLogBrowser::HandleItemSelected(const TSharedPtr<FChaosVDLogViewListItem>& InLogViewListItem, ESelectInfo::Type SelectInfo)
{
	if (SelectInfo != ESelectInfo::Direct)
	{
		HandleItemFocused(InLogViewListItem);
	}
}

void SChaosVDRecordedLogBrowser::HandleItemFocused(const TSharedPtr<FChaosVDLogViewListItem>& InLogViewListItem)
{
	if (!InLogViewListItem)
	{
		return;
	}

	TSharedPtr<FChaosVDCachedLogItemEntry> LogEntryPtr = InLogViewListItem->ItemWeakPtr.Pin();

	if (!LogEntryPtr)
	{
		return;
	}

	TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = GetObservedController().Pin();
	TSharedPtr<FChaosVDRecording> RecordingData = PlaybackControllerPtr ? PlaybackControllerPtr->GetCurrentRecording().Pin() : nullptr;
	if (!RecordingData)
	{
		return;
	}

	TSharedPtr<FChaosVDEngine> CVDEngineInstance = EngineInstanceWeakPtr.Pin();
	TSharedPtr<const TraceServices::IAnalysisSession> Session = CVDEngineInstance && !CurrentSessionName.IsEmpty() ?
																FChaosVDModule::Get().GetTraceSessionsManager()->GetSession(CurrentSessionName) :
																nullptr;

	if (!Session)
	{
		return;
	}

	int32 GameFrameNumber = RecordingData->GetLowestGameFrameNumberAtTime(LogEntryPtr->Time);
	if (GameFrameNumber != INDEX_NONE)
	{
		constexpr int32 StageNumber = 0;
		PlaybackControllerPtr->GoToTrackFrameAndSync(GetInstigatorID(), EChaosVDTrackType::Game, FChaosVDPlaybackController::GameTrackID, GameFrameNumber, StageNumber);
	}
}

void SChaosVDRecordedLogBrowser::HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InController)
{
	FChaosVDPlaybackControllerObserver::HandlePlaybackControllerDataUpdated(InController);
	SetDirtyFlag(EChaosVDLogBrowserDirtyFlags::Messages);
}

void SChaosVDRecordedLogBrowser::HandleControllerTrackFrameUpdated(TWeakPtr<FChaosVDPlaybackController> InController, TWeakPtr<const FChaosVDTrackInfo> UpdatedTrackInfo, FGuid InstigatorGuid)
{
	if (InstigatorGuid != GetInstigatorID())
	{
		PendingTrackUpdatesToProcess.Enqueue(UpdatedTrackInfo);
		SetDirtyFlag(EChaosVDLogBrowserDirtyFlags::Playback);
	}
}

void SChaosVDRecordedLogBrowser::SetDirtyFlag(EChaosVDLogBrowserDirtyFlags Flag)
{
	FWriteScopeLock WriteLock(DirtyFlagsLock);
	EnumAddFlags(DirtyFlags, Flag);
}

void SChaosVDRecordedLogBrowser::RemoveDirtyFlag(EChaosVDLogBrowserDirtyFlags Flag)
{
	FWriteScopeLock WriteLock(DirtyFlagsLock);
	EnumRemoveFlags(DirtyFlags, Flag);
}

void SChaosVDRecordedLogBrowser::ClearAllDirtyFlags()
{
	FWriteScopeLock WriteLock(DirtyFlagsLock);
	DirtyFlags = EChaosVDLogBrowserDirtyFlags::None;
}

EChaosVDLogBrowserDirtyFlags SChaosVDRecordedLogBrowser::GetDirtyFlags() const
{
	FReadScopeLock ReadLock(DirtyFlagsLock);
	return DirtyFlags;
}

SChaosVDRecordedLogBrowser::FCategorizedItemsContainer& SChaosVDRecordedLogBrowser::FindOrAddCategorizedItemsContainer_AssumesLocked(FName CategoryName)
{
	if (FCategorizedItemsContainer* Container = CachedLogItemsByCategory.Find(CategoryName))
	{
		return *Container;
	}

	FCategorizedItemsContainer NewContainer;
	NewContainer.CategoryName = CategoryName;
	NewContainer.bIsEnabled = bShowAllCategories;
	return CachedLogItemsByCategory.Add(CategoryName, NewContainer);
}

bool SChaosVDRecordedLogBrowser::IsVerbosityEnabled(ELogVerbosity::Type VerbosityLevel)
{
	FReadScopeLock ReadScopeLock(VerbosityFlagsLock);

	switch(VerbosityLevel)
	{
	case ELogVerbosity::Type::Error:
		{
			return EnumHasAnyFlags(VerbosityFlags, EChaosVDLogVerbosityFlags::Errors);
		}
	case ELogVerbosity::Type::Warning:
		{
			return EnumHasAnyFlags(VerbosityFlags, EChaosVDLogVerbosityFlags::Warnings);
		}
	default:
		{
			return EnumHasAnyFlags(VerbosityFlags, EChaosVDLogVerbosityFlags::Messages);
		}
	}	
}

void SChaosVDRecordedLogBrowser::ProcessTrackUpdate(const TSharedRef<FChaosVDRecording>& InRecordingData, const TSharedRef<SChaosVDRecordedLogView>& InLogViewList, const TraceServices::ILogProvider& InLogProvider, const TWeakPtr<const FChaosVDTrackInfo>& InPendingTrackInfo)
{
	TSharedPtr<const FChaosVDTrackInfo> TrackInfoPtr = InPendingTrackInfo.Pin();
	if (!TrackInfoPtr)
	{
		return;
	}

	FMemMark StackMarker(FMemStack::Get());
	TArray<uint64, TInlineAllocator<64, TMemStackAllocator<>>> MessageIndexesBuffer;

	switch (TrackInfoPtr->TrackType)
	{
	case EChaosVDTrackType::Game:
		{
			if (FChaosVDGameFrameData* FrameData = InRecordingData->GetGameFrameData_AssumesLocked(TrackInfoPtr->CurrentFrame))
			{
				GetLogMessageIndexesForFrame(InLogProvider, *FrameData, MessageIndexesBuffer);
			}
			break;
		}
	case EChaosVDTrackType::Solver:
		{
			if (FChaosVDSolverFrameData* FrameData = InRecordingData->GetSolverFrameData_AssumesLocked(TrackInfoPtr->TrackID, TrackInfoPtr->CurrentFrame))
			{
				GetLogMessageIndexesForFrame(InLogProvider, *FrameData, MessageIndexesBuffer);
			}
			break;
		}
	default: break;
	}

	for (uint64 MessageIndex : MessageIndexesBuffer)
	{
		if (TSharedPtr<FChaosVDLogViewListItem>* FoundItem = CachedLogItemsByID.Find(MessageIndex))
		{
			InLogViewList->SelectItem(*FoundItem, ESelectInfo::Type::Direct);
		}
	}
}

void SChaosVDRecordedLogBrowser::ProcessPendingTrackUpdates()
{
	if (PendingTrackUpdatesToProcess.IsEmpty())
	{
		return;
	}

	TSharedPtr<SChaosVDRecordedLogView> LogViewList = LogViewWidget.Pin();
	if (!LogViewList)
	{
		return;
	}

	TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = GetObservedController().Pin();
	TSharedPtr<FChaosVDRecording> RecordingData = PlaybackControllerPtr ? PlaybackControllerPtr->GetCurrentRecording().Pin() : nullptr;
	if (!RecordingData)
	{
		return;
	}

	TSharedPtr<FChaosVDEngine> CVDEngineInstance = EngineInstanceWeakPtr.Pin();
	TSharedPtr<UE::TraceBasedDebuggers::FTraceSessionsManager> TraceManager = FChaosVDModule::Get().GetTraceSessionsManager();
	TSharedPtr<const TraceServices::IAnalysisSession> Session = CVDEngineInstance && TraceManager && !CurrentSessionName.IsEmpty() ?
																TraceManager->GetSession(CurrentSessionName) :
																nullptr;
	if (!Session)
	{
		return;
	}
	
	TraceServices::FAnalysisSessionReadScope SessionReadScope(*Session.Get());
	const TraceServices::ILogProvider& LogProvider = TraceServices::ReadLogProvider(*Session.Get());

	LogViewList->ClearSelection();

	TWeakPtr<const FChaosVDTrackInfo> PendingTrackInfo;
	while (PendingTrackUpdatesToProcess.Dequeue(PendingTrackInfo))
	{
		ProcessTrackUpdate(RecordingData.ToSharedRef(), LogViewList.ToSharedRef(), LogProvider, PendingTrackInfo);
	}
}

void SChaosVDRecordedLogBrowser::Reset()
{
	FWriteScopeLock WriteScopeLock(SourceDataCacheLock);

	CachedReadLogData.Reset();
	CachedLogItemsByID.Reset();
	CachedLogItemsByCategory.Reset();
	UnfilteredCachedLogItems.Reset();
}

#undef LOCTEXT_NAMESPACE
