// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/AsyncWork.h"
#include "ChaosVDPlaybackControllerInstigator.h"
#include "ChaosVDPlaybackControllerObserver.h"
#include "ContentBrowserDataMenuContexts.h"
#include "Framework/Docking/TabManager.h"
#include "Misc/TextFilterExpressionEvaluator.h"
#include "TraceServices/Model/Log.h"
#include "Widgets/SCompoundWidget.h"

#include "SChaosVDRecordedLogBrowser.generated.h"

class FTabManager;
class FChaosVDEngine;
class SChaosVDLogBrowserToolbar;
class SChaosVDRecordedLogBrowser;
class SChaosVDRecordedLogView;

struct FChaosVDCachedLogItemEntry;
struct FChaosVDLogViewListItem;
struct FChaosVDRecording;

namespace TraceServices
{
	class ILogProvider;
}

class FChaosVDBasicLogFilterExpressionContext : public ITextFilterExpressionContext
{
public:
	explicit FChaosVDBasicLogFilterExpressionContext(const TSharedRef<FChaosVDCachedLogItemEntry>& InTestLogEntry)
		: LogEntry(InTestLogEntry)
	{
	}

	virtual bool TestBasicStringExpression(const FTextFilterString& InValue, const ETextFilterTextComparisonMode InTextComparisonMode) const override;
	virtual bool TestComplexExpression(const FName& InKey, const FTextFilterString& InValue, const ETextFilterComparisonOperation InComparisonOperation, const ETextFilterTextComparisonMode InTextComparisonMode) const override
	{
		return false;
	}

private:
	const TSharedRef<FChaosVDCachedLogItemEntry> LogEntry;
};

class FChaosVDLogFilteringAsyncTask : public FNonAbandonableTask
{
public:

	explicit FChaosVDLogFilteringAsyncTask(const TWeakPtr<SChaosVDRecordedLogBrowser>& InLogBrowser) : LogBrowser(InLogBrowser)
	{
	}
	
	void DoWork();

	FORCEINLINE TStatId GetStatId() const
	{
		RETURN_QUICK_DECLARE_CYCLE_STAT(FChaosVDLogFilteringAsyncTask, STATGROUP_ThreadPoolAsyncTasks);
	}

private:

	TWeakPtr<SChaosVDRecordedLogBrowser> LogBrowser;

	bool bAbandonTask = false;
};

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDLogVerbosityFlags : uint8
{
	None = 0 UMETA(Hidden),
	Messages = 1 << 0,
	Warnings = 1 << 1,
	Errors = 1 << 2,

	All = Messages | Warnings | Errors
};
ENUM_CLASS_FLAGS(EChaosVDLogVerbosityFlags)

UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDLogBrowserDirtyFlags : uint8
{
	None = 0 UMETA(Hidden),
	Categories = 1 << 0,
	Messages = 1 << 1,
	Verbosity = 1 << 2,
	Filtering = 1 << 3,
	Playback = 1 << 4
};
ENUM_CLASS_FLAGS(EChaosVDLogBrowserDirtyFlags)

class FChaosVDLogInFlightFilteringTaskWrapper
{
public:
	struct FContext
	{
		explicit FContext(uint64 CurrentLogEntriesNum, const FText& InTextFilter, EChaosVDLogBrowserDirtyFlags InDirtyFlags, EChaosVDLogVerbosityFlags InnVerbosityFlags)
			: CurrentLogEntriesNum(CurrentLogEntriesNum),
			DirtyFlags(InDirtyFlags),
			VerbosityFlags(InnVerbosityFlags),
			TextFilter(InTextFilter.ToString())
		{
		}

		uint64 CurrentLogEntriesNum;
		EChaosVDLogBrowserDirtyFlags DirtyFlags;
		EChaosVDLogVerbosityFlags VerbosityFlags;

		FName TextFilter;
	
		bool operator==(const FContext& OtherContext) const = default;
	};
	
	explicit FChaosVDLogInFlightFilteringTaskWrapper(const FContext& InContext, const TWeakPtr<SChaosVDRecordedLogBrowser>& LogBrowser)
	: Context(InContext)
	{
		AsyncTask = MakeUnique<FAsyncTask<FChaosVDLogFilteringAsyncTask>>(LogBrowser);
	}

	bool HasSameContext(const FContext& InContext) const
	{
		return InContext == Context;
	}

	FAsyncTask<FChaosVDLogFilteringAsyncTask>& GetAsyncTaskRef() const
	{
		return *AsyncTask.Get();
	}
	
private:

	TUniquePtr<FAsyncTask<FChaosVDLogFilteringAsyncTask>> AsyncTask;
	FContext Context;
};

/**
 * Widget used to render the recorded log stream in a CVD Recording
 */
class SChaosVDRecordedLogBrowser : public SCompoundWidget, public FChaosVDPlaybackControllerObserver, public IChaosVDPlaybackControllerInstigator
{
public:
	SLATE_BEGIN_ARGS(SChaosVDRecordedLogBrowser)
		{
		}


	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<FChaosVDEngine>& InEngineInstance);

	virtual ~SChaosVDRecordedLogBrowser() override;
	
	void HandleSearchTextChanged(const FText& NewText);
	void UpdateLogLineSelectionFromGameTrack();

	void SetSessionName(const FString& NewSessionName);

	TSharedPtr<SChaosVDLogBrowserToolbar> GetToolBar()
	{
		return ToolbarPtr;
	};

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	
	static const FName ToolBarName;

	bool GetShowAllCategories() const
	{
		return bShowAllCategories;
	}
	
	void ToggleShowAllCategories();

	template<typename Visitor>
	void EnumerateNonEmptyCategories(Visitor VisitorFunc)
	{
		FReadScopeLock ReadScopeLock(SourceDataCacheLock);

		for (const TPair<FName, FCategorizedItemsContainer>& ItemsContainer : CachedLogItemsByCategory)
		{
			VisitorFunc(ItemsContainer.Value);
		}
	}

	bool IsCategoryEnabled(FName CategoryName);
	void ToggleCategoryEnabled(FName CategoryName);

	void SetVerbosityFlags(EChaosVDLogVerbosityFlags NewFlags);

	EChaosVDLogVerbosityFlags GetVerbosityFlags() const
	{
		return VerbosityFlags;
	}

	struct FCategorizedItemsContainer
	{
		TArray<TSharedPtr<FChaosVDLogViewListItem>> Items;
		FName CategoryName = NAME_None;
		bool bIsEnabled = true;
	};

protected:
	virtual void HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InController) override;
	virtual void HandleControllerTrackFrameUpdated(TWeakPtr<FChaosVDPlaybackController> InController, TWeakPtr<const FChaosVDTrackInfo> UpdatedTrackInfo, FGuid InstigatorGuid) override;

private:

	void ProcessTrackUpdate(const TSharedRef<FChaosVDRecording>& InRecordingData, const TSharedRef<SChaosVDRecordedLogView>& InLogViewList, const TraceServices::ILogProvider& InLogProvider, const TWeakPtr<const FChaosVDTrackInfo>& InPendingTrackInfo);

	void SetDirtyFlag(EChaosVDLogBrowserDirtyFlags Flag);
	void RemoveDirtyFlag(EChaosVDLogBrowserDirtyFlags Flag);
	void ClearAllDirtyFlags();
	EChaosVDLogBrowserDirtyFlags GetDirtyFlags() const;

	FCategorizedItemsContainer& FindOrAddCategorizedItemsContainer_AssumesLocked(FName CategoryName);

	bool IsVerbosityEnabled(ELogVerbosity::Type VerbosityLevel);

	void ProcessPendingTrackUpdates();

	void Reset();

	FText GetFilterStatusText() const;
	FSlateColor GetFilterStatusTextColor() const;

	void CleanPendingCanceledTaskList();
	void UpdateBrowserContents();

	TSharedPtr<FChaosVDLogViewListItem> MakeLogItem_AssumesLocked(const TSharedPtr<FChaosVDCachedLogItemEntry>& InLogData);

	void ApplyFiltersAsync();

	void ApplyFilterToData_AssumesLocked(TConstArrayView<TSharedPtr<FChaosVDLogViewListItem>> InDataSource, const TSharedRef<TArray<TSharedPtr<FChaosVDLogViewListItem>>>& OutFilteredData);
	void RefreshLogListView();

	void HandleItemSelected(const TSharedPtr<FChaosVDLogViewListItem>& InLogViewListItem, ESelectInfo::Type SelectInfo);
	void HandleItemFocused(const TSharedPtr<FChaosVDLogViewListItem>& InLogViewListItem);

	template<typename FrameType, typename IndexesContainerType>
	void GetLogMessageIndexesForFrame(const TraceServices::ILogProvider& InLogProvider, const FrameType& InFrameData, IndexesContainerType& OutIndexesContainer);

	TQueue<TWeakPtr<const FChaosVDTrackInfo>> PendingTrackUpdatesToProcess;
	TSharedPtr<FTextFilterExpressionEvaluator> FilterEvaluator;

	TWeakPtr<FChaosVDEngine> EngineInstanceWeakPtr;
	
	TWeakPtr<SChaosVDRecordedLogView> LogViewWidget;
	
	TSharedPtr<TArray<TSharedPtr<FChaosVDLogViewListItem>>> FilteredCachedLogItems;
	TArray<TSharedPtr<FChaosVDLogViewListItem>> UnfilteredCachedLogItems;

	TArray<TSharedPtr<FChaosVDLogViewListItem>> SelectedLogItems;

	TArray<TSharedPtr<FChaosVDCachedLogItemEntry>> CachedReadLogData;
	TMap<uint64, TSharedPtr<FChaosVDLogViewListItem>> CachedLogItemsByID;
	TMap<FName, FCategorizedItemsContainer> CachedLogItemsByCategory;

	TSharedPtr<FChaosVDLogInFlightFilteringTaskWrapper> CurrentFilteringTask;
	TArray<TSharedRef<FChaosVDLogInFlightFilteringTaskWrapper>> FilteringTasksBeingCancelled;

	TSharedPtr<SChaosVDLogBrowserToolbar> ToolbarPtr;

	bool bShowAllCategories = true;

	EChaosVDLogVerbosityFlags VerbosityFlags = EChaosVDLogVerbosityFlags::All;

	EChaosVDLogBrowserDirtyFlags DirtyFlags = EChaosVDLogBrowserDirtyFlags::None;

	mutable FRWLock SourceDataCacheLock;
	mutable FRWLock DirtyFlagsLock;
	mutable FRWLock VerbosityFlagsLock;

	FString CurrentSessionName;

	friend FChaosVDLogFilteringAsyncTask;
};

template <typename FrameType, typename IndexesContainerType>
void SChaosVDRecordedLogBrowser::GetLogMessageIndexesForFrame(const TraceServices::ILogProvider& InLogProvider, const FrameType& InFrameData, IndexesContainerType& OutIndexesContainer)
{
	InLogProvider.EnumerateMessages(InFrameData.StartTime, InFrameData.EndTime, [&OutIndexesContainer](const TraceServices::FLogMessageInfo& InMessageInfo)
			{
				OutIndexesContainer.Add(InMessageInfo.Index);
			});

	if (OutIndexesContainer.IsEmpty())
	{
		OutIndexesContainer.Add(InLogProvider.BinarySearchClosestByTime(InFrameData.StartTime));
	}
}
