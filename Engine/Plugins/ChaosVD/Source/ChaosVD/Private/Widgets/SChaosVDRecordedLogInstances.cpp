// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDRecordedLogInstances.h"

#include "Application/SlateApplicationBase.h"
#include "ChaosVDEngine.h"
#include "ChaosVDModule.h"
#include "ChaosVDStyle.h"
#include "SChaosVDRecordedLogBrowser.h"
#include "TraceSessionsManager.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SToolTip.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

SChaosVDRecordedLogInstances::~SChaosVDRecordedLogInstances()
{
	InstancesTabManager->CloseAllAreas();

	if (TSharedPtr<FChaosVDEngine> EngineInstance = EngineInstanceWeakPtr.Pin())
	{
		EngineInstance->OnTraceSessionOpened().RemoveAll(this);
		EngineInstance->OnTraceSessionClosed().RemoveAll(this);
	}
}

void SChaosVDRecordedLogInstances::Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& InOwnerTab, const TSharedRef<FChaosVDEngine>& InEngineInstance)
{
	EngineInstanceWeakPtr = InEngineInstance;

	InstancesTabManager = FGlobalTabmanager::Get()->NewTabManager(InOwnerTab).ToSharedPtr();
	
	TabsStack = FTabManager::NewStack()
				->SetHideTabWell(true);

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			InstancesTabManager->RestoreFrom(GenerateMainLayout(), InstancesTabManager->GetOwnerTab()->GetParentWindow()).ToSharedRef()
		]
	];

	InEngineInstance->OnTraceSessionClosed().AddSP(this, &SChaosVDRecordedLogInstances::HandleSessionClosed);
	InEngineInstance->OnTraceSessionOpened().AddSP(this, &SChaosVDRecordedLogInstances::HandleSessionOpened);

	using namespace UE::TraceBasedDebuggers;
	TArrayView<FTraceSessionDescriptor> SessionDescriptors = InEngineInstance->GetTraceSessionDescriptors();
	if (!SessionDescriptors.IsEmpty())
	{
		for (const FTraceSessionDescriptor& SessionDescriptor : SessionDescriptors)
		{
			HandleSessionOpened(SessionDescriptor);
		}
	}
}

void SChaosVDRecordedLogInstances::HandleSessionOpened(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor)
{
	FName TabID = GetAvailableTabID();

	ActiveTabsBySessionNameByTabID.Add(TabID, InSessionDescriptor.SessionName);

	InstancesTabManager->TryInvokeTab(TabID);
}

void SChaosVDRecordedLogInstances::HandleSessionClosed(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor)
{
	if (TWeakPtr<SDockTab>* ActiveTabForSessionPtrPtr = ActiveTabsBySessionName.Find(InSessionDescriptor.SessionName))
	{
		if (TSharedPtr<SDockTab> ActiveTabPtr = ActiveTabForSessionPtrPtr->Pin())
		{
			ActiveTabPtr->RequestCloseTab();
		}
	}
}

TSharedRef<SDockTab> SChaosVDRecordedLogInstances::HandleTabSpawnRequest(const FSpawnTabArgs& Args, FName TabSessionID)
{
	TSharedRef<SDockTab> RecordedLogTab = SNew(SDockTab)
											.TabRole(ETabRole::PanelTab)
											.Label(LOCTEXT("InnerRecordedOutputLogTabLabel", "Output Log"));
	if (ensure(!ActiveTabsByID.Contains(TabSessionID)))
	{
		if (const TSharedPtr<FChaosVDEngine> EngineInstance = EngineInstanceWeakPtr.Pin())
		{
			TSharedRef<SChaosVDRecordedLogBrowser> LogBrowser = SNew(SChaosVDRecordedLogBrowser, EngineInstance.ToSharedRef());

			RecordedLogTab->SetContent(LogBrowser);

			LogBrowser->SetSessionName(ActiveTabsBySessionNameByTabID.FindChecked(TabSessionID));
		}

		const FString& CurrentSessionNameForTab = ActiveTabsBySessionNameByTabID.FindChecked(TabSessionID);
		TSharedRef<IToolTip> TabToolTip = FSlateApplicationBase::Get().MakeToolTip(FText::AsCultureInvariant(CurrentSessionNameForTab));

		RecordedLogTab->SetTabToolTipWidget(StaticCastSharedRef<SToolTip>(TabToolTip));
		RecordedLogTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("TabIconOutputLog"));
		RecordedLogTab->SetOnTabClosed(SDockTab::FOnTabClosedCallback::CreateSP(this, &SChaosVDRecordedLogInstances::HandleTabClosed, TabSessionID));

		
		ActiveTabsByID.Add(TabSessionID, RecordedLogTab);
		ActiveTabsBySessionName.Add(ActiveTabsBySessionNameByTabID.FindChecked(TabSessionID), RecordedLogTab);
	}
	else
	{
		RecordedLogTab->SetContent(
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("ChaosVDEditorLogTabSpawnerError", "Failed to create log browser tab."))
			]
		);
	}

	return RecordedLogTab;
}

void SChaosVDRecordedLogInstances::HandleTabClosed(TSharedRef<SDockTab> InTabClosed, FName TabSessionID)
{
	AvailableTabIDs.Add(TabSessionID);
	ActiveTabsByID.Remove(TabSessionID);
	ActiveTabsBySessionName.Remove(ActiveTabsBySessionNameByTabID.FindChecked(TabSessionID));
	ActiveTabsBySessionNameByTabID.Remove(TabSessionID);
}

FName SChaosVDRecordedLogInstances::GenerateTabID()
{
	static FName PerInstanceLogBrowserTab = FName("RecordedLogInstanceTab");
	FName NewID = FName(PerInstanceLogBrowserTab, LastInstanceNumberUsed);
	LastInstanceNumberUsed++;
	InstancesTabManager->RegisterTabSpawner(NewID, FOnSpawnTab::CreateSP(this, &SChaosVDRecordedLogInstances::HandleTabSpawnRequest, NewID));
	TabsStack->AddTab(NewID, ETabState::ClosedTab);

	return NewID;
}

FName SChaosVDRecordedLogInstances::GetAvailableTabID()
{
	if (AvailableTabIDs.IsEmpty())
	{
		return GenerateTabID();
	}

	return AvailableTabIDs.Pop();
}

TSharedRef<FTabManager::FLayout> SChaosVDRecordedLogInstances::GenerateMainLayout()
{
	constexpr int32 MaxDefaultTabs = 16;
	for (int32 DefaultTabsCounter = 0; DefaultTabsCounter < MaxDefaultTabs; ++DefaultTabsCounter)
	{
		FName GeneratedTabID = GenerateTabID();
		AvailableTabIDs.Emplace(GeneratedTabID);

		UE_LOGF(LogChaosVDEditor, Verbose, "[%ls] Generated default log browser tab with ID [%ls]", ANSI_TO_TCHAR(__FUNCTION__), *GeneratedTabID.ToString());
	}

	return FTabManager::NewLayout("RecordedLogBrowser_Layout")
		->AddArea
		(
			FTabManager::NewPrimaryArea()
			->SetOrientation(Orient_Horizontal)
			->SetExtensionId("TopLevelArea")
			->Split
			(
				TabsStack.ToSharedRef()
			)
		);
}

#undef LOCTEXT_NAMESPACE
