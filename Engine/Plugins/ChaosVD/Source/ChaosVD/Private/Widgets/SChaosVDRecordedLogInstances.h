// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Docking/TabManager.h"
#include "Widgets/SCompoundWidget.h"

namespace UE::TraceBasedDebuggers
{
struct FTraceSessionDescriptor;
}

class FSpawnTabArgs;
class FChaosVDEngine;
class SDockTab;

/**
 * Widget that owns and manages multiple tabs of the recorded log browser widget
 */
class SChaosVDRecordedLogInstances : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChaosVDRecordedLogInstances)
	{
	}
	SLATE_END_ARGS()

	virtual ~SChaosVDRecordedLogInstances() override;

	void Construct(const FArguments& InArgs, const TSharedRef<SDockTab>& InOwnerTab, const TSharedRef<FChaosVDEngine>& InEngineInstance);

private:

	void HandleSessionOpened(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor);
	void HandleSessionClosed(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor);

	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args, FName TabSessionID);

	void HandleTabClosed(TSharedRef<SDockTab> InTabClosed, FName TabSessionID);
	FName GenerateTabID();

	FName GetAvailableTabID();
	
	TSharedRef<FTabManager::FLayout> GenerateMainLayout();

	TSharedPtr<FTabManager> InstancesTabManager;

	TSharedPtr<FTabManager::FStack> TabsStack;

	TWeakPtr<FChaosVDEngine> EngineInstanceWeakPtr;

	TMap<FName, TWeakPtr<SDockTab>> ActiveTabsByID;
	TMap<FString, TWeakPtr<SDockTab>> ActiveTabsBySessionName;
	TMap<FName, FString> ActiveTabsBySessionNameByTabID;

	int32 LastInstanceNumberUsed = 0;

	TArray<FName> AvailableTabIDs;
};
