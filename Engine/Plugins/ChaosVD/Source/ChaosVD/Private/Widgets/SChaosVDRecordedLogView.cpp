// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDRecordedLogView.h"

#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

SChaosVDRecordedLogView::FColumNames SChaosVDRecordedLogView::ColumnNames = SChaosVDRecordedLogView::FColumNames();

void SChaosVDLogViewRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
									.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

	SMultiColumnTableRow<TSharedPtr<const FChaosVDLogViewListItem>>::Construct(Args, InOwnerTableView);
}

TSharedRef<SWidget> SChaosVDLogViewRow::GenerateTextWidgetFromText(const FText& Text, const FLinearColor& InColor)
{
	constexpr float MarginLeft = 4.0f;
	constexpr float NoMargin = 0.0f;

	return SNew(STextBlock)
			.Margin(FMargin(MarginLeft, NoMargin, NoMargin, NoMargin))
			.ColorAndOpacity(InColor)
			.Text(Text);
}

TSharedRef<SWidget> SChaosVDLogViewRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (!Item)
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<const FChaosVDCachedLogItemEntry> LogEntryDataPtr = Item->ItemWeakPtr.Pin();
	if (!LogEntryDataPtr)
	{
		return SNullWidget::NullWidget;
	}

	FLinearColor LogLineColor = FLinearColor::White;
	if (LogEntryDataPtr->Verbosity == ELogVerbosity::Error)
	{
		LogLineColor = FLinearColor::Red;
	}
	else if (LogEntryDataPtr->Verbosity == ELogVerbosity::Warning)
	{
		LogLineColor = FLinearColor::Yellow;
	}
	
	if (ColumnName == SChaosVDRecordedLogView::ColumnNames.Time)
	{
		static const FNumberFormattingOptions RecordedTimeFormatOptions = FNumberFormattingOptions()
		.SetMinimumFractionalDigits(3)
		.SetMaximumFractionalDigits(3);

		const FText RecordingTimeSecondsAsText = FText::AsNumber(LogEntryDataPtr->Time, &RecordedTimeFormatOptions);
		return GenerateTextWidgetFromText(FText::FormatOrdered(FText::AsCultureInvariant("{0}s"), RecordingTimeSecondsAsText), LogLineColor);
	}
	
	if (ColumnName == SChaosVDRecordedLogView::ColumnNames.Verbosity)
	{
		return GenerateTextWidgetFromText(FText::FromString(::ToString(LogEntryDataPtr->Verbosity)), LogLineColor);
	}

	if (ColumnName == SChaosVDRecordedLogView::ColumnNames.Category)
	{
		return GenerateTextWidgetFromText(FText::FromName(LogEntryDataPtr->Category), LogLineColor);
	}

	if (ColumnName == SChaosVDRecordedLogView::ColumnNames.Message)
	{
		return GenerateTextWidgetFromText(FText::FromString(LogEntryDataPtr->Message), LogLineColor);
	}
	
	return SNullWidget::NullWidget;
}

void SChaosVDRecordedLogView::Construct(const FArguments& InArgs)
{
	ItemSelectedDelegate = InArgs._OnItemSelected;
	ItemFocusedDelegate = InArgs._OnItemFocused;

	constexpr float BottomPadding = 2.0f;
	constexpr float NoPadding = 0.0f;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(NoPadding, NoPadding, NoPadding, BottomPadding)
		[
			SAssignNew(LogListWidget, SListView<TSharedPtr<FChaosVDLogViewListItem>>)
			.OnGenerateRow(this, &SChaosVDRecordedLogView::GenerateLogEntryRow)
			.OnSelectionChanged(this, &SChaosVDRecordedLogView::LogItemSelectionChanged)
			.OnMouseButtonDoubleClick(this, &SChaosVDRecordedLogView::HandleFocusRequest)
			.SelectionMode(ESelectionMode::Type::Multi)
			.ListItemsSource(&InternalItemSourceData)
			.HeaderRow
			(
				// TODO: Add sort support for the columns
				SNew(SHeaderRow)
				+SHeaderRow::Column(ColumnNames.Time)
				.SortMode(EColumnSortMode::None)
				.ManualWidth(80.0f)
				.DefaultLabel(LOCTEXT("LogTimeHeader", "Time"))

				+SHeaderRow::Column(ColumnNames.Category)
				.SortMode(EColumnSortMode::None)
				.ManualWidth(160.0f)
				.DefaultLabel(LOCTEXT("LogCategoryHeader", "Category"))

				+SHeaderRow::Column(ColumnNames.Verbosity)
				.SortMode(EColumnSortMode::None)
				.ManualWidth(80.0f)
				.DefaultLabel(LOCTEXT("LogVerbosityHeader", "Verbosity"))
				
				+SHeaderRow::Column(ColumnNames.Message)
				.SortMode(EColumnSortMode::None)
				.FillWidth(1.0f)
				.DefaultLabel(LOCTEXT("LogMessageHeader", "Message"))
			)
		]
	];
}

TSharedRef<ITableRow> SChaosVDRecordedLogView::GenerateLogEntryRow(TSharedPtr<FChaosVDLogViewListItem> LogEntryData, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (!LogEntryData)
	{
		return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SChaosVDSceneQueryListErrorMessage", "Failed to read data for solver."))
			]
		];
	}

	return SNew(SChaosVDLogViewRow, OwnerTable)
			.Item(LogEntryData);
}

void SChaosVDRecordedLogView::LogItemSelectionChanged(TSharedPtr<FChaosVDLogViewListItem> SelectedLogItem, ESelectInfo::Type Type)
{
	ItemSelectedDelegate.ExecuteIfBound(SelectedLogItem, Type);
}

void SChaosVDRecordedLogView::SelectItem(const TSharedPtr<FChaosVDLogViewListItem>& ItemToSelect, ESelectInfo::Type Type)
{
	if (LogListWidget->IsItemSelected(ItemToSelect))
	{
		return;
	}

	constexpr bool bIsSelected = true;
	LogListWidget->SetItemSelection(ItemToSelect, bIsSelected, Type);
	LogListWidget->RequestScrollIntoView(ItemToSelect);
}

void SChaosVDRecordedLogView::SelectItems(TConstArrayView<TSharedPtr<FChaosVDLogViewListItem>> ItemsToSelect, ESelectInfo::Type Type)
{
	for (TSharedPtr<FChaosVDLogViewListItem> Item : ItemsToSelect)
	{
		SelectItem(Item, ESelectInfo::Direct);
	}
}

int32 SChaosVDRecordedLogView::GetSelectedItems(TArray<TSharedPtr<FChaosVDLogViewListItem>>& OutItems)
{
	return LogListWidget->GetSelectedItems(OutItems);
}

void SChaosVDRecordedLogView::ClearSelection()
{
	LogListWidget->ClearSelection();
}

void SChaosVDRecordedLogView::SetSourceList(const TSharedPtr<TArray<TSharedPtr<FChaosVDLogViewListItem>>>& InSourceList)
{
	LogListWidget->SetItemsSource(InSourceList.Get());
	LogListWidget->RebuildList();
}

void SChaosVDRecordedLogView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SChaosVDRecordedLogView::HandleFocusRequest(TSharedPtr<FChaosVDLogViewListItem> InFocusedItem)
{
	ItemFocusedDelegate.ExecuteIfBound(InFocusedItem);
}

#undef LOCTEXT_NAMESPACE

