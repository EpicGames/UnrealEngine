// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class FChaosVDEngine;

struct FChaosVDCachedLogItemEntry
{
	uint64 Index = 0;
	FName Category = NAME_None;
	ELogVerbosity::Type Verbosity = ELogVerbosity::Type::All;
	FString Message;
	double Time = 0.0;
};

struct FChaosVDLogViewListItem
{
	TWeakPtr<FChaosVDCachedLogItemEntry> ItemWeakPtr;
	uint64 EntryIndex = 0;
};

class SChaosVDSceneQueryBrowser;
struct FChaosVDQueryDataWrapper;
struct FChaosVDSceneQueryTreeItem;

/**
 * Widget used to represent a row on the Scene Query Browser tree view
 */
class SChaosVDLogViewRow : public SMultiColumnTableRow<TSharedPtr<const FChaosVDLogViewListItem>>
{
public:
	SLATE_BEGIN_ARGS(SChaosVDLogViewRow)
	{
	}
	SLATE_ARGUMENT(TSharedPtr<FChaosVDLogViewListItem>, Item)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

	TSharedRef<SWidget> GenerateTextWidgetFromText(const FText& Text, const FLinearColor& InColor = FLinearColor::White);

private:
	TSharedPtr<FChaosVDLogViewListItem> Item;
};

DECLARE_DELEGATE_TwoParams(FChaosVDLogIetmItemSelected, const TSharedPtr<FChaosVDLogViewListItem>&, ESelectInfo::Type)
DECLARE_DELEGATE_OneParam(FChaosVDLogItemFocused, const TSharedPtr<FChaosVDLogViewListItem>&)

class SChaosVDRecordedLogView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChaosVDRecordedLogView) {}
	SLATE_EVENT(FChaosVDLogIetmItemSelected, OnItemSelected)
	SLATE_EVENT(FChaosVDLogItemFocused, OnItemFocused)
SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

	int32 GetSelectedItems(TArray<TSharedPtr<FChaosVDLogViewListItem>>& OutItems);
	void SelectItem(const TSharedPtr<FChaosVDLogViewListItem>& ItemToSelect, ESelectInfo::Type Type);
	void SelectItems(TConstArrayView<TSharedPtr<FChaosVDLogViewListItem>> ItemsToSelect, ESelectInfo::Type Type);
	void ClearSelection();

	void SetSourceList(const TSharedPtr<TArray<TSharedPtr<FChaosVDLogViewListItem>>>& InSourceList);

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	void HandleFocusRequest(TSharedPtr<FChaosVDLogViewListItem> InFocusedItem);

	TSharedRef<ITableRow> GenerateLogEntryRow(TSharedPtr<FChaosVDLogViewListItem> LogEntryData, const TSharedRef<STableViewBase>& OwnerTable);

	void LogItemSelectionChanged(TSharedPtr<FChaosVDLogViewListItem> SelectedLogItem, ESelectInfo::Type Type);

	TSharedPtr<SListView<TSharedPtr<FChaosVDLogViewListItem>>> LogListWidget;

	TArray<TSharedPtr<FChaosVDLogViewListItem>> InternalItemSourceData;

	FChaosVDLogIetmItemSelected ItemSelectedDelegate;
	FChaosVDLogItemFocused ItemFocusedDelegate;

public:

	struct FColumNames
	{
		const FName Time = FName("Time");
		const FName Category = FName("Category");
		const FName Verbosity = FName("Verbosity");
		const FName Message = FName("Message");
	};

	static FColumNames ColumnNames;
	
};
