// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDSceneQueryBrowser.h"

#include "ChaosVDEngine.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "ChaosVDScene.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "Components/ChaosVDSceneQueryDataComponent.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "SChaosVDGameFramesPlaybackControls.h"
#include "Widgets/SChaosVDMainTab.h"
#include "SChaosVDSceneQueryTree.h"
#include "SChaosVDWarningMessageBox.h"
#include "Settings/ChaosVDSceneQueryVisualizationSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"
#include "Widgets/Input/SSearchBox.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SChaosVDSceneQueryBrowser)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

SChaosVDSceneQueryBrowser::~SChaosVDSceneQueryBrowser()
{
	UnregisterSceneEvents();

	if (UChaosVDSceneQueriesVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSceneQueriesVisualizationSettings>())
	{
		Settings->OnSettingsChanged().RemoveAll(this);
	}
}

void SChaosVDSceneQueryBrowser::Construct(const FArguments& InArgs, TWeakPtr<FChaosVDScene> Scene,  TWeakPtr<FEditorModeTools> EditorModeTools)
{
	SceneWeakPtr = Scene;
	EditorModeToolsWeakPtr = EditorModeTools;

	FilteredCachedTreeItems = MakeShared<TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>>();

	//TODO : At this point we should extract these values toa  style file other widgets can access so we have a coherent style between widgets
	constexpr float NoPadding = 0.0f;
	constexpr float MainContentBoxHorizontalPadding = 2.0f;
	constexpr float MainContentBoxVerticalPadding = 5.0f;
	constexpr float StatusBarSlotVerticalPadding = 1.0f;
	constexpr float StatusBarInnerVerticalPadding = 9.0f;
	constexpr float StatusBarInnerHorizontalPadding = 14.0f;

	RegisterMainToolbarMenu();
	
	RegisterSceneEvents();
	
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(NoPadding)
		[
			GenerateMainToolbarWidget()
		]
		+SVerticalBox::Slot()
		.Padding(MainContentBoxHorizontalPadding, MainContentBoxVerticalPadding, MainContentBoxHorizontalPadding, NoPadding)
		.AutoHeight()
		[
			SNew(SChaosVDWarningMessageBox)
			.Visibility(this, &SChaosVDSceneQueryBrowser::GetUpdatesPausedMessageVisibility)
			.WarningText(LOCTEXT("SceneQueryBrowserDataNoAvailableMessage", "Browser data updates disabled during playback..."))
		]
		+SVerticalBox::Slot()
		.Padding(MainContentBoxHorizontalPadding, MainContentBoxVerticalPadding, MainContentBoxHorizontalPadding, NoPadding)
		.FillHeight(1.0f)
		[
			SAssignNew(SceneQueryTreeWidget, SChaosVDSceneQueryTree)
			.IsEnabled(this, &SChaosVDSceneQueryBrowser::GetQueryTreeWidgetEnabled)
			.OnItemSelected(this, &SChaosVDSceneQueryBrowser::HandleTreeItemSelected)
			.OnItemFocused(this, &SChaosVDSceneQueryBrowser::HandleTreeItemFocused)
		]
		+SVerticalBox::Slot()
		.Padding(NoPadding, StatusBarSlotVerticalPadding, NoPadding, StatusBarSlotVerticalPadding)
		.AutoHeight()
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("Brushes.Header") )
			.VAlign(VAlign_Center)
			.HAlign(HAlign_Left)
			.Padding(FMargin(StatusBarInnerHorizontalPadding, StatusBarInnerVerticalPadding))
			[
				SNew(STextBlock)
				.Text(this, &SChaosVDSceneQueryBrowser::GetFilterStatusText)
				.ColorAndOpacity(this, &SChaosVDSceneQueryBrowser::GetFilterStatusTextColor)
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		[
			GenerateQueriesPlaybackControls()
		]
	];
	
	if (UChaosVDSceneQueriesVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSceneQueriesVisualizationSettings>())
	{
		Settings->OnSettingsChanged().AddSP(this, &SChaosVDSceneQueryBrowser::HandleSettingsChanged);

		HandleSettingsChanged(Settings);
	}
}

void SChaosVDSceneQueryBrowser::RegisterSceneEvents()
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->OnSolverVisibilityUpdated().AddRaw(this, &SChaosVDSceneQueryBrowser::HandleSolverVisibilityChanged);

		ScenePtr->OnSceneUpdated().AddRaw(this, &SChaosVDSceneQueryBrowser::HandleSceneUpdated);
		if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			SelectionObject->GetDataSelectionChangedDelegate().AddRaw(this, &SChaosVDSceneQueryBrowser::HandleExternalSelectionEvent);
		}
	}
}

void SChaosVDSceneQueryBrowser::UnregisterSceneEvents()
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->OnSceneUpdated().RemoveAll(this);
		ScenePtr->OnSolverVisibilityUpdated().RemoveAll(this);

		if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			SelectionObject->GetDataSelectionChangedDelegate().RemoveAll(this);
		}
	}
}

void SChaosVDSceneQueryBrowser::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (!bIsUpToDate && CanUpdate())
	{
		UpdateBrowserContents();
	}

	if (bNeedsToUpdateSettings)
	{
		ApplySettingsChange();
	}
}

void SChaosVDSceneQueryBrowser::SelectSceneQuery(const TSharedPtr<const FChaosVDQueryDataWrapper>& InQuery, ESelectInfo::Type Type)
{
	if (TSharedPtr<FChaosVDSceneQueryTreeItem>* ItemFound = CachedTreeItemsByID.Find(HashCombineFast(InQuery->ID, InQuery->WorldSolverID)))
	{
		SelectSceneQuery(*ItemFound, Type);
	}
}

void SChaosVDSceneQueryBrowser::SelectSceneQuery(const TSharedPtr<FChaosVDSceneQueryTreeItem>& SceneQueryTreeItem, ESelectInfo::Type Type)
{
	TSharedPtr<SChaosVDSceneQueryTree> SceneQueryTreeWidgetPtr = SceneQueryTreeWidget.Pin();
	if (!SceneQueryTreeWidgetPtr)
	{
		return;
	}

	if (SceneQueryTreeItem)
	{
		SceneQueryTreeWidgetPtr->SelectItem(SceneQueryTreeItem, Type);
	}
}

void SChaosVDSceneQueryBrowser::HandleExternalSelectionEvent(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle)
{
	TSharedPtr<FChaosVDQueryDataWrapper> SelectedQueryDataPtr = InDataSelectionHandle ? InDataSelectionHandle->GetDataAsShared<FChaosVDQueryDataWrapper>() : nullptr;
	if (!SelectedQueryDataPtr)
	{
		return;
	}

	SelectSceneQuery(SelectedQueryDataPtr, ESelectInfo::Type::Direct);
	UpdateAllTreeItemsVisibility();
}

void SChaosVDSceneQueryBrowser::HandleSolverVisibilityChanged(int32 SolverID, bool bNewVisibility)
{
	CachedSolverVisibilityByID.FindOrAdd(SolverID) = bNewVisibility;

	for (const TSharedPtr<FChaosVDSceneQueryTreeItem>& TreeItem : UnfilteredCachedTreeItems)
	{
		if (TreeItem && TreeItem->OwnerSolverID == SolverID)
		{
			UpdateTreeItemVisibility(TreeItem);
		}
	}
}

void SChaosVDSceneQueryBrowser::RegisterMainToolbarMenu()
{
	const UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered(ToolBarName))
	{
		return;
	}

	UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);

	FName MenuSection("SceneQueryBrowser.Toolbar.Settings");
	FText MenuSectionLabel = LOCTEXT("SceneQueryBrowserMenuLabel", "Scene Query Visualization Settings");
	FText FlagsMenuLabel = LOCTEXT("SceneQueryBrowserVisFlagsMenuLabel", "Visualization Flags");
	FText FlagsMenuTooltip = LOCTEXT("SceneQueryBrowserVisFlagsMenuToolTip", "Set of flags to enable/disable visibility of specific types of scene query data");
	FSlateIcon FlagsMenuIcon = FSlateIcon(FChaosVDStyle::Get().GetStyleSetName(), TEXT("SceneQueriesInspectorIcon"));

	FText SettingsMenuLabel = LOCTEXT("SceneQueryBrowserVisSettingsMenuLabel", "General Settings");
	FText SettingsMenuTooltip = LOCTEXT("SceneQueryBrowserVisMenuToolTip", "Options to change how the recorded scene query data is debug drawn");

	using namespace Chaos::VisualDebugger::Utils;
	CreateVisualizationOptionsMenuSections<UChaosVDSceneQueriesVisualizationSettings, EChaosVDSceneQueryVisualizationFlags>(ToolBar, MenuSection, MenuSectionLabel, FlagsMenuLabel, FlagsMenuTooltip, FlagsMenuIcon, SettingsMenuLabel, SettingsMenuTooltip);

	FToolMenuSection& Section = ToolBar->AddSection(TEXT("SceneQueryBrowser.Toolbar"));

	Section.AddSeparator(NAME_None);

	Section.AddDynamicEntry("SearchBar", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		const UChaosVDSceneQueryBrowserToolbarMenuContext* Context = InSection.FindContext<UChaosVDSceneQueryBrowserToolbarMenuContext>();
		TSharedRef<SChaosVDSceneQueryBrowser> BrowserWidget = Context->BrowserInstance.Pin().ToSharedRef();
		
		InSection.AddEntry(
		FToolMenuEntry::InitWidget(
			"SearchBar",
			BrowserWidget->GenerateSearchBarWidget(),
			FText::GetEmpty(),
			false,
			false
		));
		
	}));

	Section.AddDynamicEntry("VisualizationModes", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		const UChaosVDSceneQueryBrowserToolbarMenuContext* Context = InSection.FindContext<UChaosVDSceneQueryBrowserToolbarMenuContext>();
		TSharedRef<SChaosVDSceneQueryBrowser> BrowserWidget = Context->BrowserInstance.Pin().ToSharedRef();
		
		InSection.AddSeparator(NAME_None);
			
		InSection.AddEntry(
		FToolMenuEntry::InitWidget(
			"VisualizationModes",
			BrowserWidget->GenerateQueryVisualizationModeWidget(),
			FText::GetEmpty(),
			false,
			false
		));
	}));
}

TSharedRef<SWidget> SChaosVDSceneQueryBrowser::GenerateMainToolbarWidget()
{
	RegisterMainToolbarMenu();

	FToolMenuContext MenuContext;

	UChaosVDSceneQueryBrowserToolbarMenuContext* CommonContextObject = NewObject<UChaosVDSceneQueryBrowserToolbarMenuContext>();
	CommonContextObject->BrowserInstance = SharedThis(this);

	MenuContext.AddObject(CommonContextObject);

	return UToolMenus::Get()->GenerateWidget(ToolBarName, MenuContext);
}

TSharedRef<SWidget> SChaosVDSceneQueryBrowser::GenerateSearchBarWidget()
{
	TSharedPtr<SWidget> SearchBox = SNew(SSearchBox)
										.IsEnabled(this, &SChaosVDSceneQueryBrowser::GetQueryTreeWidgetEnabled) // If the tree widget is disabled, we can't serach
										.HintText(FText::FromString(TEXT("Search...")))
										.OnTextChanged(this, &SChaosVDSceneQueryBrowser::HandleSearchTextChanged);

	return SearchBox.ToSharedRef();
}

TSharedRef<SWidget> SChaosVDSceneQueryBrowser::GenerateQueryVisualizationModeWidget()
{
	TAttribute<int32> GetCurrentMode;
	GetCurrentMode.BindLambda([]()
	{
		if (UChaosVDSceneQueriesVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSceneQueriesVisualizationSettings>())
		{
			return static_cast<int32>(Settings->CurrentVisualizationMode);
		}
		return 0;
	});

	SEnumComboBox::FOnEnumSelectionChanged ValueChangedDelegate;
	ValueChangedDelegate.BindLambda([](int32 NewValue, ESelectInfo::Type)
	{
		if (UChaosVDSceneQueriesVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSceneQueriesVisualizationSettings>())
		{
			Settings->CurrentVisualizationMode = static_cast<EChaosVDSQFrameVisualizationMode>(NewValue);
			Settings->OnSettingsChanged().Broadcast(Settings);
		}
	});

	return Chaos::VisualDebugger::Utils::MakeEnumMenuEntryWidget<EChaosVDSQFrameVisualizationMode>(LOCTEXT("SQVisualizationModeModeMenuLabel", "Visualization Mode"), MoveTemp(ValueChangedDelegate), MoveTemp(GetCurrentMode));
}

EVisibility SChaosVDSceneQueryBrowser::GetUpdatesPausedMessageVisibility() const
{
	return !bIsUpToDate && !CanUpdate() ? EVisibility::Visible : EVisibility::Collapsed;
}

void SChaosVDSceneQueryBrowser::HandleSearchTextChanged(const FText& NewText)
{
	CurrentTextFilter = NewText;
	bIsUpToDate = false;
}

void SChaosVDSceneQueryBrowser::HandleSettingsChanged(UObject* SettingsObject)
{
	bNeedsToUpdateSettings = true;
}

void SChaosVDSceneQueryBrowser::ApplySettingsChange()
{
	UChaosVDSceneQueriesVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSceneQueriesVisualizationSettings>();
	if (!Settings)
	{
		return;
	}

	CurrentVisualizationMode = Settings->CurrentVisualizationMode;

	UpdateBrowserContents();

	switch (CurrentVisualizationMode)
	{
		case EChaosVDSQFrameVisualizationMode::AllEnabledQueries:
			{
				CurrentPlaybackIndex = 0;
				break;
			}
		case EChaosVDSQFrameVisualizationMode::PerSolverRecordingOrder:
			{

				if (FilteredCachedTreeItems->IsValidIndex(CurrentPlaybackIndex))
				{
					SelectSceneQuery((*FilteredCachedTreeItems)[CurrentPlaybackIndex], ESelectInfo::Type::OnMouseClick);
				}
				break;
			}
		default:
			break;
	}

	if (TSharedPtr<FEditorModeTools> EditorModeToolsPtr = EditorModeToolsWeakPtr.Pin())
	{
		// Make sure the viewport is re-drawn so the selection feedback is shown
		if (FEditorViewportClient* ViewportClient = EditorModeToolsPtr->GetFocusedViewportClient())
		{
			ViewportClient->Invalidate();
		}
	}

	bNeedsToUpdateSettings = false;
}

void SChaosVDSceneQueryBrowser::UpdateBrowserContents()
{
	TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return;
	}

	TSharedPtr<SChaosVDSceneQueryTree> SceneQueryTreeWidgetPtr = SceneQueryTreeWidget.Pin();
	if (!SceneQueryTreeWidgetPtr)
	{
		return;
	}

	UnfilteredCachedTreeItems.Reset();
	CachedTreeItemsByID.Reset();

	const TMap<int32, AChaosVDSolverInfoActor*>& SolverInfoDataMap = ScenePtr->GetSolverInfoActorsMap();

	for (const TPair<int32, AChaosVDSolverInfoActor*>& SolverDataWithID : SolverInfoDataMap)
	{
		if (SolverDataWithID.Key)
		{
			if (UChaosVDSceneQueryDataComponent* DataComponent = SolverDataWithID.Value->GetSceneQueryDataComponent())
			{
				for (const TSharedPtr<FChaosVDQueryDataWrapper>& QueryData : DataComponent->GetAllQueries())
				{
					if (QueryData->ParentQueryID == INDEX_NONE)
					{
						if (TSharedPtr<FChaosVDSceneQueryTreeItem> NewTreeItem = MakeSceneQueryTreeItem(QueryData, DataComponent))
						{
							UnfilteredCachedTreeItems.Add(NewTreeItem);
						}
					}
				}
			}
		}
	}

	UpdateAllTreeItemsVisibility();

	ApplyFilterToData(UnfilteredCachedTreeItems, FilteredCachedTreeItems.ToSharedRef());

	if (CurrentVisualizationMode == EChaosVDSQFrameVisualizationMode::PerSolverRecordingOrder)
	{
		// Note: Queries need to be sorted by query id but within their specific solver because query ids are created from global counters.
		// As CVD now supports multi-file mode we can't just rely on the IDs directly as they might collide between loaded recordings (and even
		// if they don't, the numeric value does not indicate the order of each query across recordings.

		// TODO: Now that this sorting is more complex, we might need to make this post-filtering async - Jira for tracking UE-241976

		FMemMark StackMarker(FMemStack::Get());
		typedef TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>, TMemStackAllocator<>> FTempQueryItemsArray;

		TMap<int32, TSharedPtr<FTempQueryItemsArray>> FilteredItemsBySolverID;

		for (TSharedPtr<FChaosVDSceneQueryTreeItem>& TreeItem : (*FilteredCachedTreeItems))
		{
			TSharedPtr<FTempQueryItemsArray>& SolverQueries = FilteredItemsBySolverID.FindOrAdd(TreeItem->OwnerSolverID);
			if (!SolverQueries)
			{
				SolverQueries = MakeShared<FTempQueryItemsArray>();
			}

			SolverQueries->HeapPush(TreeItem, [](const TSharedPtr<FChaosVDSceneQueryTreeItem>& ItemA, const TSharedPtr<FChaosVDSceneQueryTreeItem>& ItemB)
			{
				return ItemA && ItemB && ItemA->QueryID < ItemB->QueryID;
			});
		}

		FilteredCachedTreeItems->Reset();

		for (const TPair<int32, TSharedPtr<FTempQueryItemsArray>>& SortedQueriesPerSolverWithID : FilteredItemsBySolverID)
		{
			FilteredCachedTreeItems->Append((*SortedQueriesPerSolverWithID.Value));
		}
	}

	SceneQueryTreeWidgetPtr->SetExternalSourceData(FilteredCachedTreeItems);

	bIsUpToDate = true;
}

void SChaosVDSceneQueryBrowser::HandleSceneUpdated()
{
	bIsUpToDate = false;
}

bool SChaosVDSceneQueryBrowser::CanUpdate() const
{
	// Updating a list/tree widget every frame during playback is too expensive, so we disable it.
	// TODO: We should modify this to have playback controller access without going through multiple objects 
	TSharedPtr<FEditorModeTools> EditorModeToolsPtr = EditorModeToolsWeakPtr.Pin();
	TSharedPtr<SChaosVDMainTab> ToolkitHost = EditorModeToolsPtr ? StaticCastSharedPtr<SChaosVDMainTab>(EditorModeToolsPtr->GetToolkitHost()) : nullptr;
	TSharedPtr<FChaosVDPlaybackController> PlaybackController = ToolkitHost ? ToolkitHost->GetChaosVDEngineInstance()->GetPlaybackController() : nullptr;

	if (PlaybackController)
	{
		return !PlaybackController->IsPlaying();
	}

	return true;
}

bool SChaosVDSceneQueryBrowser::GetQueryTreeWidgetEnabled() const
{
	return bIsUpToDate || CanUpdate();
}

FText SChaosVDSceneQueryBrowser::GetFilterStatusText() const
{
	const int32 FilteredItemsNum = FilteredCachedTreeItems->Num();
	const int32 UnFilteredItemsNum = UnfilteredCachedTreeItems.Num();

	return FText::FormatOrdered(LOCTEXT("SceneQueryBrowserFilterStatusMessage","Showing {0} queries | {1} queries are hidden by search filter."), FilteredItemsNum, UnFilteredItemsNum - FilteredItemsNum);
}

FSlateColor SChaosVDSceneQueryBrowser::GetFilterStatusTextColor() const
{
	const int32 FilteredItemsNum = FilteredCachedTreeItems->Num();
	const int32 UnFilteredItemsNum = UnfilteredCachedTreeItems.Num();
	
	if (CurrentTextFilter.IsEmpty())
	{
		return FSlateColor::UseForeground();
	}
	else if (FilteredItemsNum == 0 && UnFilteredItemsNum > 0)
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentRed");
	}
	else
	{
		return FAppStyle::Get().GetSlateColor("Colors.AccentGreen");
	}
}

TSharedRef<SWidget> SChaosVDSceneQueryBrowser::GenerateQueriesPlaybackControls()
{
	constexpr float NoPadding = 0.0f;
	constexpr float ContainerHorizontalPadding = 2.0f;
	constexpr float ContainerTopPadding = 4.0f;
	constexpr float ContainerBottomPadding = 10.0f;

	constexpr float ControlsLabelVerticalPadding = 2.0f;

	return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
			.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Panel"))
			.Padding(ContainerHorizontalPadding, ContainerTopPadding, ContainerHorizontalPadding, ContainerBottomPadding)
			[
				SNew(SVerticalBox)
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(NoPadding, ControlsLabelVerticalPadding, NoPadding, ControlsLabelVerticalPadding)
				[
					SNew(STextBlock)
					.Justification(ETextJustify::Center)
					.Text(this, &SChaosVDSceneQueryBrowser::GetPlaybackQueryControlText)
				]
				+SVerticalBox::Slot()
				[
					SAssignNew(PlaybackControlsTimelineWidget, SChaosVDTimelineWidget)
					.ButtonVisibilityFlags(EChaosVDTimelineElementIDFlags::AllManualStepping)
					.IsEnabled(this, &SChaosVDSceneQueryBrowser::GetPlaybackControlsEnabled)
					.OnFrameChanged(this, &SChaosVDSceneQueryBrowser::HandlePlaybackQueryIndexUpdated)
					.OnButtonClicked(this, &SChaosVDSceneQueryBrowser::HandlePlaybackControlInput)
					.MinFrames(this, &SChaosVDSceneQueryBrowser::GetCurrentMinPlaybackQueryIndex)
					.MaxFrames(this, &SChaosVDSceneQueryBrowser::GetCurrentMaxPlaybackQueryIndex)
					.CurrentFrame(this, &SChaosVDSceneQueryBrowser::GetCurrentPlaybackQueryIndex)
				]
			];
}

bool SChaosVDSceneQueryBrowser::GetPlaybackControlsEnabled() const
{
	return CurrentVisualizationMode == EChaosVDSQFrameVisualizationMode::PerSolverRecordingOrder && GetCurrentMaxPlaybackQueryIndex() > 0;
}

void SChaosVDSceneQueryBrowser::HandlePlaybackControlInput(EChaosVDPlaybackButtonsID InputID)
{
	switch (InputID)
	{
	case EChaosVDPlaybackButtonsID::Next:
		{
			int32 NextQueryIndex = CurrentPlaybackIndex +1;
			if (FilteredCachedTreeItems->IsValidIndex(NextQueryIndex))
			{
				SelectSceneQuery((*FilteredCachedTreeItems)[NextQueryIndex], ESelectInfo::Type::OnMouseClick);
				CurrentPlaybackIndex = NextQueryIndex;
			}
			
			break;	
		}
	case EChaosVDPlaybackButtonsID::Prev:
		{
			int32 PrevQueryIndex = CurrentPlaybackIndex -1;
			if (FilteredCachedTreeItems->IsValidIndex(PrevQueryIndex))
			{
				SelectSceneQuery((*FilteredCachedTreeItems)[PrevQueryIndex], ESelectInfo::Type::OnMouseClick);
				CurrentPlaybackIndex = PrevQueryIndex;
			}
			break;	
		}
	case EChaosVDPlaybackButtonsID::Play:
	case EChaosVDPlaybackButtonsID::Pause:
	case EChaosVDPlaybackButtonsID::Stop:
	default:
		break;
	}
}

void SChaosVDSceneQueryBrowser::HandlePlaybackQueryIndexUpdated(int32 NewIndex)
{
	CurrentPlaybackIndex = NewIndex;
	if (FilteredCachedTreeItems->IsValidIndex(CurrentPlaybackIndex))
	{
		SelectSceneQuery((*FilteredCachedTreeItems)[CurrentPlaybackIndex], ESelectInfo::Type::OnMouseClick);
	}
}

FText SChaosVDSceneQueryBrowser::GetPlaybackQueryControlText() const
{
	return LOCTEXT("SceneQueryBrowserPlaybackControlsLabel", "Recorded Scene Queries");
}

int32 SChaosVDSceneQueryBrowser::GetCurrentMinPlaybackQueryIndex() const
{
	return 0;
}

int32 SChaosVDSceneQueryBrowser::GetCurrentMaxPlaybackQueryIndex() const
{
	return FilteredCachedTreeItems ? FMath::Clamp(FilteredCachedTreeItems->Num() - 1, 0, TNumericLimits<int32>::Max()) : 0;
}

int32 SChaosVDSceneQueryBrowser::GetCurrentPlaybackQueryIndex() const
{
	return CurrentPlaybackIndex;
}

TSharedPtr<FChaosVDSceneQueryTreeItem> SChaosVDSceneQueryBrowser::MakeSceneQueryTreeItem(const TSharedPtr<FChaosVDQueryDataWrapper>& InQueryData, const UChaosVDSceneQueryDataComponent* DataComponent)
{
	if (!InQueryData || !DataComponent)
	{
		return nullptr;
	}

	if (TSharedPtr<FChaosVDSceneQueryTreeItem>* FoundItemPtr = CachedTreeItemsByID.Find(HashCombineFast(InQueryData->ID, InQueryData->WorldSolverID)))
	{
		return *FoundItemPtr;
	}

	TSharedPtr<FChaosVDSceneQueryTreeItem> NewTreeItem = MakeShared<FChaosVDSceneQueryTreeItem>();
	NewTreeItem->ItemWeakPtr = InQueryData;

	for (int32 SubQueryID : InQueryData->SubQueriesIDs)
	{
		if (TSharedPtr<FChaosVDSceneQueryTreeItem> NewSubTreeItem = MakeSceneQueryTreeItem(DataComponent->GetQueryByID(SubQueryID), DataComponent))
		{
			NewTreeItem->SubItems.Add(NewSubTreeItem);
		}
	}

	NewTreeItem->QueryID = InQueryData->ID;
	NewTreeItem->OwnerSolverID = InQueryData->WorldSolverID;

	if (AChaosVDSolverInfoActor* SolverInfo = Cast<AChaosVDSolverInfoActor>(DataComponent->GetOwner()))
	{
		NewTreeItem->OwnerSolverName = SolverInfo->GetSolverName();
	}
	
	CachedTreeItemsByID.Add(HashCombineFast(InQueryData->ID, InQueryData->WorldSolverID), NewTreeItem);

	return NewTreeItem;
}

bool SChaosVDSceneQueryBrowser::IsQueryDataVisible(const TSharedRef<FChaosVDQueryDataWrapper>& InQueryData)
{
	bool bIsVisible = false;

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return bIsVisible;
	}

	EChaosVDSceneQueryVisualizationFlags VisualizationFlags = UChaosVDSceneQueriesVisualizationSettings::GetDataVisualizationFlags();

	if (!EnumHasAnyFlags(VisualizationFlags, EChaosVDSceneQueryVisualizationFlags::EnableDraw))
	{
		return bIsVisible;
	}

	switch (InQueryData->Type)
	{
		case EChaosVDSceneQueryType::RayCast:
				bIsVisible = EnumHasAnyFlags(VisualizationFlags, EChaosVDSceneQueryVisualizationFlags::DrawLineTraceQueries);
				break;
		case EChaosVDSceneQueryType::Overlap:
				bIsVisible = EnumHasAnyFlags(VisualizationFlags, EChaosVDSceneQueryVisualizationFlags::DrawOverlapQueries);
				break;
		case EChaosVDSceneQueryType::Sweep:
				bIsVisible = EnumHasAnyFlags(VisualizationFlags, EChaosVDSceneQueryVisualizationFlags::DrawSweepQueries);
				break;
		default:
			break;
	}

	// Only do the more expensive visibility checks if the simple ones passed
	if (bIsVisible)
	{
		if (EnumHasAnyFlags(VisualizationFlags, EChaosVDSceneQueryVisualizationFlags::OnlyDrawSelectedQuery))
		{
			if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
			{
				bIsVisible = SelectionObject->IsDataSelected(InQueryData.ToSharedPtr());
			}
		}

		bIsVisible &= GetCachedSolverVisibility(InQueryData->WorldSolverID);
	}

	return bIsVisible;
}

void SChaosVDSceneQueryBrowser::UpdateAllTreeItemsVisibility()
{
	for (const TSharedPtr<FChaosVDSceneQueryTreeItem>& TreeItem : UnfilteredCachedTreeItems)
	{
		UpdateTreeItemVisibility(TreeItem);
	}
}

void SChaosVDSceneQueryBrowser::UpdateTreeItemVisibility(const TSharedPtr<FChaosVDSceneQueryTreeItem>& InTreeItem)
{
	if (TSharedPtr<FChaosVDQueryDataWrapper> QueryData = InTreeItem ? InTreeItem->ItemWeakPtr.Pin() : nullptr)
	{
		InTreeItem->bIsVisible = IsQueryDataVisible(QueryData.ToSharedRef());

		for (const TSharedPtr<FChaosVDSceneQueryTreeItem>& TreeItem : InTreeItem->SubItems)
		{
			UpdateTreeItemVisibility(TreeItem);
		}
	}
}

void SChaosVDSceneQueryBrowser::HandleTreeItemSelected(const TSharedPtr<FChaosVDSceneQueryTreeItem>& SelectedTreeItem, ESelectInfo::Type Type)
{
	// Only handle this selection event if it came from the UI
	if (Type == ESelectInfo::Type::Direct)
	{
		return;
	}

	if (!SelectedTreeItem)
	{
		return;
	}

	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			TSharedPtr<FChaosVDSolverDataSelectionHandle> SelectionHandle = SelectionObject->MakeSelectionHandle(SelectedTreeItem->ItemWeakPtr.Pin());
			SelectionObject->SelectData(SelectionHandle);

			if (TSharedPtr<FEditorModeTools> EditorModeToolsPtr = EditorModeToolsWeakPtr.Pin())
			{
				// Bring the constraint details tab into focus if available
				if (TSharedPtr<SChaosVDMainTab> ToolkitHost = StaticCastSharedPtr<SChaosVDMainTab>(EditorModeToolsPtr->GetToolkitHost()))
				{
					if (const TSharedPtr<FTabManager> TabManager = ToolkitHost->GetTabManager())
					{
						TabManager->TryInvokeTab(FChaosVDTabID::SceneQueryDataDetails);
					}
				}

				// Make sure the viewport is re-drawn so the selection feedback is shown
				//TODO: This likely needs to be done automatically when any selection event is triggered
				if (FEditorViewportClient* ViewportClient = EditorModeToolsPtr->GetFocusedViewportClient())
				{
					ViewportClient->Invalidate();
				}
			}
		}
	}
}

void SChaosVDSceneQueryBrowser::HandleTreeItemFocused(const TSharedPtr<FChaosVDSceneQueryTreeItem>& FocusedTreeItem)
{
	if (!FocusedTreeItem)
	{
		return;
	}
	
	TSharedPtr<FChaosVDQueryDataWrapper> QueryData = FocusedTreeItem->ItemWeakPtr.Pin();
	
	if (!QueryData)
	{
		return;
	}
	
	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return;
	}

	TSharedPtr<FChaosVDRecording> RecordedData = ScenePtr->GetLoadedRecording();
	if (!RecordedData)
	{
		return;
	}

	return ScenePtr->OnFocusRequest().Broadcast(Chaos::VisualDebugger::Utils::CalculateSceneQueryShapeBounds(QueryData.ToSharedRef(), RecordedData.ToSharedRef()));
}

void SChaosVDSceneQueryBrowser::ApplyFilterToData(TConstArrayView<TSharedPtr<FChaosVDSceneQueryTreeItem>> InDataSource, const TSharedRef<TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>>& OutFilteredData)
{
	OutFilteredData->Reset(InDataSource.Num());

	if (CurrentTextFilter.IsEmpty())
	{
		*OutFilteredData = InDataSource;
	}
	else
	{
		FString CurrentFilterAsString = CurrentTextFilter.ToString();

		for (const TSharedPtr<FChaosVDSceneQueryTreeItem>& QueryDataItem : InDataSource)
		{
			TSharedPtr<FChaosVDQueryDataWrapper> QueryData = QueryDataItem ? QueryDataItem->ItemWeakPtr.Pin() : nullptr;
			if (QueryData)
			{
				// TODO: Add support for fuzzy search?. We will likely need to make the search an async operation
				if (GetCachedStringFromName(QueryData->CollisionQueryParams.TraceTag)->Contains(CurrentFilterAsString) ||
					GetCachedStringFromName(QueryData->CollisionQueryParams.OwnerTag)->Contains(CurrentFilterAsString) ||
					GetCachedStringFromName(QueryDataItem->OwnerSolverName)->Contains(CurrentFilterAsString))
				{
					OutFilteredData->Add(QueryDataItem);
				}
			}
		}
	}
}

TSharedRef<FString> SChaosVDSceneQueryBrowser::GetCachedStringFromName(FName Name)
{
	if (TSharedPtr<FString>* FoundStringPtr = CachedNameToStringMap.Find(Name))
	{
		return FoundStringPtr->ToSharedRef();
	}
	else
	{
		TSharedPtr<FString> NameAsString = MakeShared<FString>(Name.ToString());
		CachedNameToStringMap.Add(Name, NameAsString);

		return NameAsString.ToSharedRef();
	}
}

bool SChaosVDSceneQueryBrowser::GetCachedSolverVisibility(int32 SolverID)
{
	if (const bool* bIsSolverVisiblePtr = CachedSolverVisibilityByID.Find(SolverID))
	{
		return *bIsSolverVisiblePtr;
	}

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return false;
	}
	
	if (AChaosVDSolverInfoActor* OwningSolver = ScenePtr->GetSolverInfoActor(SolverID))
	{
		bool bIsVisible = OwningSolver->IsVisible();

		CachedSolverVisibilityByID.Add(SolverID, bIsVisible);
		
		return bIsVisible;
	}

	return false;
}


#undef LOCTEXT_NAMESPACE

