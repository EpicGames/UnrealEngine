// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "SChaosVDGameFramesPlaybackControls.h"
#include "Settings/ChaosVDSceneQueryVisualizationSettings.h"
#include "Widgets/SCompoundWidget.h"

#include "SChaosVDSceneQueryBrowser.generated.h"

class FEditorModeTools;
class SChaosVDSceneQueryTree;
class UChaosVDSceneQueryDataComponent;

struct FChaosVDQueryDataWrapper;
struct FChaosVDSceneQueryTreeItem;
struct FChaosVDSolverDataSelectionHandle;

UCLASS()
class UChaosVDSceneQueryBrowserToolbarMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<class SChaosVDSceneQueryBrowser> BrowserInstance;
};

/**
 * Widget class of the Scene Query Browser window, where all available scene queries are shown for the currently visualized frame
 * in a Scene Outliner kind of way
 */
class SChaosVDSceneQueryBrowser : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChaosVDSceneQueryBrowser)
		{
		}


	SLATE_END_ARGS()

	virtual ~SChaosVDSceneQueryBrowser() override;

	void Construct(const FArguments& InArgs, TWeakPtr<FChaosVDScene> Scene, TWeakPtr<FEditorModeTools> EditorModeTools);
	void RegisterSceneEvents();
	void UnregisterSceneEvents();

	static inline const FName ToolBarName = FName("ChaosVD.SceneQueryBrowser.ToolBar");

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	void SelectSceneQuery(const TSharedPtr<const FChaosVDQueryDataWrapper>& InQuery, ESelectInfo::Type Type);
	void SelectSceneQuery(const TSharedPtr<FChaosVDSceneQueryTreeItem>& SceneQueryTreeItem, ESelectInfo::Type Type);
	void HandleExternalSelectionEvent(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle);
	void HandleSolverVisibilityChanged(int32 SolverID, bool bNewVisibility);

	void RegisterMainToolbarMenu();

	TSharedRef<SWidget> GenerateMainToolbarWidget();
	TSharedRef<SWidget> GenerateSearchBarWidget();
	TSharedRef<SWidget> GenerateQueryVisualizationModeWidget();

	EVisibility GetUpdatesPausedMessageVisibility() const;

	void HandleSearchTextChanged(const FText& NewText);
	
	void HandleSettingsChanged(UObject* SettingsObject);

	void ApplySettingsChange();

	void UpdateBrowserContents();

	void HandleSceneUpdated();

	bool CanUpdate() const;

	bool GetQueryTreeWidgetEnabled() const;
	
	FText GetFilterStatusText() const;
	FSlateColor GetFilterStatusTextColor() const;

	TSharedRef<SWidget> GenerateQueriesPlaybackControls();

	bool GetPlaybackControlsEnabled() const;
	
	void HandlePlaybackControlInput(EChaosVDPlaybackButtonsID InputID);

	void HandlePlaybackQueryIndexUpdated(int32 NewIndex);

	FText GetPlaybackQueryControlText() const;
	int32 GetCurrentMinPlaybackQueryIndex() const;
	int32 GetCurrentMaxPlaybackQueryIndex() const;
	int32 GetCurrentPlaybackQueryIndex() const;

	TSharedPtr<FChaosVDSceneQueryTreeItem> MakeSceneQueryTreeItem(const TSharedPtr<FChaosVDQueryDataWrapper>& InQueryData, const UChaosVDSceneQueryDataComponent* DataComponent);

	bool IsQueryDataVisible(const TSharedRef<FChaosVDQueryDataWrapper>& InQueryData);

	void UpdateAllTreeItemsVisibility();
	void UpdateTreeItemVisibility(const TSharedPtr<FChaosVDSceneQueryTreeItem>& InTreeItem);
	
	void HandleTreeItemSelected(const TSharedPtr<FChaosVDSceneQueryTreeItem>& SelectedTreeItem, ESelectInfo::Type Type);
	void HandleTreeItemFocused(const TSharedPtr<FChaosVDSceneQueryTreeItem>& FocusedTreeItem);

	void ApplyFilterToData(TConstArrayView<TSharedPtr<FChaosVDSceneQueryTreeItem>> InDataSource, const TSharedRef<TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>>& OutFilteredData);

	TSharedRef<FString> GetCachedStringFromName(FName Name);

	bool GetCachedSolverVisibility(int32 SolverID);
	
	TSharedPtr<SChaosVDTimelineWidget> PlaybackControlsTimelineWidget;
	
	TWeakPtr<FChaosVDScene> SceneWeakPtr;
	
	TWeakPtr<FEditorModeTools> EditorModeToolsWeakPtr;

	TWeakPtr<SChaosVDSceneQueryTree> SceneQueryTreeWidget;

	TSharedPtr<TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>> FilteredCachedTreeItems;
	TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>> UnfilteredCachedTreeItems;
	TMap<uint32, TSharedPtr<FChaosVDSceneQueryTreeItem>> CachedTreeItemsByID;
	TMap<int32, bool> CachedSolverVisibilityByID;

	TMap<FName, TSharedPtr<FString>> CachedNameToStringMap;
	
	EChaosVDSQFrameVisualizationMode CurrentVisualizationMode = EChaosVDSQFrameVisualizationMode::AllEnabledQueries;

	FText CurrentTextFilter;

	int32 CurrentPlaybackIndex = 0;
	
	bool bIsUpToDate = false;
	bool bNeedsToUpdateSettings = true;
};
