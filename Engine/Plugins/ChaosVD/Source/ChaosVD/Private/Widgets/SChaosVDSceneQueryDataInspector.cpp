// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDSceneQueryDataInspector.h"

#include "ChaosVDModule.h"
#include "ChaosVDScene.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "IStructureDetailsView.h"
#include "PropertyEditorModule.h"
#include "Widgets/SChaosVDMainTab.h"
#include "Actors/ChaosVDSolverInfoActor.h"
#include "Components/ChaosVDSceneQueryDataComponent.h"
#include "Modules/ModuleManager.h"
#include "Toolkits/IToolkitHost.h"
#include "Visualizers/ChaosVDSceneQueryDataComponentVisualizer.h"
#include "Widgets/SChaosVDNameListPicker.h"
#include "Widgets/SChaosVDTimelineWidget.h"
#include "Widgets/SChaosVDWarningMessageBox.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/Layout/SUniformGridPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

SChaosVDSceneQueryDataInspector::SChaosVDSceneQueryDataInspector() : CurrentSceneQueryBeingInspectedHandle(MakeShared<FChaosVDSolverDataSelectionHandle>())
{
	
}

void SChaosVDSceneQueryDataInspector::RegisterSceneEvents()
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->OnSceneUpdated().AddRaw(this, &SChaosVDSceneQueryDataInspector::HandleSceneUpdated);
		if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			SelectionObject->GetDataSelectionChangedDelegate().AddRaw(this, &SChaosVDSceneQueryDataInspector::SetQueryDataToInspect);
		}
	}
}

void SChaosVDSceneQueryDataInspector::UnregisterSceneEvents()
{
	if (const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin())
	{
		ScenePtr->OnSceneUpdated().RemoveAll(this);

		if (TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin())
		{
			SelectionObject->GetDataSelectionChangedDelegate().RemoveAll(this);
		}
	}
}

SChaosVDSceneQueryDataInspector::~SChaosVDSceneQueryDataInspector()
{
	UnregisterSceneEvents();
}

void SChaosVDSceneQueryDataInspector::Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene>& InScenePtr, const TSharedRef<SChaosVDMainTab>& InMainTab)
{
	SceneWeakPtr = InScenePtr;
	EditorModeToolsWeakPtr = InMainTab->GetEditorModeManager().AsWeak();
	MainTabWeakPtr = InMainTab;

	RegisterSceneEvents();

	SceneQueryDataDetailsView = CreateDataDetailsView();
	SceneQueryHitDataDetailsView = CreateDataDetailsView();

	constexpr float NoPadding = 0.0f;
	constexpr float OuterBoxPadding = 2.0f;
	constexpr float OuterInnerPadding = 5.0f;
	constexpr float TagTitleBoxHorizontalPadding = 10.0f;
	constexpr float TagTitleBoxVerticalPadding = 5.0f;
	constexpr float InnerDetailsPanelsHorizontalPadding = 15.0f;
	constexpr float InnerDetailsPanelsVerticalPadding = 15.0f;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(OuterInnerPadding)
		[
			SNew(SBox)
			.Visibility_Raw(this, &SChaosVDSceneQueryDataInspector::GetOutOfDateWarningVisibility)
			.Padding(OuterBoxPadding, OuterBoxPadding,OuterBoxPadding,NoPadding)
			[
				SNew(SChaosVDWarningMessageBox)
				.WarningText(LOCTEXT("SceneQueryDataOutOfData", "Scene change detected!. Selected scene query data is out of date..."))
			]
		]
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(TagTitleBoxHorizontalPadding, TagTitleBoxVerticalPadding, TagTitleBoxHorizontalPadding, NoPadding)
		[
			GenerateQueryTagInfoRow()
		]
		+SVerticalBox::Slot()
		.Padding(TagTitleBoxHorizontalPadding, TagTitleBoxVerticalPadding, TagTitleBoxHorizontalPadding, NoPadding)
		.AutoHeight()
		[
			SNew(STextBlock)
			.Visibility_Raw(this, &SChaosVDSceneQueryDataInspector::GetNothingSelectedMessageVisibility)
			.Justification(ETextJustify::Center)
			.TextStyle(FAppStyle::Get(), "DetailsView.BPMessageTextStyle")
			.Text(LOCTEXT("SceneQueryDataNoSelectedMessage", "Select a scene query or scene query hit in the viewport to see its details..."))
			.AutoWrapText(true)
		]
		+SVerticalBox::Slot()
		.Padding(TagTitleBoxHorizontalPadding, TagTitleBoxVerticalPadding, TagTitleBoxHorizontalPadding, NoPadding)
		.AutoHeight()
		[
			GenerateQueryNavigationBoxWidget(TagTitleBoxHorizontalPadding, TagTitleBoxVerticalPadding)
		]
		+SVerticalBox::Slot()
		.Padding(OuterInnerPadding)
		.FillHeight(0.75f)
		[
			GenerateQueryDetailsPanelSection(InnerDetailsPanelsHorizontalPadding, InnerDetailsPanelsVerticalPadding)
		]
		+SVerticalBox::Slot()
		.FillHeight(0.1f)
		[
			GenerateVisitStepControls()
		]
	];
}

TSharedRef<SWidget> SChaosVDSceneQueryDataInspector::GenerateQueryNavigationBoxWidget(float TagTitleBoxHorizontalPadding, float TagTitleBoxVerticalPadding)
{
	return SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(SUniformGridPanel)
				.Visibility_Raw(this, &SChaosVDSceneQueryDataInspector::GetParentQuerySelectorVisibility)
				.SlotPadding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
				.MinDesiredSlotWidth(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotWidth"))
				.MinDesiredSlotHeight(FAppStyle::GetFloat("StandardDialog.MinDesiredSlotHeight"))
				+SUniformGridPanel::Slot(0, 0)
				[
					SNew(SButton)
					.HAlign(HAlign_Center)
					.VAlign(VAlign_Center)
					.ContentPadding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					.Text(LOCTEXT("SelectParentQuery", "Go to parent query"))
					.OnClicked_Raw(this, &SChaosVDSceneQueryDataInspector::SelectParentQuery)
				]
			]
			+SVerticalBox::Slot()
			.Padding(FMargin(TagTitleBoxHorizontalPadding, TagTitleBoxVerticalPadding, TagTitleBoxHorizontalPadding, 0.0f))
			[
				SNew(SHorizontalBox)
				.Visibility_Raw(this, &SChaosVDSceneQueryDataInspector::GetSubQuerySelectorVisibility)
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.VAlign(VAlign_Center)
				[
					SNew(STextBlock)
					.Text(LOCTEXT("SelectSubQueryDropDown", "Go To Subquery"))
					.Font(FCoreStyle::GetDefaultFontStyle("Regular", 8))
				]
				+SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.VAlign(VAlign_Center)
				[
					SAssignNew(SubQueryNamePickerWidget, SChaosVDNameListPicker)
					.OnNameSleceted_Raw(this, &SChaosVDSceneQueryDataInspector::HandleSubQueryNameSelected)
				]	
			];
}

TSharedRef<SWidget> SChaosVDSceneQueryDataInspector::GenerateQueryDetailsPanelSection(float InnerDetailsPanelsHorizontalPadding, float InnerDetailsPanelsVerticalPadding)
{
	return SNew(SScrollBox)
			.Visibility_Raw(this, &SChaosVDSceneQueryDataInspector::GetQueryDetailsSectionVisibility)
			+SScrollBox::Slot()
			.Padding(InnerDetailsPanelsHorizontalPadding,0.0f,InnerDetailsPanelsHorizontalPadding,InnerDetailsPanelsVerticalPadding)
			[
				SceneQueryDataDetailsView->GetWidget().ToSharedRef()
			]
			+SScrollBox::Slot()
			.Padding(InnerDetailsPanelsHorizontalPadding,0.0f,InnerDetailsPanelsHorizontalPadding,0.0f)
			[
				SNew(SVerticalBox)
				.Visibility_Raw(this, &SChaosVDSceneQueryDataInspector::GetSQVisitDetailsSectionVisibility)
				+SVerticalBox::Slot()
				[
					SceneQueryHitDataDetailsView->GetWidget().ToSharedRef()
				]
			];
}

TSharedRef<SWidget> SChaosVDSceneQueryDataInspector::GenerateVisitStepControls()
{
	return SNew(SBorder)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryMiddle"))
			.BorderBackgroundColor(FAppStyle::Get().GetSlateColor("Colors.Panel"))
			[
				SNew(SVerticalBox)
				.Visibility_Raw(this, &SChaosVDSceneQueryDataInspector::GetQueryStepPlaybackControlsVisibility)
				+SVerticalBox::Slot()
				.AutoHeight()
				[
					SNew(SBorder)
					.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
					.BorderBackgroundColor(FLinearColor::White)
					[
						SNew(SBox)
						.VAlign(VAlign_Center)
						.MinDesiredHeight(26.0f)
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.Text_Raw(this, &SChaosVDSceneQueryDataInspector::GetSQVisitsStepsText)
							.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
							.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
						]
					]
				]
				+SVerticalBox::Slot()
				.Padding(0.0f, 6.0f, 0.0f, 2.0f)
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.FillWidth(0.9)
					[
						SAssignNew(QueryStepsTimelineWidget, SChaosVDTimelineWidget)
						.ButtonVisibilityFlags(EChaosVDTimelineElementIDFlags::AllManualStepping)
						.IsEnabled_Raw(this, &SChaosVDSceneQueryDataInspector::GetSQVisitStepsEnabled)
						.OnFrameChanged_Raw(this, &SChaosVDSceneQueryDataInspector::HandleQueryStepSelectionUpdated)
						.OnButtonClicked(this, &SChaosVDSceneQueryDataInspector::HandleSQVisitTimelineInput)
						.MinFrames_Raw(this, &SChaosVDSceneQueryDataInspector::GetCurrentMinSQVisitIndex)
						.MaxFrames_Raw(this, &SChaosVDSceneQueryDataInspector::GetCurrentMaxSQVisitIndex)
						.CurrentFrame_Raw(this, &SChaosVDSceneQueryDataInspector::GetCurrentSQVisitIndex)
					]
					+SHorizontalBox::Slot()
					.AutoWidth()
					[
						SNew(SButton)
						.OnClicked_Raw(this, &SChaosVDSceneQueryDataInspector::SelectParticleForCurrentQueryData)
						.ToolTipText(LOCTEXT("SelectSQVisitToolTip","Selects the current visited particle and collision shape"))
						[
							SNew(SImage)
							.Image(FAppStyle::GetBrush("GenericCommands.SelectAll"))
							.DesiredSizeOverride(FVector2D(16.0f,16.0f))
							.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
				]
			];
}

TSharedRef<SWidget> SChaosVDSceneQueryDataInspector::GenerateQueryTagInfoRow()
{
	return SNew(SBorder)
			.Padding(0.5f)
			.BorderImage(FAppStyle::Get().GetBrush("DetailsView.CategoryTop"))
			.BorderBackgroundColor(FLinearColor::White)
			[
				SNew(SBox)
				.MinDesiredHeight(26.0f)
				.VAlign(VAlign_Center)
				.Padding(2.0f)
				[
					SNew(STextBlock)
					.Text_Raw(this, &SChaosVDSceneQueryDataInspector::GetQueryBeingInspectedTag)
					.Font(FAppStyle::GetFontStyle("DetailsView.CategoryFontStyle"))
					.TextStyle(FAppStyle::Get(), "DetailsView.CategoryTextStyle")
				]
			];
}

void SChaosVDSceneQueryDataInspector::SetQueryDataToInspect(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle)
{
	ClearInspector();

	if (!InDataSelectionHandle)
	{
		return;
	}

	if (const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataToInspect = InDataSelectionHandle->GetDataAsShared<FChaosVDQueryDataWrapper>())
	{
		CurrentSceneQueryBeingInspectedHandle = InDataSelectionHandle.ToSharedRef();

		const TSharedPtr<FStructOnScope> QueryDataView = MakeShared<FStructOnScope>(FChaosVDQueryDataWrapper::StaticStruct(), reinterpret_cast<uint8*>(QueryDataToInspect.Get()));
		SceneQueryDataDetailsView->SetStructureData(QueryDataView);

		if(FChaosVDSceneQuerySelectionContext* SelectionContext = InDataSelectionHandle->GetContextData<FChaosVDSceneQuerySelectionContext>())
		{
			if (QueryDataToInspect->SQVisitData.IsValidIndex(SelectionContext->SQVisitIndex))
			{
				const TSharedPtr<FStructOnScope> SQVisitDataDataView = MakeShared<FStructOnScope>(FChaosVDQueryVisitStep::StaticStruct(), reinterpret_cast<uint8*>(&QueryDataToInspect->SQVisitData[SelectionContext->SQVisitIndex]));
				SceneQueryHitDataDetailsView->SetStructureData(SQVisitDataDataView);
				QueryDataToInspect->CurrentVisitIndex = SelectionContext->SQVisitIndex;

				FReply SelectionResponse = SelectParticleForCurrentQueryData();
				if (!SelectionResponse.IsEventHandled())
				{
					UE_LOGF(LogChaosVDEditor, Warning, "[%s] Failed to auto select visited particle", __func__);
				}
			}
		}

		if (QueryDataToInspect->SubQueriesIDs.Num() > 0)
		{
			TArray<TSharedPtr<FName>> NewSubQueryNameList;
			NewSubQueryNameList.Reserve(QueryDataToInspect->SubQueriesIDs.Num());
			Algo::Transform(QueryDataToInspect->SubQueriesIDs, NewSubQueryNameList, [this, &QueryDataToInspect](int32 QueryID)
			{
				TSharedPtr<FName> NewName = MakeShared<FName>(FString::Format(TEXT("Query ID {0}"), {QueryID}));
				CurrentSubQueriesByName.Add(NewName, {QueryID, QueryDataToInspect->WorldSolverID});
				return NewName;
			});

			SubQueryNamePickerWidget->UpdateNameList(MoveTemp(NewSubQueryNameList));
		}
		else
		{
			CurrentSubQueriesByName.Empty();
			SubQueryNamePickerWidget->UpdateNameList({});
		}
	}
	else
	{
		ClearInspector();	
	}

	bIsUpToDate = true;
}

void SChaosVDSceneQueryDataInspector::HandleQueryStepSelectionUpdated(int32 NewStepIndex)
{
	if (!bListenToSelectionEvents)
	{
		return;
	}

	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	if (!QueryDataBeingInspected)
	{
		ClearInspector();
		return;
	}

	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		ClearInspector();
		return;
	}

	TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin();

	if (!SelectionObject)
	{
		ClearInspector();
		return;
	}

	AChaosVDSolverInfoActor* SolverInfoActor = ScenePtr->GetSolverInfoActor(QueryDataBeingInspected->WorldSolverID);
	UChaosVDSceneQueryDataComponent* SQDataComponent = SolverInfoActor ? SolverInfoActor->GetSceneQueryDataComponent() : nullptr;
	if (!SQDataComponent)
	{
		ClearInspector();
		return;
	}

	// If we reach this point no need to clear the inspector, we have valid data. This is can be caused by the timeline widget going over as we no longer restrict th button actions
	if (!QueryDataBeingInspected->SQVisitData.IsValidIndex(NewStepIndex))
	{
		UE_LOGF(LogChaosVDEditor, VeryVerbose, "[%ls] Attempted to process and invalid SQ Visit index | Input Index [%d] | Available SQ Visit Data Num [%d]", ANSI_TO_TCHAR(__FUNCTION__), NewStepIndex, QueryDataBeingInspected->SQVisitData.Num())
		return;
	}

	TSharedPtr<FChaosVDSolverDataSelectionHandle> NewSelection = SelectionObject->MakeSelectionHandle(QueryDataBeingInspected);
	FChaosVDSceneQuerySelectionContext ContextData;
	ContextData.SQVisitIndex = NewStepIndex;
	NewSelection->SetHandleContext(MoveTemp(ContextData));

	QueryDataBeingInspected->CurrentVisitIndex = NewStepIndex;

	FScopedSQInspectorSilencedSelectionEvents IgnoreSelectionEventsScope(*this);
	SelectionObject->SelectData(NewSelection);

	FReply SelectionResponse = SelectParticleForCurrentQueryData();
	if (!SelectionResponse.IsEventHandled())
	{
		UE_LOGF(LogChaosVDEditor, Warning, "[%s] Failed to auto select visited particle", __func__);
	}

	if (const TSharedPtr<FEditorModeTools> EditorModeToolsPtr = EditorModeToolsWeakPtr.Pin())
	{
		// We need to request a re-draw to make sure the debug draw view and selection outline is updated
		if (FEditorViewportClient* ViewportClient = EditorModeToolsPtr->GetFocusedViewportClient())
		{
			ViewportClient->bNeedsRedraw = true;
		}
	}
}

FText SChaosVDSceneQueryDataInspector::GetQueryBeingInspectedTag() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	const FText QueryTag = FText::AsCultureInvariant(QueryDataBeingInspected ? QueryDataBeingInspected->CollisionQueryParams.TraceTag.ToString() : TEXT("None"));
	return FText::Format(FTextFormat(LOCTEXT("SceneQueriesNameLabel", "Query Tag | {0}")), QueryTag);
}

FText SChaosVDSceneQueryDataInspector::GetSQVisitsStepsText() const
{
	return LOCTEXT("SQVisitStepsPlaybackControlsTitle", "Visited Particle Shapes");
}

FReply SChaosVDSceneQueryDataInspector::SelectParticleForCurrentQueryData() const
{
	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return FReply::Handled();
	}

	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();

	if (!QueryDataBeingInspected || !QueryDataBeingInspected->SQVisitData.IsValidIndex(QueryDataBeingInspected->CurrentVisitIndex))
	{
		return FReply::Handled();
	}

	if (TSharedPtr<FChaosVDSceneParticle> ParticleActor = ScenePtr->GetParticleInstance(QueryDataBeingInspected->WorldSolverID, QueryDataBeingInspected->SQVisitData[QueryDataBeingInspected->CurrentVisitIndex].ParticleIndex))
	{
		const int32 ShapeInstanceIndexToSelect = QueryDataBeingInspected->SQVisitData[QueryDataBeingInspected->CurrentVisitIndex].ShapeIndex;

		// TODO : This will not work properly when the visited shape was a Union within a union
		// CVD currently doesn't support multi selection, so we can't easily select all mesh instances that represent a union within a union
		// We need to revisit this when multi selection support is added. Jira for tracking UE-212733
		const TConstArrayView<TSharedRef<FChaosVDInstancedMeshData>> AvailableMeshInstances = ParticleActor->GetMeshInstances();
		for (const TSharedRef<FChaosVDInstancedMeshData>& MeshInstance : AvailableMeshInstances)
		{
			if (MeshInstance->GetState().ImplicitObjectInfo.ShapeInstanceIndex == ShapeInstanceIndexToSelect)
			{
				Chaos::VisualDebugger::SelectParticleWithGeometryInstance(ScenePtr.ToSharedRef(), ParticleActor.Get(), MeshInstance);
				break;
			}
		}
		
		if (const TSharedPtr<FEditorModeTools> EditorModeToolsPtr = EditorModeToolsWeakPtr.Pin())
		{
			// We need to request a re-draw to make sure the debug draw view is updated
			if (FEditorViewportClient* ViewportClient = EditorModeToolsPtr->GetFocusedViewportClient())
			{
				ViewportClient->bNeedsRedraw = true;
			}
		}
	}
	
	return FReply::Handled();
}

FReply SChaosVDSceneQueryDataInspector::SelectQueryToInspectByID(int32 QueryID, int32 SolverID)
{
	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return FReply::Handled();
	}
	
	TSharedPtr<FChaosVDSolverDataSelection> SelectionObject = ScenePtr->GetSolverDataSelectionObject().Pin();
	if (!SelectionObject)
	{
		return FReply::Handled();
	}

	AChaosVDSolverInfoActor* SolverInfoActor = ScenePtr->GetSolverInfoActor(SolverID);
	if (UChaosVDSceneQueryDataComponent* SQDataComponent = SolverInfoActor ? SolverInfoActor->GetSceneQueryDataComponent() : nullptr)
	{						
		SelectionObject->SelectData(SelectionObject->MakeSelectionHandle(SQDataComponent->GetQueryByID(QueryID)));
	}
	else
	{
		ClearInspector();	
	}
						
	return FReply::Handled();
}

FReply SChaosVDSceneQueryDataInspector::SelectParentQuery()
{
	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();

	if (const TSharedPtr<FChaosVDQueryDataWrapper> SelectedQuery = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>())
	{		
		SelectQueryToInspectByID(SelectedQuery->ParentQueryID, SelectedQuery->WorldSolverID);
	}
	else
	{
		ClearInspector();
	}

	return FReply::Handled();
}

TSharedPtr<IStructureDetailsView> SChaosVDSceneQueryDataInspector::CreateDataDetailsView() const
{
	TSharedPtr<SChaosVDMainTab> MainTabPtr = MainTabWeakPtr.Pin();
	if (!MainTabPtr)
	{
		return nullptr;
	}

	const FStructureDetailsViewArgs StructDetailsViewArgs;
	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bShowOptions = false;
	DetailsViewArgs.bAllowFavoriteSystem = false;
	DetailsViewArgs.bAllowSearch = false;
	DetailsViewArgs.bShowScrollBar = false;

	return MainTabPtr->CreateStructureDetailsView(DetailsViewArgs,StructDetailsViewArgs, nullptr);
}

void SChaosVDSceneQueryDataInspector::HandleSceneUpdated()
{
	if (GetCurrentDataBeingInspected())
	{
		bIsUpToDate = false;
	}
	else
	{
		ClearInspector();
	}
}

void SChaosVDSceneQueryDataInspector::HandleSubQueryNameSelected(TSharedPtr<FName> Name)
{
	const TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return;
	}

	if (const FChaosVDSQSubQueryID* SelectedQueryID = CurrentSubQueriesByName.Find(Name))
	{
		SelectQueryToInspectByID(SelectedQueryID->QueryID, SelectedQueryID->SolverID);
		return;
	}

	ClearInspector();

	UE_LOGF(LogChaosVDEditor, Error,"[%ls] Failed to find selected subquery.", ANSI_TO_TCHAR(__FUNCTION__));
}

void SChaosVDSceneQueryDataInspector::ClearInspector()
{
	if (SceneQueryDataDetailsView && SceneQueryDataDetailsView->GetStructureProvider())
	{
		SceneQueryDataDetailsView->SetStructureData(nullptr);
	}

	if (SceneQueryHitDataDetailsView && SceneQueryHitDataDetailsView->GetStructureProvider())
	{
		SceneQueryHitDataDetailsView->SetStructureData(nullptr);
	}

	if (SubQueryNamePickerWidget)
	{
		SubQueryNamePickerWidget->UpdateNameList({});
	}

	CurrentSubQueriesByName.Empty();

	CurrentSceneQueryBeingInspectedHandle = MakeShared<FChaosVDSolverDataSelectionHandle>();

	bIsUpToDate = true;
}

EVisibility SChaosVDSceneQueryDataInspector::GetOutOfDateWarningVisibility() const
{
	return bIsUpToDate ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SChaosVDSceneQueryDataInspector::GetQueryDetailsSectionVisibility() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	return QueryDataBeingInspected ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SChaosVDSceneQueryDataInspector::GetQueryStepPlaybackControlsVisibility() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	// If the this inspector no longer reflects data represented in the viewport, we can't offer playback so we need to hide the controls
	return bIsUpToDate && QueryDataBeingInspected ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SChaosVDSceneQueryDataInspector::GetSQVisitDetailsSectionVisibility() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	return QueryDataBeingInspected && QueryDataBeingInspected->SQVisitData.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SChaosVDSceneQueryDataInspector::GetNothingSelectedMessageVisibility() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	return QueryDataBeingInspected ? EVisibility::Collapsed : EVisibility::Visible;
}

EVisibility SChaosVDSceneQueryDataInspector::GetSubQuerySelectorVisibility() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	return QueryDataBeingInspected && QueryDataBeingInspected->SubQueriesIDs.Num() > 0 ? EVisibility::Visible : EVisibility::Collapsed;
}

EVisibility SChaosVDSceneQueryDataInspector::GetParentQuerySelectorVisibility() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	return QueryDataBeingInspected && QueryDataBeingInspected->ParentQueryID != INDEX_NONE ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SChaosVDSceneQueryDataInspector::GetSelectParticleHitStateEnable() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	if (QueryDataBeingInspected && QueryDataBeingInspected->SQVisitData.IsValidIndex(QueryDataBeingInspected->CurrentVisitIndex))
	{
		return QueryDataBeingInspected->SQVisitData[QueryDataBeingInspected->CurrentVisitIndex].ParticleIndex != INDEX_NONE;
	}

	return false;
}

bool SChaosVDSceneQueryDataInspector::GetSQVisitStepsEnabled() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	return QueryDataBeingInspected && QueryDataBeingInspected->SQVisitData.Num() > 0;
}

TSharedPtr<FChaosVDQueryDataWrapper> SChaosVDSceneQueryDataInspector::GetCurrentDataBeingInspected() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = CurrentSceneQueryBeingInspectedHandle->GetDataAsShared<FChaosVDQueryDataWrapper>();
	return QueryDataBeingInspected;
}

int32 SChaosVDSceneQueryDataInspector::GetCurrentMinSQVisitIndex() const
{
	return 0;
}

int32 SChaosVDSceneQueryDataInspector::GetCurrentMaxSQVisitIndex() const
{
	if (const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = GetCurrentDataBeingInspected())
	{
		const int32 SQVisitsNum = QueryDataBeingInspected->SQVisitData.Num();
		return SQVisitsNum > 0 ? SQVisitsNum -1 : 0;
	}

	return 0;
}

int32 SChaosVDSceneQueryDataInspector::GetCurrentSQVisitIndex() const
{
	const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = GetCurrentDataBeingInspected();
	return  QueryDataBeingInspected ? QueryDataBeingInspected->CurrentVisitIndex : 0;
}

void SChaosVDSceneQueryDataInspector::HandleSQVisitTimelineInput(EChaosVDPlaybackButtonsID InputID)
{
	if (const TSharedPtr<FChaosVDQueryDataWrapper> QueryDataBeingInspected = GetCurrentDataBeingInspected())
	{
		switch (InputID)
		{
		case EChaosVDPlaybackButtonsID::Next:
			{
				HandleQueryStepSelectionUpdated(QueryDataBeingInspected->CurrentVisitIndex +1);
				break;	
			}
		case EChaosVDPlaybackButtonsID::Prev:
			{
				HandleQueryStepSelectionUpdated(QueryDataBeingInspected->CurrentVisitIndex -1);
				break;	
			}
		case EChaosVDPlaybackButtonsID::Play:
		case EChaosVDPlaybackButtonsID::Pause:
		case EChaosVDPlaybackButtonsID::Stop:
		default:
			break;
		}
	}
}

#undef LOCTEXT_NAMESPACE
