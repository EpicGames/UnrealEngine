// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChaosVDSceneQueryDataComponent.h"
#include "Widgets/SCompoundWidget.h"

class SChaosVDMainTab;
enum class EChaosVDPlaybackButtonsID : uint8;
class SChaosVDTimelineWidget;
struct FChaosVDSceneQuerySelectionHandle;
class SChaosVDNameListPicker;
class FEditorModeTools;
class IToolkitHost;
class SChaosVDTimelineWidget;
struct FChaosVDQueryDataWrapper;
class IStructureDetailsView;
class FChaosVDScene;

struct FChaosVDSQSubQueryID
{
	int32 QueryID = INDEX_NONE;
	int32 SolverID = INDEX_NONE;
};

/**
 * Widget for the Chaos Visual Debugger Scene Queries data inspector
 */
class SChaosVDSceneQueryDataInspector : public SCompoundWidget
{
public:
	SChaosVDSceneQueryDataInspector();
	void RegisterSceneEvents();
	void UnregisterSceneEvents();

	SLATE_BEGIN_ARGS(SChaosVDSceneQueryDataInspector)
		{
		}
	SLATE_END_ARGS()

	virtual ~SChaosVDSceneQueryDataInspector() override;

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene>& InScenePtr, const TSharedRef<SChaosVDMainTab>& InMainTab);

	/** Sets a new query data to be inspected */
	void SetQueryDataToInspect(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle);

protected:

	TSharedRef<SWidget> GenerateQueryTagInfoRow();
	TSharedRef<SWidget> GenerateQueryNavigationBoxWidget(float TagTitleBoxHorizontalPadding, float TagTitleBoxVerticalPadding);
	TSharedRef<SWidget> GenerateQueryDetailsPanelSection(float InnerDetailsPanelsHorizontalPadding, float InnerDetailsPanelsVerticalPadding);
	TSharedRef<SWidget> GenerateVisitStepControls();

	void HandleQueryStepSelectionUpdated(int32 NewStepIndex);
	
	FText GetQueryBeingInspectedTag() const;
	FText GetSQVisitsStepsText() const;
	
	FReply SelectParticleForCurrentQueryData() const;
	FReply SelectQueryToInspectByID(int32 QueryID, int32 SolverID);
	FReply SelectParentQuery();

	TSharedPtr<IStructureDetailsView> CreateDataDetailsView() const;
	
	void HandleSceneUpdated();
	void HandleSubQueryNameSelected(TSharedPtr<FName> Name);

	void ClearInspector();
	
	EVisibility GetOutOfDateWarningVisibility() const;
	EVisibility GetQueryDetailsSectionVisibility() const;
	EVisibility GetQueryStepPlaybackControlsVisibility() const;
	EVisibility GetSQVisitDetailsSectionVisibility() const;
	EVisibility GetNothingSelectedMessageVisibility() const;
	EVisibility GetSubQuerySelectorVisibility() const;
	EVisibility GetParentQuerySelectorVisibility() const;

	bool GetSelectParticleHitStateEnable() const;
	bool GetSQVisitStepsEnabled() const;

	TSharedPtr<FChaosVDQueryDataWrapper> GetCurrentDataBeingInspected() const;

	TSharedPtr<SChaosVDTimelineWidget> QueryStepsTimelineWidget;
	
	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	TSharedPtr<IStructureDetailsView> SceneQueryDataDetailsView;
	
	TSharedPtr<IStructureDetailsView> SceneQueryHitDataDetailsView;

	TSharedPtr<SChaosVDNameListPicker> SubQueryNamePickerWidget;

	TWeakPtr<FEditorModeTools> EditorModeToolsWeakPtr;

	TMap<TSharedPtr<FName>, FChaosVDSQSubQueryID> CurrentSubQueriesByName;
	
	TSharedRef<FChaosVDSolverDataSelectionHandle> CurrentSceneQueryBeingInspectedHandle;

	bool bIsUpToDate = true;

	bool bListenToSelectionEvents = true;

	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr;

	int32 GetCurrentMinSQVisitIndex() const;
	int32 GetCurrentMaxSQVisitIndex() const;
	int32 GetCurrentSQVisitIndex() const;

	void HandleSQVisitTimelineInput(EChaosVDPlaybackButtonsID InputID);

	friend struct FScopedSQInspectorSilencedSelectionEvents;
};

/** Structure that makes a SQ Inspector ignore selection events withing a scope */
struct FScopedSQInspectorSilencedSelectionEvents
{
	FScopedSQInspectorSilencedSelectionEvents(SChaosVDSceneQueryDataInspector& InInspectorIgnoringEvents) : InspectorIgnoringSelectionEvents(InInspectorIgnoringEvents)
	{
		InspectorIgnoringSelectionEvents.bListenToSelectionEvents = false;
	}
	
	~FScopedSQInspectorSilencedSelectionEvents()
	{
		InspectorIgnoringSelectionEvents.bListenToSelectionEvents = true;
	}
	
private:
	SChaosVDSceneQueryDataInspector& InspectorIgnoringSelectionEvents;
};
