// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDSceneQueryTree.h"

#include "SChaosVDSceneQueryTreeRow.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

SChaosVDSceneQueryTree::FColumNames SChaosVDSceneQueryTree::ColumnNames = SChaosVDSceneQueryTree::FColumNames();

void SChaosVDSceneQueryTree::Construct(const FArguments& InArgs)
{
	QueryItemSelectedDelegate = InArgs._OnItemSelected;
	QueryItemFocusedDelegate = InArgs._OnItemFocused;

	constexpr float BottomPadding = 2.0f;
	constexpr float NoPadding = 0.0f;
	constexpr float VerticalPadding = 0.0f;

	static FMargin ColumHeaderTextMargin(NoPadding,VerticalPadding);

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.Padding(NoPadding, NoPadding, NoPadding, BottomPadding)
		[
			SAssignNew(SceneQueriesListWidget, STreeView<TSharedPtr<FChaosVDSceneQueryTreeItem>>)
			.OnGenerateRow(this, &SChaosVDSceneQueryTree::GenerateSceneQueryDataRow)
			.OnSelectionChanged(this, &SChaosVDSceneQueryTree::QueryTreeSelectionChanged)
			.OnGetChildren(this, &SChaosVDSceneQueryTree::OnGetChildrenForQueryItem)
			.TreeItemsSource(&InternalTreeItemSourceData)
			.OnMouseButtonDoubleClick(this, &SChaosVDSceneQueryTree::HandleFocusRequest)
			.SelectionMode(ESelectionMode::Type::Single)
			.HighlightParentNodesForSelection(true)
			.HeaderRow(
						// TODO: Add sort support for the columns
						SNew(SHeaderRow)
						+SHeaderRow::Column(ColumnNames.Visibility)
						.SortMode(EColumnSortMode::None)
						.FixedWidth(24.f)
						.HAlignHeader(HAlign_Left)
						.VAlignHeader(VAlign_Center)
						.HAlignCell(HAlign_Center)
						.VAlignCell(VAlign_Center)
						.HeaderContentPadding(FMargin(4.0f, 0.0f, 0.0f, 0.0f))
						[
							SNew(SImage)
							.ColorAndOpacity(FSlateColor::UseForeground())
							.Image(FAppStyle::Get().GetBrush("Level.VisibleIcon16x"))
						]
						+SHeaderRow::Column(ColumnNames.TraceTag)
						.SortMode(EColumnSortMode::None)
						.HAlignCell(HAlign_Left)
						[
							SNew(STextBlock)
							.Margin(ColumHeaderTextMargin)
							.Text(LOCTEXT("QueryListTagHeader", "Trace Tag"))
						]
						+SHeaderRow::Column(ColumnNames.TraceOwner)
						.SortMode(EColumnSortMode::None)
						.HAlignCell(HAlign_Left)
						[
							SNew(STextBlock)
							.Margin(ColumHeaderTextMargin)
							.Text(LOCTEXT("QueryListOwnerHeader", "Trace Owner"))
						]
						+SHeaderRow::Column(ColumnNames.QueryType)
						.SortMode(EColumnSortMode::None)
						.HAlignCell(HAlign_Left)
						[
							SNew(STextBlock)
							.Margin(ColumHeaderTextMargin)
							.Text(LOCTEXT("QueryListTypeHeader", "Query Type"))
						]
						+SHeaderRow::Column(ColumnNames.SolverName)
						.SortMode(EColumnSortMode::None)
						.HAlignCell(HAlign_Left)
						[
							SNew(STextBlock)
							.Margin(ColumHeaderTextMargin)
							.Text(LOCTEXT("QueryListSolverNameHeader", "Solver Name"))
						]
					)
		]
	];
}

TSharedRef<ITableRow> SChaosVDSceneQueryTree::GenerateSceneQueryDataRow(TSharedPtr<FChaosVDSceneQueryTreeItem> SceneQueryData, const TSharedRef<STableViewBase>& OwnerTable)
{
	if (!SceneQueryData)
	{
		return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("SChaosVDSceneQueryListErrorMessage", "Failed to read data for solver."))
			]
		];
	}

	return SNew(SChaosVDSceneQueryTreeRow, OwnerTable)
			.Item(SceneQueryData);
}

void SChaosVDSceneQueryTree::QueryTreeSelectionChanged(TSharedPtr<FChaosVDSceneQueryTreeItem> SelectedQuery, ESelectInfo::Type Type)
{
	QueryItemSelectedDelegate.ExecuteIfBound(SelectedQuery, Type);
}

void SChaosVDSceneQueryTree::OnGetChildrenForQueryItem(TSharedPtr<FChaosVDSceneQueryTreeItem> QueryEntry, TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>& OutQueries)
{
	OutQueries.Append(QueryEntry->SubItems);
}

void SChaosVDSceneQueryTree::SetExternalSourceData(const TSharedPtr<TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>>& InUpdatedSceneQueryDataSource)
{
	if (ExternalTreeItemSourceData != InUpdatedSceneQueryDataSource)
	{
		ExternalTreeItemSourceData = InUpdatedSceneQueryDataSource;
		SceneQueriesListWidget->SetTreeItemsSource(ExternalTreeItemSourceData.Get());
	}

	for (const TSharedPtr<FChaosVDSceneQueryTreeItem>& TreeItem : *ExternalTreeItemSourceData)
	{
		SceneQueriesListWidget->SetItemExpansion(TreeItem, true);
	}

	SceneQueriesListWidget->RebuildList();
	SceneQueriesListWidget->RequestTreeRefresh();
}

void SChaosVDSceneQueryTree::SelectItem(const TSharedPtr<FChaosVDSceneQueryTreeItem>& ItemToSelect, ESelectInfo::Type Type)
{
	SceneQueriesListWidget->SetSelection(ItemToSelect, Type);
}

void SChaosVDSceneQueryTree::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);
}

void SChaosVDSceneQueryTree::HandleFocusRequest(TSharedPtr<FChaosVDSceneQueryTreeItem> InFocusedItem)
{
	QueryItemFocusedDelegate.ExecuteIfBound(InFocusedItem);
}

#undef LOCTEXT_NAMESPACE
