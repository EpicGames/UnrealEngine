// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"

class ITableRow;
class UChaosVDSceneQueryDataComponent;

struct FChaosVDQueryDataWrapper;

struct FChaosVDSceneQueryTreeItem
{
	TWeakPtr<FChaosVDQueryDataWrapper> ItemWeakPtr;
	TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>> SubItems;
	int32 QueryID = INDEX_NONE;
	FName OwnerSolverName = FName("Invalid");
	int32 OwnerSolverID = INDEX_NONE;
	bool bIsVisible = true;
};

DECLARE_DELEGATE_TwoParams(FChaosVDQueryTreeItemSelected, const TSharedPtr<FChaosVDSceneQueryTreeItem>&, ESelectInfo::Type)
DECLARE_DELEGATE_OneParam(FChaosVDQueryTreeItemFocused, const TSharedPtr<FChaosVDSceneQueryTreeItem>&)

/**
 * Tree Widget used to represent the recorded scene queries hierachy
 */
class SChaosVDSceneQueryTree : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChaosVDSceneQueryTree) {}
		SLATE_EVENT(FChaosVDQueryTreeItemSelected, OnItemSelected)
		SLATE_EVENT(FChaosVDQueryTreeItemFocused, OnItemFocused)
	SLATE_END_ARGS()
	
	void Construct(const FArguments& InArgs);

	void SetExternalSourceData(const TSharedPtr<TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>>& InUpdatedSceneQueryDataSource);

	void SelectItem(const TSharedPtr<FChaosVDSceneQueryTreeItem>& ItemToSelect, ESelectInfo::Type Type);

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

protected:

	void HandleFocusRequest(TSharedPtr<FChaosVDSceneQueryTreeItem> InFocusedItem);

	TSharedRef<ITableRow> GenerateSceneQueryDataRow(TSharedPtr<FChaosVDSceneQueryTreeItem> SceneQueryData, const TSharedRef<STableViewBase>& OwnerTable);

	void QueryTreeSelectionChanged(TSharedPtr<FChaosVDSceneQueryTreeItem> SelectedQuery, ESelectInfo::Type Type);

	void OnGetChildrenForQueryItem(TSharedPtr<FChaosVDSceneQueryTreeItem> QueryEntry, TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>& OutQueries);

	TSharedPtr<STreeView<TSharedPtr<FChaosVDSceneQueryTreeItem>>> SceneQueriesListWidget;

	TSharedPtr<TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>>> ExternalTreeItemSourceData;

	TArray<TSharedPtr<FChaosVDSceneQueryTreeItem>> InternalTreeItemSourceData;

	FChaosVDQueryTreeItemSelected QueryItemSelectedDelegate;
	FChaosVDQueryTreeItemFocused QueryItemFocusedDelegate;

public:

	struct FColumNames
	{
		const FName Visibility = FName("Visibility");
		const FName TraceTag = FName("TraceTag");
		const FName TraceOwner = FName("TraceOwner");
		const FName QueryType = FName("QueryType");
		const FName SolverName = FName("SolverName");
	};

	static FColumNames ColumnNames;
};
