// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDSceneQueryTreeRow.h"

#include "SChaosVDSceneQueryTree.h"
#include "SlateOptMacros.h"
#include "DataWrappers/ChaosVDQueryDataWrappers.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosVDSceneQueryTreeRow::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView)
{
	Item = InArgs._Item;
	FSuperRowType::FArguments Args = FSuperRowType::FArguments()
									.Style(&FAppStyle::Get().GetWidgetStyle<FTableRowStyle>("SceneOutliner.TableViewRow"));

	SMultiColumnTableRow<TSharedPtr<const FChaosVDSceneQueryTreeItem>>::Construct(Args, InOwnerTableView);
}

TSharedRef<SWidget> SChaosVDSceneQueryTreeRow::GenerateWidgetForColumn(const FName& ColumnName)
{
	if (!Item)
	{
		return SNullWidget::NullWidget;
	}

	TSharedPtr<const FChaosVDQueryDataWrapper> QueryDataPtr = Item->ItemWeakPtr.Pin();
	if (!QueryDataPtr)
	{
		return SNullWidget::NullWidget;
	}

	if (ColumnName == SChaosVDSceneQueryTree::ColumnNames.TraceTag)
	{
		constexpr float NoPadding = 0.0f;
		constexpr float ExpanderLeftPadding = 6.0f;
		constexpr float ExpanderIndentAmount = 12.0f;
		// The first column gets the tree expansion arrow for this row
		return SNew(SBox)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.Padding(ExpanderLeftPadding, NoPadding, NoPadding, NoPadding)
				[
					SNew(SExpanderArrow, SharedThis(this)).IndentAmount(ExpanderIndentAmount)
				]
				+SHorizontalBox::Slot()
				.FillWidth(1.0f)
				[
					GenerateTextWidgetFromName(QueryDataPtr->CollisionQueryParams.TraceTag)
				]
			];
	}

	if (ColumnName == SChaosVDSceneQueryTree::ColumnNames.TraceOwner)
	{
		return GenerateTextWidgetFromName(QueryDataPtr->CollisionQueryParams.OwnerTag);
	}

	if (ColumnName == SChaosVDSceneQueryTree::ColumnNames.QueryType)
	{
		return GenerateTextWidgetFromText(UEnum::GetDisplayValueAsText(QueryDataPtr->Type));
	}

	if (ColumnName == SChaosVDSceneQueryTree::ColumnNames.SolverName)
	{
		return GenerateTextWidgetFromName(Item->OwnerSolverName);
	}

	if (ColumnName == SChaosVDSceneQueryTree::ColumnNames.Visibility)
	{
		return SNew(SImage)
				.IsEnabled(false)
				.Visibility(this, &SChaosVDSceneQueryTreeRow::GetVisibilityIconVisibility)
				.ColorAndOpacity(FSlateColor::UseForeground())
				.ToolTipText(LOCTEXT("SceneUqreyBrowserItemVisibilityToolTip","Visibility is controlled by the Visualization Flags menu"))
				.Image(this, &SChaosVDSceneQueryTreeRow::GetVisibilityIconForCurrentItem);
	}

	return SNullWidget::NullWidget;
}

EVisibility SChaosVDSceneQueryTreeRow::GetVisibilityIconVisibility() const
{
	// For now, we only want to show the visibility icon to indicate the item is hidden.
	// TODO: Support Hover like the visibility widget in the scene outliner
	return Item && Item->bIsVisible ? EVisibility::Hidden : EVisibility::Visible;
}

TSharedRef<SWidget> SChaosVDSceneQueryTreeRow::GenerateTextWidgetFromName(FName Name)
{
	return GenerateTextWidgetFromText(FText::FromName(Name));
}

TSharedRef<SWidget> SChaosVDSceneQueryTreeRow::GenerateTextWidgetFromText(const FText& Text)
{
	constexpr float MarginLeft = 4.0f;
	constexpr float NoMargin = 0.0f;

	return SNew(STextBlock)
			.Margin(FMargin(MarginLeft,NoMargin,NoMargin,NoMargin))
			.Text(Text);
}

const FSlateBrush* SChaosVDSceneQueryTreeRow::GetVisibilityIconForCurrentItem() const
{
	if (!Item)
	{
		return nullptr;
	}

	return Item->bIsVisible ? FAppStyle::Get().GetBrush(TEXT("Level.VisibleIcon16x")) : FAppStyle::Get().GetBrush(TEXT("Level.NotVisibleIcon16x"));
}

#undef LOCTEXT_NAMESPACE