// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"

class SChaosVDSceneQueryBrowser;
struct FChaosVDQueryDataWrapper;
struct FChaosVDSceneQueryTreeItem;

/**
 * Widget used to represent a row on the Scene Query Browser tree view
 */
class SChaosVDSceneQueryTreeRow : public SMultiColumnTableRow<TSharedPtr<const FChaosVDSceneQueryTreeItem>>
{
public:
	SLATE_BEGIN_ARGS(SChaosVDSceneQueryTreeRow)
		{
		}
		SLATE_ARGUMENT(TSharedPtr<FChaosVDSceneQueryTreeItem>, Item)

	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTableView);

	virtual TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName) override;

private:

	EVisibility GetVisibilityIconVisibility() const;
	
	TSharedRef<SWidget> GenerateTextWidgetFromName(FName Name);
	TSharedRef<SWidget> GenerateTextWidgetFromText(const FText& Text);

	const FSlateBrush* GetVisibilityIconForCurrentItem() const;

	TSharedPtr<FChaosVDSceneQueryTreeItem> Item;
};
