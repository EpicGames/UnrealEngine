// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDSolverPlaybackControls.h"

#include "ChaosVDPlaybackController.h"
#include "ChaosVDScene.h"
#include "ChaosVDStyle.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SChaosVDPlaybackViewport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosVDSolverPlaybackControls::Construct(const FArguments& InArgs, const TSharedRef<const FChaosVDTrackInfo>& InSolverTrackInfo, const TWeakPtr<FChaosVDPlaybackController>& InPlaybackController)
{
	SolverTrackInfoRef = InSolverTrackInfo;

	static const FName NAME_VisibleNotHoveredBrush = TEXT("Level.VisibleIcon16x");
	static const FName NAME_NotVisibleNotHoveredBrush = TEXT("Level.NotVisibleIcon16x");
	static const FName NAME_TrackSyncEnabledBrush = TEXT("LinkedIcon");
	static const FName NAME_TrackSyncDisabledBrush = TEXT("UnLinkedIcon");

	SolverVisibleIconBrush = FAppStyle::Get().GetBrush(NAME_VisibleNotHoveredBrush);
	SolverHiddenIconBrush = FAppStyle::Get().GetBrush(NAME_NotVisibleNotHoveredBrush);

	SolverTrackSyncEnabledBrush = FChaosVDStyle::Get().GetBrush(NAME_TrackSyncEnabledBrush);
	SolverTrackSyncDisabledBrush = FChaosVDStyle::Get().GetBrush(NAME_TrackSyncDisabledBrush);
	
	ResimBadgeButtonStyle = FAppStyle::Get().GetWidgetStyle<FButtonStyle>("Menu.Button");

	ChildSlot
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		.AutoWidth()
		.FillWidth(0.8f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 2.0f)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.Text(LOCTEXT("PlaybackViewportWidgetPhysicsFramesLabel", "Solver Frames" ))
			]
			+SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(0.9f)
				[
					SAssignNew(FramesTimelineWidget, SChaosVDTimelineWidget)
					.IsEnabled_Raw(this, &SChaosVDSolverPlaybackControls::CanPlayback)
					.ButtonVisibilityFlags(EChaosVDTimelineElementIDFlags::AllPlayback)
					.IsPlaying_Raw(this, &SChaosVDSolverPlaybackControls::IsPlaying)
					.MinFrames_Raw(this, &SChaosVDSolverPlaybackControls::GetMinFrames)
					.MaxFrames_Raw(this, &SChaosVDSolverPlaybackControls::GetMaxFrames)
					.CurrentFrame_Raw(this, &SChaosVDSolverPlaybackControls::GetCurrentFrame)
					.OnFrameChanged_Raw(this, &SChaosVDSolverPlaybackControls::OnFrameSelectionUpdated)
					.OnButtonClicked_Raw(this, &SChaosVDSolverPlaybackControls::HandleFramePlaybackButtonClicked)
					.OnTimelineScrubStart(this, &SChaosVDSolverPlaybackControls::HandleTimelineScrubStart)
					.OnTimelineScrubEnd(this, &SChaosVDSolverPlaybackControls::HandleTimelineScrubEnd)
				]
				+SHorizontalBox::Slot()
				.Padding(6.0f,0.0f)
				.AutoWidth()
				[
					SNew(SBorder)
					.BorderImage_Raw(this, &SChaosVDSolverPlaybackControls::GetFrameTypeBadgeBrush)
					.Padding(2.0f)
					.Content()
					[
						SNew(SBox)
						.Padding(4.0f,0.0f)
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Justification(ETextJustify::Center)
							.Text_Lambda([this]()->FText{ return SolverTrackInfoRef->bIsReSimulated ? LOCTEXT("PlaybackViewportWidgetPhysicsFramesResimLabel", "ReSim" ) : LOCTEXT("PlaybackViewportWidgetPhysicsFramesNormalLabel", "Normal" );})
						]
					]
				]
			]
		]
		+SHorizontalBox::Slot()
		.FillWidth(0.2f)
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			.AutoHeight()
			.Padding(0.0f, 0.0f, 0.0f, 2.0f)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.Text_Lambda([this]()->FText{ return FText::Format(LOCTEXT("PlaybackViewportWidgetStepsLabel","Solver Stage: {0}"), FText::FromStringView(GetCurrentSolverStageName()));})
			]
			+SVerticalBox::Slot()
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.FillWidth(0.9f)
				[
					SAssignNew(StepsTimelineWidget, SChaosVDTimelineWidget)
					.IsEnabled_Raw(this, &SChaosVDSolverPlaybackControls::CanPlayback)
					.ButtonVisibilityFlags(EChaosVDTimelineElementIDFlags::AllManualStepping)
					.OnFrameChanged_Raw(this, &SChaosVDSolverPlaybackControls::OnSolverStageSelectionUpdated)
					.MaxFrames_Raw(this, &SChaosVDSolverPlaybackControls::GetMaxSolverStage)
					.MinFrames_Raw(this, &SChaosVDSolverPlaybackControls::GetMinSolverStage)
					.CurrentFrame_Raw(this, &SChaosVDSolverPlaybackControls::GetCurrentSolverStage)
					.OnButtonClicked_Raw(this, &SChaosVDSolverPlaybackControls::HandleSolverStagePlaybackButtonClicked)
					.OnTimelineScrubStart(this, &SChaosVDSolverPlaybackControls::HandleTimelineScrubStart)
					.OnTimelineScrubEnd(this, &SChaosVDSolverPlaybackControls::HandleTimelineScrubEnd)
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					CreateVisibilityWidget().ToSharedRef()
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					CreateSyncLinkWidget().ToSharedRef()
				]
			]
		]
	];

	RegisterNewController(InPlaybackController);
}

SChaosVDSolverPlaybackControls::~SChaosVDSolverPlaybackControls()
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{	
		if (TSharedPtr<FChaosVDScene> Scene = PlaybackControllerPtr->GetControllerScene().Pin())
		{
			Scene->OnSolverVisibilityUpdated().RemoveAll(this);
		}
	}
}

void SChaosVDSolverPlaybackControls::HandleTimelineScrubStart()
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->SetScrubbingTimeline(true);
	}
}

void SChaosVDSolverPlaybackControls::HandleTimelineScrubEnd()
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->SetScrubbingTimeline(false);
	}
}

bool SChaosVDSolverPlaybackControls::CanPlayback() const
{
	bool bCanControlPlayback = false;

	if (const TSharedPtr<FChaosVDPlaybackController> CurrentPlaybackControllerPtr = PlaybackController.Pin())
	{
		TSharedRef<FChaosVDTrackInfo> ActivePlaybackTrack = CurrentPlaybackControllerPtr->GetActiveTrackInfo();
		bCanControlPlayback = CurrentPlaybackControllerPtr->IsPlayingLiveSession() ? !ActivePlaybackTrack->bIsPlaying : true;

		// When it is not a live session, the Game Frames timeline follows the same rule as other timelines. The controls are locked unless we are who started a Play action
		if (bCanControlPlayback)
		{
			const bool bIsCompatibleSyncMode = CurrentPlaybackControllerPtr->IsCompatibleWithSyncMode(SolverTrackInfoRef, CurrentPlaybackControllerPtr->GetTimelineSyncMode());

			if (!bIsCompatibleSyncMode)
			{
				return false;
			}

			bool bActiveTrackIsThisTrack = FChaosVDTrackInfo::AreSameTrack(ActivePlaybackTrack, SolverTrackInfoRef);
			bCanControlPlayback =  bActiveTrackIsThisTrack || (!bActiveTrackIsThisTrack && !ActivePlaybackTrack->bIsPlaying);
		}
	}

	return bCanControlPlayback;
}

void SChaosVDSolverPlaybackControls::HandleSolverVisibilityChanged(int32 InSolverID, bool bNewVisibility)
{
	if (SolverTrackInfoRef->TrackID != InSolverID)
	{
		return;
	}

	bIsVisible = bNewVisibility;
}

FReply SChaosVDSolverPlaybackControls::ToggleSolverVisibility() const
{
	if (const TSharedPtr<FChaosVDPlaybackController> CurrentPlaybackControllerPtr = PlaybackController.Pin())
	{
		CurrentPlaybackControllerPtr->UpdateTrackVisibility(EChaosVDTrackType::Solver, SolverTrackInfoRef->TrackID, !bIsVisible);
	}

	return FReply::Handled();
}

FReply SChaosVDSolverPlaybackControls::ToggleSolverSyncLink() const
{
	if (const TSharedPtr<FChaosVDPlaybackController> CurrentPlaybackControllerPtr = PlaybackController.Pin())
	{
		CurrentPlaybackControllerPtr->ToggleTrackSyncEnabled(SolverTrackInfoRef);
	}
	
	return FReply::Handled();
}

bool SChaosVDSolverPlaybackControls::CanChangeVisibility() const
{
	return SolverTrackInfoRef->bSupportsVisibilityChange;
}

const FSlateBrush* SChaosVDSolverPlaybackControls::GetBrushForCurrentVisibility() const
{
	return bIsVisible ? SolverVisibleIconBrush : SolverHiddenIconBrush;
}

const FSlateBrush* SChaosVDSolverPlaybackControls::GetBrushForCurrentLinkState() const
{
	return SolverTrackInfoRef->bTrackSyncEnabled ? SolverTrackSyncEnabledBrush : SolverTrackSyncDisabledBrush;
}

FStringView SChaosVDSolverPlaybackControls::GetCurrentSolverStageName() const
{
	static TCHAR const* UnknownStepName = TEXT("Unknown");

	if (SolverTrackInfoRef->CurrentStageNames.IsValidIndex(SolverTrackInfoRef->CurrentStage))
	{
		return SolverTrackInfoRef->CurrentStageNames[SolverTrackInfoRef->CurrentStage];
	}

	return UnknownStepName;
}

void SChaosVDSolverPlaybackControls::HandleFramePlaybackButtonClicked(EChaosVDPlaybackButtonsID ButtonID)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->HandleFramePlaybackControlInput(ButtonID, SolverTrackInfoRef, GetInstigatorID());
	}
}

void SChaosVDSolverPlaybackControls::HandleSolverStagePlaybackButtonClicked(EChaosVDPlaybackButtonsID ButtonID)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		PlaybackControllerPtr->HandleFrameStagePlaybackControlInput(ButtonID, SolverTrackInfoRef, GetInstigatorID());
	}
}

const FSlateBrush* SChaosVDSolverPlaybackControls::GetFrameTypeBadgeBrush() const
{
	return SolverTrackInfoRef->bIsReSimulated ? &ResimBadgeButtonStyle.Pressed : FCoreStyle::Get().GetBrush("Border");
}

TSharedPtr<SWidget> SChaosVDSolverPlaybackControls::CreateVisibilityWidget()
{
	return SNew(SButton)
			.OnClicked_Raw(this, &SChaosVDSolverPlaybackControls::ToggleSolverVisibility)
			.ToolTipText_Raw(this, &SChaosVDSolverPlaybackControls::GetVisibilityButtonToolTipText)
			.IsEnabled_Raw(this,&SChaosVDSolverPlaybackControls::CanChangeVisibility)
			[
				SNew(SImage)
				.Image_Raw(this,&SChaosVDSolverPlaybackControls::GetBrushForCurrentVisibility)
				.DesiredSizeOverride(FVector2D(16.0f,16.0f))
				.ColorAndOpacity(FSlateColor::UseForeground())
			];
}

TSharedPtr<SWidget> SChaosVDSolverPlaybackControls::CreateSyncLinkWidget()
{
	return SNew(SButton)
		.OnClicked_Raw(this, &SChaosVDSolverPlaybackControls::ToggleSolverSyncLink)
		.ToolTipText_Raw(this, &SChaosVDSolverPlaybackControls::GetSyncLinkTipText)
		[
			SNew(SImage)
			.Image_Raw(this,&SChaosVDSolverPlaybackControls::GetBrushForCurrentLinkState)
			.DesiredSizeOverride(FVector2D(16.0f,16.0f))
			.ColorAndOpacity(FSlateColor::UseForeground())
		];	
}

FText SChaosVDSolverPlaybackControls::GetVisibilityButtonToolTipText() const
{
	if (SolverTrackInfoRef->bSupportsVisibilityChange)
	{
		return LOCTEXT("VisibilityControlDisabledButtonToolTipText", "This track does not support visibility changes");
	}

	return bIsVisible ? LOCTEXT("HideVisibilityButtonToolTipText", "Click to hide all the visualization data corresponding to this solver track") : LOCTEXT("ShowVisibilityButtonToolTipText", "Click to show all the visualization data corresponding to this solver track");
}

FText SChaosVDSolverPlaybackControls::GetSyncLinkTipText() const
{
	return SolverTrackInfoRef->bTrackSyncEnabled ? LOCTEXT("DisableSyncLinkToolTipText", "Click to disable track syncing so this timeline can be player independently") : LOCTEXT("EnableSyncLinkToolTipText", "Click to eanble track syncing so this will be played in sync with other tracks");
}

bool SChaosVDSolverPlaybackControls::IsPlaying() const
{
	return SolverTrackInfoRef->bIsPlaying;
}

int32 SChaosVDSolverPlaybackControls::GetCurrentFrame() const
{
	return SolverTrackInfoRef->CurrentFrame;
}

int32 SChaosVDSolverPlaybackControls::GetMinFrames() const
{
	return 0;
}

int32 SChaosVDSolverPlaybackControls::GetMaxFrames() const
{
	return SolverTrackInfoRef->MaxFrames - 1;
}

int32 SChaosVDSolverPlaybackControls::GetCurrentSolverStage() const
{
	return SolverTrackInfoRef->CurrentStage;
}

int32 SChaosVDSolverPlaybackControls::GetMinSolverStage() const
{
	return 0;
}

int32 SChaosVDSolverPlaybackControls::GetMaxSolverStage() const
{
	return SolverTrackInfoRef->CurrentStageNames.Num() -1;
}

void SChaosVDSolverPlaybackControls::OnFrameSelectionUpdated(int32 NewFrameIndex)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		// By default we always playback frames at the last recorded stage as that represents the end of frame state
		constexpr int32 LastStepNumber = INDEX_NONE;
		PlaybackControllerPtr->TrySetActiveTrack(SolverTrackInfoRef);
		PlaybackControllerPtr->GoToTrackFrameAndSync(GetInstigatorID(), EChaosVDTrackType::Solver, SolverTrackInfoRef->TrackID, NewFrameIndex, LastStepNumber);
	}
}

void SChaosVDSolverPlaybackControls::OnSolverStageSelectionUpdated(int32 NewStepIndex)
{
	if (const TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
	{
		// On Steps updates. Always use the current Frame
		PlaybackControllerPtr->GoToTrackFrame(GetInstigatorID(), EChaosVDTrackType::Solver, SolverTrackInfoRef->TrackID, SolverTrackInfoRef->CurrentFrame, NewStepIndex);
	}
}

void SChaosVDSolverPlaybackControls::RegisterNewController(TWeakPtr<FChaosVDPlaybackController> NewController)
{
	if (const TSharedPtr<FChaosVDPlaybackController> OldPlaybackControllerPtr = PlaybackController.Pin())
	{
		if (TSharedPtr<FChaosVDScene> Scene = OldPlaybackControllerPtr->GetControllerScene().Pin())
		{
			Scene->OnSolverVisibilityUpdated().RemoveAll(this);
		}
	}

	FChaosVDPlaybackControllerObserver::RegisterNewController(NewController);

	if (const TSharedPtr<FChaosVDPlaybackController> NewPlaybackControllerPtr = PlaybackController.Pin())
	{
		if (TSharedPtr<FChaosVDScene> Scene = NewPlaybackControllerPtr->GetControllerScene().Pin())
		{
			bIsVisible = NewPlaybackControllerPtr->IsTrackVisible(SolverTrackInfoRef->TrackType, SolverTrackInfoRef->TrackID);
			Scene->OnSolverVisibilityUpdated().AddRaw(this, &SChaosVDSolverPlaybackControls::HandleSolverVisibilityChanged);
		}
	}
}

#undef LOCTEXT_NAMESPACE
