// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDPlaybackController.h"
#include "ChaosVDPlaybackControllerInstigator.h"
#include "ChaosVDPlaybackControllerObserver.h"
#include "SChaosVDTimelineWidget.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateTypes.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"

enum class EChaosVDPlaybackButtonsID : uint8;
struct FChaosVDTrackInfo;
class FChaosVDPlaybackController;
class SChaosVDTimelineWidget;

/**
 * Options flags to control how the Step timeline widgets should be updated
 */
enum class EChaosVDStepsWidgetUpdateFlags
{
	UpdateText =  1 << 0,
	SetTimelineStep = 1 << 1,

	Default = UpdateText | SetTimelineStep
};
ENUM_CLASS_FLAGS(EChaosVDStepsWidgetUpdateFlags)

/** Widget that Generates playback controls for solvers
 * Which are two timelines, one for physics frames and other for steps
 */
class SChaosVDSolverPlaybackControls : public SCompoundWidget, public FChaosVDPlaybackControllerObserver, public IChaosVDPlaybackControllerInstigator
{
public:

	explicit SChaosVDSolverPlaybackControls()
		: SolverTrackInfoRef(MakeShared<const FChaosVDTrackInfo>())
	{
	}


	SLATE_BEGIN_ARGS(SChaosVDSolverPlaybackControls) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TSharedRef<const FChaosVDTrackInfo>& InSolverTrackInfo, const TWeakPtr<FChaosVDPlaybackController>& InPlaybackController);

	virtual ~SChaosVDSolverPlaybackControls() override;
	void HandleTimelineScrubStart();
	void HandleTimelineScrubEnd();

private:

	void OnFrameSelectionUpdated(int32 NewFrameIndex);
	void OnSolverStageSelectionUpdated(int32 NewStepIndex);

	virtual void RegisterNewController(TWeakPtr<FChaosVDPlaybackController> NewController) override;

	FStringView GetCurrentSolverStageName() const;

	void HandleFramePlaybackButtonClicked(EChaosVDPlaybackButtonsID ButtonID);
	void HandleSolverStagePlaybackButtonClicked(EChaosVDPlaybackButtonsID ButtonID);

	void HandleSolverVisibilityChanged(int32 InSolverID, bool bNewVisibility);

	FReply ToggleSolverVisibility() const;
	FReply ToggleSolverSyncLink() const;

	bool CanChangeVisibility() const;
	const FSlateBrush* GetBrushForCurrentVisibility() const;
	const FSlateBrush* GetBrushForCurrentLinkState() const;

	const FSlateBrush* GetFrameTypeBadgeBrush() const;

	TSharedPtr<SWidget> CreateVisibilityWidget();
	TSharedPtr<SWidget> CreateSyncLinkWidget();

	FText GetVisibilityButtonToolTipText() const;
	FText GetSyncLinkTipText() const;
	
	bool CanPlayback() const;

	bool IsPlaying() const;

	int32 GetCurrentFrame() const;
	int32 GetMinFrames() const;
	int32 GetMaxFrames() const;
	
	int32 GetCurrentSolverStage() const;
	int32 GetMinSolverStage() const;
	int32 GetMaxSolverStage() const;

	TSharedRef<const FChaosVDTrackInfo> SolverTrackInfoRef;
	bool bIsReSimFrame = false;
	FString CurrentStepName;
	TSharedPtr<SChaosVDTimelineWidget> FramesTimelineWidget;
	TSharedPtr<SChaosVDTimelineWidget> StepsTimelineWidget;
	bool bIsVisible = true;

	const FSlateBrush* SolverVisibleIconBrush = nullptr;
	const FSlateBrush* SolverHiddenIconBrush = nullptr;
	
	const FSlateBrush* SolverTrackSyncEnabledBrush = nullptr;
	const FSlateBrush* SolverTrackSyncDisabledBrush = nullptr;

	FButtonStyle ResimBadgeButtonStyle;
};
