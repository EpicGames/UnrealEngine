// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDSolverTracks.h"

#include "ChaosVDModule.h"
#include "ChaosVDPlaybackController.h"
#include "Widgets/SChaosVDMainTab.h"
#include "SChaosVDWarningMessageBox.h"
#include "Settings/ChaosVDSolverTrackSettings.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Framework/Application/SlateApplication.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"
#include "Widgets/SChaosVDPlaybackViewport.h"
#include "Widgets/SChaosVDSolverPlaybackControls.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Layout/SExpandableArea.h"
#include "Widgets/Layout/SSeparator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SChaosVDSolverTracks)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosVDSolverTracks::Construct(const FArguments& InArgs, TWeakPtr<FChaosVDPlaybackController> InPlaybackController, TWeakPtr<SChaosVDMainTab> MainTab)
{
	MenuName = FName(TEXT("ChaosVD.SolverTracks.MenuToolbar"));

	MainTabWeakPtr = MainTab;

	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0.0f, 0, 0.0f)
		[
			SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				GenerateToolbarWidget()
			]
		]
		+SVerticalBox::Slot()
		.Padding(0, 0.0f, 0, 2.0f)
		[
			SAssignNew(SolverTracksListWidget, SListView<TSharedPtr<const FChaosVDTrackInfo>>)
			.ListItemsSource(&CachedTrackInfoArray)
			.SelectionMode( ESelectionMode::None )
			.ListViewStyle(&FAppStyle::Get().GetWidgetStyle<FTableViewStyle>("SimpleListView"))
			.OnGenerateRow(this, &SChaosVDSolverTracks::MakeSolverTrackControlsFromTrackInfo)
		]
	
	];

	ensure(InPlaybackController.IsValid());

	RegisterNewController(InPlaybackController);
	
	if (const TSharedPtr<FChaosVDPlaybackController> CurrentPlaybackControllerPtr = InPlaybackController.Pin())
	{
		if (const TSharedPtr<const FChaosVDTrackInfo> GameTrackInfo = CurrentPlaybackControllerPtr->GetTrackInfo(EChaosVDTrackType::Game, FChaosVDPlaybackController::GameTrackID))
		{
			HandleControllerTrackFrameUpdated(InPlaybackController, GameTrackInfo.ToSharedRef(), InvalidGuid);
		}
	}
	else
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Solver tracks constructed with an invalid player controller. The solver tracks widget will not be functional", ANSI_TO_TCHAR(__FUNCTION__))	
	}
	
	if (UChaosVDSolverTrackSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSolverTrackSettings>())
	{
		Settings->OnSettingsChanged().AddSP(this, &SChaosVDSolverTracks::HandleSettingsChanged);
		
		HandleSettingsChanged(Settings);
	}
}

SChaosVDSolverTracks::~SChaosVDSolverTracks()
{
	if (UChaosVDSolverTrackSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSolverTrackSettings>())
	{
		Settings->OnSettingsChanged().RemoveAll(this);
	}
}

void SChaosVDSolverTracks::HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InPlaybackController)
{
	if (PlaybackController != InPlaybackController)
	{
		RegisterNewController(InPlaybackController);
	}

	if (const TSharedPtr<FChaosVDPlaybackController> CurrentPlaybackControllerPtr = InPlaybackController.Pin())
	{
		// If the controller data was updated, need to update our cache track info data as it could have been changed.
		// For example this can happen when we load another recording. We use the GameTrack info for that as it is the one that is always valid
		if (const TSharedPtr<const FChaosVDTrackInfo> GameTrackInfo = CurrentPlaybackControllerPtr->GetTrackInfo(EChaosVDTrackType::Game, FChaosVDPlaybackController::GameTrackID))
		{
			UpdatedCachedTrackInfoData(InPlaybackController, GameTrackInfo.ToSharedRef());
		}
	}
}

void SChaosVDSolverTracks::UpdatedCachedTrackInfoData(TWeakPtr<FChaosVDPlaybackController> InPlaybackController, const TSharedRef<const FChaosVDTrackInfo>& UpdatedTrackInfo)
{
	if (const TSharedPtr<FChaosVDPlaybackController> CurrentPlaybackControllerPtr = InPlaybackController.Pin())
	{
		TArray<TSharedPtr<const FChaosVDTrackInfo>> TrackInfoArray;

		if (CurrentPlaybackControllerPtr->GetTimelineSyncMode() == EChaosVDSyncTimelinesMode::Manual)
		{
			CurrentPlaybackControllerPtr->GetAvailableTracks(EChaosVDTrackType::Solver,TrackInfoArray);
		}
		else
		{
			CurrentPlaybackControllerPtr->GetAvailableTrackInfosAtTrackFrame(EChaosVDTrackType::Solver, UpdatedTrackInfo, TrackInfoArray);
		}

		if (TrackInfoArray != CachedTrackInfoArray)
		{
			CachedTrackInfoArray = MoveTemp(TrackInfoArray);
			SolverTracksListWidget->RebuildList();
		}
	}
	else
	{
		CachedTrackInfoArray.Empty();
		SolverTracksListWidget->RebuildList();
	}
}

void SChaosVDSolverTracks::HandleControllerTrackFrameUpdated(TWeakPtr<FChaosVDPlaybackController> InPlaybackController, TWeakPtr<const FChaosVDTrackInfo> UpdatedTrackInfo, FGuid InstigatorGuid)
{
	if (InstigatorGuid == GetInstigatorID())
	{
		// Ignore the update if we initiated it
		return;
	}

	TSharedPtr<const FChaosVDTrackInfo> UpdatedTrackInfoPtr = UpdatedTrackInfo.Pin();

	if (!UpdatedTrackInfoPtr)
	{
		return;
	}

	// Only Game Frame Track Update can change the available solvers
	if (UpdatedTrackInfoPtr->TrackType == EChaosVDTrackType::Solver)
	{
		return;
	}

	UpdatedCachedTrackInfoData(InPlaybackController, UpdatedTrackInfoPtr.ToSharedRef());
}

TSharedRef<ITableRow> SChaosVDSolverTracks::MakeSolverTrackControlsFromTrackInfo(TSharedPtr<const FChaosVDTrackInfo> TrackInfo, const TSharedRef<STableViewBase>& OwnerTable)
{
	TSharedPtr<SWidget> RowWidget;
	if (TrackInfo)
	{
		RowWidget = SNew(SVerticalBox)
				.Visibility_Lambda([WeakTrackInfo = TrackInfo.ToWeakPtr()]()
				{
					TSharedPtr<const FChaosVDTrackInfo> TrackInfoPtr = StaticCastSharedPtr<const FChaosVDTrackInfo>(WeakTrackInfo.Pin());
					return TrackInfoPtr && TrackInfoPtr->bCanShowTrackControls ?  EVisibility::Visible : EVisibility::Collapsed;
				})
				+SVerticalBox::Slot()
				.AutoHeight()
				.Padding(10.0f, 4.0f, 10.0f, 0.0f)
				[
					SNew(SExpandableArea)
						.InitiallyCollapsed(false)
						.BorderBackgroundColor(FLinearColor::White)
						.Padding(FMargin(8.f))
						.HeaderContent()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							.AutoWidth()
							.Padding(2.0f, 0.0f, 8.0f, 0.0f)
							[
								SNew(SCheckBox)
								.IsEnabled(false)
								.Style(FAppStyle::Get(), "Menu.RadioButton")
								.IsChecked_Lambda([WeakThis = AsWeak(), WeakTrackInfo = TrackInfo.ToWeakPtr()]()
								{
									if (TSharedPtr<SChaosVDSolverTracks> SolverTracksWidget = StaticCastSharedPtr<SChaosVDSolverTracks>(WeakThis.Pin()))
									{
										return SolverTracksWidget->IsActiveTrack(WeakTrackInfo) ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
									}

									return ECheckBoxState::Undetermined;
								})
							]
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
							.VAlign(VAlign_Center)
							.AutoWidth()
							.Padding(0.f, 0.f, 0.f, 0.f)
							[
								SNew(STextBlock)
								.Text(FText::FromName(TrackInfo->TrackName))
								.Font(FCoreStyle::Get().GetFontStyle("ExpandableArea.TitleFont"))
							]
							+SHorizontalBox::Slot()
							.AutoWidth()
							.Padding(8.0f, 4.0f, 8.0f, 4.0f)
							[
								SNew(SSeparator)
								.Orientation(Orient_Vertical)
								.Thickness(1.f)
							]
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Left)
                            .VAlign(VAlign_Center)
                            .AutoWidth()
                            .Padding(0.f, 0.f, 0.f, 0.f)
                            [
								SNew(SChaosVDWarningMessageBox)
								.Visibility_Lambda([TrackInfoAsWeak = TrackInfo.ToWeakPtr(), WeakPlaybackController = PlaybackController]()
								{
									TSharedPtr<const FChaosVDTrackInfo> TrackInfoPtr = TrackInfoAsWeak.Pin();
									TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = WeakPlaybackController.Pin();
									if (TrackInfoPtr && PlaybackControllerPtr)
									{
										bool bIsCompatibleMode = PlaybackControllerPtr->GetTimelineSyncMode() == EChaosVDSyncTimelinesMode::NetworkTick ? TrackInfoPtr->bHasNetworkSyncData : true;
										return bIsCompatibleMode ? EVisibility::Collapsed : EVisibility::Visible;
									}
									return EVisibility::Collapsed;
								})
								.WarningText(LOCTEXT("IncomatibleSyncModeWarning", " Incompatible sync mode selected | Attempting to fallback to TimeStamp sync mode for this track | Controls disabled"))
                            ]
							+SHorizontalBox::Slot()
							.HAlign(HAlign_Right)
							.VAlign(VAlign_Center)
							.AutoWidth()
							.Padding(0.f, 0.f, 2.f, 0.f)
							[
								SNew(STextBlock)
								.Visibility(this, &SChaosVDSolverTracks::GetSelectorKeyVisibility, TrackInfo->TrackSlot)
								.Text(FText::FormatOrdered(LOCTEXT("TrackSelectorModifier", "CTRL + {0}"), TrackInfo->TrackSlot))
								.Font(FCoreStyle::Get().GetFontStyle("ExpandableArea.TitleFont"))
							]
						]
						.BodyContent()
						[
							SNew(SHorizontalBox)
							+SHorizontalBox::Slot()
							.Padding(2.f, 4.f, 2.f, 12.f)
							[
								SNew(SChaosVDSolverPlaybackControls, TrackInfo.ToSharedRef(), PlaybackController)
							]
						]
				];
	}
	else
	{
		RowWidget = SNew(SVerticalBox)
		+SVerticalBox::Slot()
		[
			SNew(STextBlock)
			.Text(NSLOCTEXT("ChaosVisualDebugger", "SolverPlaybackControlsErrorMessage", "Failed to read data for solver."))
		];
	}

	return
		SNew(STableRow< TSharedPtr<FString> >, OwnerTable)
		[
			RowWidget.ToSharedRef()
		];
}

EVisibility SChaosVDSolverTracks::GetSelectorKeyVisibility(int32 TrackSlot) const
{
	// Currently we only support selecting tracks using CTRL + [0-9]. We never have that many tracks. If at some point we do
	// Like when we add support for other solve types, we can create another combination (or input chord), and then update this code 
	constexpr int32 MaxSlotAddressableByKeyboard = 9;
	if (TrackSlot > MaxSlotAddressableByKeyboard)
	{
		return EVisibility::Collapsed;
	}
	
	if (TSharedPtr<SChaosVDMainTab> MainTabPtr = MainTabWeakPtr.Pin())
	{
		return MainTabPtr->ShouldShowTracksKeyShortcuts() ? EVisibility::Visible : EVisibility::Collapsed;
	}
	
	return EVisibility::Collapsed;
}

bool SChaosVDSolverTracks::IsActiveTrack(const TWeakPtr<const FChaosVDTrackInfo>& TrackInfo) const
{
	TSharedPtr<const FChaosVDTrackInfo> TrackInfoPtr = TrackInfo.Pin();
	TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin();
	if (!TrackInfoPtr || !PlaybackControllerPtr)
	{
		return false;
	}

	return FChaosVDTrackInfo::AreSameTrack(TrackInfoPtr.ToSharedRef(), PlaybackControllerPtr->GetActiveTrackInfo());
}

TSharedRef<SWidget> SChaosVDSolverTracks::GenerateToolbarWidget()
{
	RegisterMenus();

	FToolMenuContext MenuContext;

	UChaosVDSolverTracksToolbarMenuContext* CommonContextObject = NewObject<UChaosVDSolverTracksToolbarMenuContext>();
	CommonContextObject->SolverTracksWidget = SharedThis(this);

	MenuContext.AddObject(CommonContextObject);

	return UToolMenus::Get()->GenerateWidget(MenuName, MenuContext);
}

TSharedRef<SWidget> SChaosVDSolverTracks::GenerateSyncModeMenuWidget()
{
	TAttribute<int32> GetCurrentMode;
	GetCurrentMode.BindLambda([]()
	{
		if (UChaosVDSolverTrackSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSolverTrackSettings>())
		{
			return static_cast<int32>(Settings->SyncMode);
		}
		return 0;
	});

	SEnumComboBox::FOnEnumSelectionChanged ValueChangedDelegate;
	ValueChangedDelegate.BindLambda([](int32 NewValue, ESelectInfo::Type)
	{
		if (UChaosVDSolverTrackSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDSolverTrackSettings>())
		{
			Settings->SyncMode = static_cast<EChaosVDSyncTimelinesMode>(NewValue);
			Settings->OnSettingsChanged().Broadcast(Settings);
			Settings->SaveConfig();
		}
	});

	return Chaos::VisualDebugger::Utils::MakeEnumMenuEntryWidget<EChaosVDSyncTimelinesMode>(LOCTEXT("SyncTimelineModeMenuLabel", "Timeline Sync Mode"), MoveTemp(ValueChangedDelegate), MoveTemp(GetCurrentMode));
}

void SChaosVDSolverTracks::HandleSettingsChanged(UObject* SettingsObject)
{
	if (UChaosVDSolverTrackSettings* Settings = Cast<UChaosVDSolverTrackSettings>(SettingsObject))
	{
		if (TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = PlaybackController.Pin())
		{
			PlaybackControllerPtr->SetTimelineSyncMode(Settings->SyncMode);
		}
	}
}

void SChaosVDSolverTracks::RegisterMenus()
{
	const UToolMenus* ToolMenus = UToolMenus::Get();
	if (ToolMenus->IsMenuRegistered(MenuName))
	{
		return;
	}

	UToolMenu* ToolBar = UToolMenus::Get()->RegisterMenu(MenuName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);

	FToolMenuSection& Section = ToolBar->AddSection("MainToolbar");
	Section.AddDynamicEntry("MainToolbarEntry", FNewToolMenuSectionDelegate::CreateLambda([](FToolMenuSection& InSection)
	{
		const UChaosVDSolverTracksToolbarMenuContext* Context = InSection.FindContext<UChaosVDSolverTracksToolbarMenuContext>();
		TSharedRef<SChaosVDSolverTracks> SolverTracksWidget = Context->SolverTracksWidget.Pin().ToSharedRef();
		
		InSection.AddEntry(
		FToolMenuEntry::InitWidget(
			"SyncModeButton",
			SolverTracksWidget->GenerateSyncModeMenuWidget(),
			FText::GetEmpty(),
			false,
			false
		));
		
	}));
}

#undef LOCTEXT_NAMESPACE
