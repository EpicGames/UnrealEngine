// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDPlaybackControllerInstigator.h"
#include "ChaosVDPlaybackControllerObserver.h"
#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/SListView.h"

#include "SChaosVDSolverTracks.generated.h"

class FChaosVDPlaybackController;
class SChaosVDMainTab;

struct FChaosVDTrackInfo;

UCLASS()
class UChaosVDSolverTracksToolbarMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<class SChaosVDSolverTracks> SolverTracksWidget;
};

/** Widget that Generates a expandable list of solver controls, based on the existing solver data
 * on the ChaosVDPlaybackController
 */
class SChaosVDSolverTracks : public SCompoundWidget, public FChaosVDPlaybackControllerObserver, public IChaosVDPlaybackControllerInstigator
{
public:

	SLATE_BEGIN_ARGS( SChaosVDSolverTracks ){}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TWeakPtr<FChaosVDPlaybackController> InPlaybackController, TWeakPtr<SChaosVDMainTab> MainTab);
	
	~SChaosVDSolverTracks();

private:

	void UpdatedCachedTrackInfoData(TWeakPtr<FChaosVDPlaybackController> InPlaybackController, const TSharedRef<const FChaosVDTrackInfo>& UpdatedTrackInfo);
	virtual void HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InPlaybackController) override;
	virtual void HandleControllerTrackFrameUpdated(TWeakPtr<FChaosVDPlaybackController> InPlaybackController, TWeakPtr<const FChaosVDTrackInfo> UpdatedTrackInfo, FGuid InstigatorGuid) override;

	TSharedRef<ITableRow> MakeSolverTrackControlsFromTrackInfo(TSharedPtr<const FChaosVDTrackInfo> TrackInfo, const TSharedRef<STableViewBase>& OwnerTable);

	TSharedPtr<SListView<TSharedPtr<const FChaosVDTrackInfo>>> SolverTracksListWidget;

	TArray<TSharedPtr<const FChaosVDTrackInfo>> CachedTrackInfoArray;

	EVisibility GetSelectorKeyVisibility(int32 TrackSlot) const;

	bool IsActiveTrack(const TWeakPtr<const FChaosVDTrackInfo>& TrackInfo) const;

	TSharedRef<SWidget> GenerateToolbarWidget();
	TSharedRef<SWidget> GenerateSyncModeMenuWidget();

	void HandleSettingsChanged(UObject* SettingsObject);

	void RegisterMenus();

	bool bShowSelectorKeyShortcut = false;
	
	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr;

	FName MenuName;
};
