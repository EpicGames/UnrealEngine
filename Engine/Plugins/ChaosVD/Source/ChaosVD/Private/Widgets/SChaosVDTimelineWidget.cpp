// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDTimelineWidget.h"

#include "ChaosVDStyle.h"
#include "Input/Reply.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SSlider.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/SBoxPanel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(SChaosVDTimelineWidget)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

void SChaosVDTimelineWidget::Construct(const FArguments& InArgs)
{
	MaxFrames = InArgs._MaxFrames;
	MinFrames = InArgs._MinFrames;
	CurrentFrame = InArgs._CurrentFrame;
	FrameChangedDelegate = InArgs._OnFrameChanged;
	ButtonClickedDelegate = InArgs._OnButtonClicked;
	ElementVisibilityFlags = InArgs._ButtonVisibilityFlags;
	ElementEnabledFlags = InArgs._ButtonEnabledFlags;
	TimelineScrubStartDelegate = InArgs._OnTimelineScrubStart;
	TimelineScrubEndDelegate = InArgs._OnTimelineScrubEnd;
	bIsPlaying = InArgs._IsPlaying;

	if (!ElementVisibilityFlags.IsSet())
	{
		ElementVisibilityFlags = EChaosVDTimelineElementIDFlags::AllPlayback;
	}
	
	if (!ElementEnabledFlags.IsSet())
	{
		ElementEnabledFlags = EChaosVDTimelineElementIDFlags::AllPlayback;
	}

	SetCanTick(true);

	PendingValueChange.Reset();

	ChildSlot
	[
		SNew(SHorizontalBox)
			+SHorizontalBox::Slot()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.AutoWidth()
			.Padding(4.0f,0.0f)
			[
				SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Visibility_Raw(this, &SChaosVDTimelineWidget::GetElementVisibility, EChaosVDTimelineElementIDFlags::Play)
					.IsEnabled_Raw(this, &SChaosVDTimelineWidget::GetElementEnabled, EChaosVDTimelineElementIDFlags::Play)
					.OnClicked( FOnClicked::CreateRaw(this, &SChaosVDTimelineWidget::TogglePlay))
					.ContentPadding( 2.0f )
					.ForegroundColor( FSlateColor::UseForeground() )
					.IsFocusable( false )
					[
						SNew( SImage )
						.Image_Raw(this, &SChaosVDTimelineWidget::GetPlayOrPauseIcon)
						.DesiredSizeOverride(FVector2D(16.0f,16.0f))
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Visibility_Raw(this, &SChaosVDTimelineWidget::GetElementVisibility, EChaosVDTimelineElementIDFlags::Stop)
					.IsEnabled_Raw(this, &SChaosVDTimelineWidget::GetElementEnabled, EChaosVDTimelineElementIDFlags::Stop)
					.OnClicked( FOnClicked::CreateRaw(this, &SChaosVDTimelineWidget::Stop))
					.ContentPadding( 2.0f )
					.ForegroundColor( FSlateColor::UseForeground() )
					.IsFocusable( false )
					[
						SNew( SImage )
						.Image(FChaosVDStyle::Get().GetBrush("StopIcon"))
						.DesiredSizeOverride(FVector2D(16.0f,16.0f))
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Visibility_Raw(this, &SChaosVDTimelineWidget::GetElementVisibility, EChaosVDTimelineElementIDFlags::Prev)
					.IsEnabled_Raw(this, &SChaosVDTimelineWidget::GetElementEnabled, EChaosVDTimelineElementIDFlags::Prev)
					.OnClicked( FOnClicked::CreateRaw(this, &SChaosVDTimelineWidget::Prev))
					.ContentPadding( 2.0f )
					.ForegroundColor( FSlateColor::UseForeground() )
					.IsFocusable( false )
					[
						SNew( SImage )
						.DesiredSizeOverride(FVector2D(16.0f,16.0f))
						.Image(FChaosVDStyle::Get().GetBrush("PrevIcon"))
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SButton)
					.Visibility_Raw(this, &SChaosVDTimelineWidget::GetElementVisibility, EChaosVDTimelineElementIDFlags::Next)
					.IsEnabled_Raw(this, &SChaosVDTimelineWidget::GetElementEnabled, EChaosVDTimelineElementIDFlags::Next)
					.OnClicked( FOnClicked::CreateRaw(this, &SChaosVDTimelineWidget::Next))
					.ContentPadding( 2.0f )
					.ForegroundColor( FSlateColor::UseForeground() )
					.IsFocusable( false )
					[
						SNew( SImage )
						.DesiredSizeOverride(FVector2D(16.0f,16.0f))
						.Image(FChaosVDStyle::Get().GetBrush("NextIcon"))
						.ColorAndOpacity( FSlateColor::UseForeground() )
					]
				]
			]
			+SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(4.0f,0.0f)
			.FillWidth(1.0f)
			[
				SAssignNew(TimelineSlider, SSlider)
				.Visibility_Raw(this, &SChaosVDTimelineWidget::GetElementVisibility, EChaosVDTimelineElementIDFlags::Timeline)
				.ToolTipText_Lambda([this]()-> FText{ return FText::AsNumber(GetCurrentFrame()); })
				.Value_Raw(this, &SChaosVDTimelineWidget::GetCurrentFrameAsFloat)
				.OnValueChanged_Raw(this, &SChaosVDTimelineWidget::SetCurrentTimelineFrame, EChaosVDSetTimelineFrameFlags::BroadcastChange)
				.StepSize(1)
				.MaxValue(0)
				.MinValue(0)
				.OnMouseCaptureBegin_Raw(this, &SChaosVDTimelineWidget::HandleTimelineScrubStart)
				.OnMouseCaptureEnd_Raw(this, &SChaosVDTimelineWidget::HandleTimelineScrubEnd)
			]
			+SHorizontalBox::Slot()
			.Padding(4.0f,0.0f)
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(STextBlock)
				.ColorAndOpacity(FLinearColor::White)
				.Text_Lambda([this]()->FText{ return FText::Format(LOCTEXT("FramesCounter","{0} / {1}"), GetCurrentFrame(), GetCurrentMaxFrames());})
			]
	];
}

FReply SChaosVDTimelineWidget::TogglePlay()
{
	if (IsPlaying())
	{
		Pause();
	}
	else
	{
		Play();
	}

	return FReply::Handled();
}

void SChaosVDTimelineWidget::Play()
{
	ButtonClickedDelegate.ExecuteIfBound(EChaosVDPlaybackButtonsID::Play);
}

FReply SChaosVDTimelineWidget::Stop()
{
	ButtonClickedDelegate.ExecuteIfBound(EChaosVDPlaybackButtonsID::Stop);

	return FReply::Handled();
}

void SChaosVDTimelineWidget::SetCurrentTimelineFrame(float FrameNumber, EChaosVDSetTimelineFrameFlags Options)
{
	if (TimelineSlider.IsValid())
	{
		if (EnumHasAnyFlags(Options, EChaosVDSetTimelineFrameFlags::BroadcastChange))
		{
			PendingValueChange = FrameNumber;
		}
	}
}

void SChaosVDTimelineWidget::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	SCompoundWidget::Tick(AllottedGeometry, InCurrentTime, InDeltaTime);

	if (TimelineSlider)
	{
		const int32 CurrentMinFrames = GetCurrentMinFrames();
		const int32 CurrentMaxFrames = GetCurrentMaxFrames();
		if (!FMath::IsNearlyEqual(TimelineSlider->GetMinValue(), CurrentMinFrames) || !FMath::IsNearlyEqual(TimelineSlider->GetMaxValue(), CurrentMaxFrames))
		{
			TimelineSlider->SetMinAndMaxValues(static_cast<float>(CurrentMinFrames), static_cast<float>(CurrentMaxFrames));
		}

		if (PendingValueChange.IsSet())
		{
			FrameChangedDelegate.ExecuteIfBound(static_cast<int32>(PendingValueChange.GetValue()));
			PendingValueChange.Reset();
		}
	}
}

void SChaosVDTimelineWidget::Pause()
{
	ButtonClickedDelegate.ExecuteIfBound(EChaosVDPlaybackButtonsID::Pause);
}

FReply SChaosVDTimelineWidget::Next()
{
	ButtonClickedDelegate.ExecuteIfBound(EChaosVDPlaybackButtonsID::Next);

	return FReply::Handled();
}

FReply SChaosVDTimelineWidget::Prev()
{	
	ButtonClickedDelegate.ExecuteIfBound(EChaosVDPlaybackButtonsID::Prev);

	return FReply::Handled();
}

const FSlateBrush* SChaosVDTimelineWidget::GetPlayOrPauseIcon() const
{
	return IsPlaying() ? FChaosVDStyle::Get().GetBrush("PauseIcon") : FChaosVDStyle::Get().GetBrush("PlayIcon");
}

EVisibility SChaosVDTimelineWidget::GetElementVisibility(EChaosVDTimelineElementIDFlags ElementID) const
{
	const bool bIsVisible = ElementVisibilityFlags.IsSet() ? EnumHasAnyFlags(ElementVisibilityFlags.Get(), ElementID) : false;
	return bIsVisible ? EVisibility::Visible : EVisibility::Collapsed;
}

bool SChaosVDTimelineWidget::GetElementEnabled(EChaosVDTimelineElementIDFlags ElementID) const
{
	if (!ElementEnabledFlags.IsSet())
	{
		return false;
	}

	
	return EnumHasAnyFlags(ElementEnabledFlags.Get(), ElementID);
}

bool SChaosVDTimelineWidget::IsPlaying() const
{
	return bIsPlaying.IsSet() ? bIsPlaying.Get() : false;
}

int32 SChaosVDTimelineWidget::GetCurrentFrame() const
{
	return CurrentFrame.IsSet() ? CurrentFrame.Get() : INDEX_NONE;
}

float SChaosVDTimelineWidget::GetCurrentFrameAsFloat() const
{
	return static_cast<float>(GetCurrentFrame());
}

int32 SChaosVDTimelineWidget::GetCurrentMinFrames() const
{
	return MinFrames.IsSet() ? MinFrames.Get() : INDEX_NONE;
}

int32 SChaosVDTimelineWidget::GetCurrentMaxFrames() const
{
	return MaxFrames.IsSet() ? MaxFrames.Get() : INDEX_NONE;
}

void SChaosVDTimelineWidget::HandleTimelineScrubStart()
{
	TimelineScrubStartDelegate.ExecuteIfBound();
}

void SChaosVDTimelineWidget::HandleTimelineScrubEnd()
{
	TimelineScrubEndDelegate.ExecuteIfBound();
}

#undef LOCTEXT_NAMESPACE
