// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Widgets/SCompoundWidget.h"
#include "SChaosVDTimelineWidget.generated.h"

class SSlider;
class FReply;
struct FSlateBrush;

UENUM()
enum class EChaosVDPlaybackButtonsID : uint8
{
	Play,
	Pause,
	Stop,
	Next,
	Prev
};

DECLARE_DELEGATE_OneParam(FChaosControlButtonClicked, EChaosVDPlaybackButtonsID)
DECLARE_DELEGATE_OneParam(FChaosVDFrameLockStateDelegate, bool)

enum class EChaosVDSetTimelineFrameFlags
{
	None = 0,
	BroadcastChange = 1 << 0,
	Silent = 1 << 1,
};
ENUM_CLASS_FLAGS(EChaosVDSetTimelineFrameFlags)

enum class EChaosVDTimelineElementIDFlags : uint16
{
	None = 0,
	Play = 1 << 0,
	Stop = 1 << 1,
	Next = 1 << 2,
	Prev = 1 << 3,
	Lock = 1 << 4,
	Timeline = 1 << 5,
	
	ManualSteppingButtons = Next | Prev,
	AllManualStepping = Next | Prev | Timeline,
	AllPlaybackButtons = Play | Stop | Next | Prev,
	AllPlayback = Play | Stop | Next | Prev | Timeline,
	All = Play | Stop | Next | Prev | Timeline | Lock,
};
ENUM_CLASS_FLAGS(EChaosVDTimelineElementIDFlags)

DECLARE_DELEGATE_OneParam(FChaosVDFrameChangedDelegate, int32)
DECLARE_DELEGATE(FChaosVDFrameScrubDelegate)

/** Simple timeline control widget */
class SChaosVDTimelineWidget : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS( SChaosVDTimelineWidget ){}
		SLATE_ATTRIBUTE(int32, MaxFrames)
		SLATE_ATTRIBUTE(int32, MinFrames)
		SLATE_ATTRIBUTE(int32, CurrentFrame)
		SLATE_ARGUMENT(EChaosVDTimelineElementIDFlags, ButtonVisibilityFlags)
		SLATE_ATTRIBUTE(EChaosVDTimelineElementIDFlags, ButtonEnabledFlags)
		SLATE_ATTRIBUTE(bool, IsPlaying)
		SLATE_EVENT(FChaosVDFrameChangedDelegate, OnFrameChanged)
		SLATE_EVENT(FChaosControlButtonClicked, OnButtonClicked)
		SLATE_EVENT(FChaosVDFrameScrubDelegate, OnTimelineScrubStart)
		SLATE_EVENT(FChaosVDFrameScrubDelegate, OnTimelineScrubEnd)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

protected:

	void Play();
	FReply Stop();

	void SetCurrentTimelineFrame(float FrameNumber, EChaosVDSetTimelineFrameFlags Options);

	void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	void Pause();
	FReply  TogglePlay();
	FReply  Next();
	FReply  Prev();

	const FSlateBrush* GetPlayOrPauseIcon() const;

	EVisibility GetElementVisibility(EChaosVDTimelineElementIDFlags ElementID) const;
	bool GetElementEnabled(EChaosVDTimelineElementIDFlags ElementID) const;

	bool IsPlaying() const;

	int32 GetCurrentFrame() const;
	float GetCurrentFrameAsFloat() const;
	int32 GetCurrentMinFrames() const;
	int32 GetCurrentMaxFrames() const;

	void HandleTimelineScrubStart();
	void HandleTimelineScrubEnd();

	TOptional<float> PendingValueChange = 0.0f;

	TSharedPtr<SSlider> TimelineSlider;

	TAttribute<int32> CurrentFrame = 0;
	TAttribute<int32> MinFrames = 0;
	TAttribute<int32> MaxFrames = 1000;
	TAttribute<EChaosVDTimelineElementIDFlags> ElementEnabledFlags = EChaosVDTimelineElementIDFlags::All;

	TAttribute<bool> bIsPlaying = false;

	FChaosVDFrameChangedDelegate FrameChangedDelegate;
	FChaosControlButtonClicked ButtonClickedDelegate;
	FChaosVDFrameScrubDelegate TimelineScrubStartDelegate;
	FChaosVDFrameScrubDelegate TimelineScrubEndDelegate;

	TAttribute<EChaosVDTimelineElementIDFlags> ElementVisibilityFlags = EChaosVDTimelineElementIDFlags::All;

	EChaosVDTimelineElementIDFlags DefaultEnabledElementsFlags =EChaosVDTimelineElementIDFlags::All;
};
