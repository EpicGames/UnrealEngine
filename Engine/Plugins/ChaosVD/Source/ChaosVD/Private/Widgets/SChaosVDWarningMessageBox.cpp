// Copyright Epic Games, Inc. All Rights Reserved.

#include "Widgets/SChaosVDWarningMessageBox.h"

#include "Components/HorizontalBox.h"
#include "Styling/AppStyle.h"

#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

void SChaosVDWarningMessageBox::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("Brushes.Panel"))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			.Padding(2)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Warning"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(2)
			[
				SNew(STextBlock)
				.Justification(ETextJustify::Center)
				.TextStyle(FAppStyle::Get(), "DetailsView.BPMessageTextStyle")
				.Text(InArgs._WarningText)
				.AutoWrapText(true)
			]
		]
	];
}
