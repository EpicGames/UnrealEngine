// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Widgets/SCompoundWidget.h"

class FText;

class SChaosVDWarningMessageBox : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChaosVDWarningMessageBox)
		: _WarningText()
	{}
		/** The  text to show in the warning */
		SLATE_ATTRIBUTE(FText, WarningText)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);
};
