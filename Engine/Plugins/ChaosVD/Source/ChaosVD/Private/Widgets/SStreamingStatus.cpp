// Copyright Epic Games, Inc. All Rights Reserved.

#include "SStreamingStatus.h"

#include "ChaosVDSettingsManager.h"
#include "Components/VerticalBox.h"
#include "Misc/MessageDialog.h"
#include "Settings/ChaosVDGeneralSettings.h"
#include "Styling/AppStyle.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

namespace Chaos::VD
{
void SStreamingStatus::Construct(const FArguments& InArgs)
{
	ChildSlot
	[
		SNew(SVerticalBox)
		+SVerticalBox::Slot()
		.VAlign(VAlign_Bottom)
		.HAlign(HAlign_Right)
		.Padding(4.0f)
		[
			SNew(SButton)
			.ButtonStyle(&FCoreStyle::Get().GetWidgetStyle<FButtonStyle>("HoverHintOnly"))
			.Cursor(EMouseCursor::Default)
			.OnClicked_Raw(this, &SStreamingStatus::OpenToggleStreamingStatePopup)
			.ContentPadding(0.f)
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::Get().GetBrush("FloatingBorder"))
				.Padding(4.f)
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					.Padding(FMargin(2, 1, 0, 1))
					.AutoWidth()
					.VAlign(VAlign_Center)
					[
						SNew(SImage)
						.Image_Raw(this,&SStreamingStatus::GetStreamingStatusImage)
						.DesiredSizeOverride(FVector2D(16, 16))
					]
					+SHorizontalBox::Slot()
					.Padding(FMargin(6, 1, 2, 1))
					.VAlign(VAlign_Center)
					.HAlign(HAlign_Left)
					.FillWidth(1)
					[
						SNew(STextBlock)
						.Text_Raw(this, &SStreamingStatus::GetStreamingStatusText)
						.ToolTipText_Raw(this, &SStreamingStatus::GetStreamingStatusTooltipText)
						.ColorAndOpacity_Raw(this,&SStreamingStatus::GetStreamingStatusColor)
					]
				]
			]
		]
	];

	if (UChaosVDGeneralSettings* GeneralSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>())
	{
		GeneralSettings->OnSettingsChanged().AddSP(this, &SStreamingStatus::HandleSettingsChanged);
		HandleSettingsChanged(GeneralSettings);
	}
}

void SStreamingStatus::HandleSettingsChanged(UObject* SettingsObject)
{
	if (UChaosVDGeneralSettings* GeneralSettings = Cast<UChaosVDGeneralSettings>(SettingsObject))
	{
		bCachedStreamingEnabledState = GeneralSettings->bStreamingSystemEnabled;
	}
}

FText SStreamingStatus::GetStreamingStatusText() const
{
	return bCachedStreamingEnabledState ?  LOCTEXT("StreamingStatusEnabledLabel", "Streaming Enabled") : LOCTEXT("StreamingStatusDisabledLabel", "Streaming Disabled");
}

FSlateColor SStreamingStatus::GetStreamingStatusColor() const
{
	return bCachedStreamingEnabledState ? FAppStyle::Get().GetSlateColor("Colors.AccentGreen") : FAppStyle::Get().GetSlateColor("Colors.Warning");
}

const FSlateBrush* SStreamingStatus::GetStreamingStatusImage() const
{
	return bCachedStreamingEnabledState ? FAppStyle::Get().GetBrush("Icons.SuccessWithColor") :  FAppStyle::Get().GetBrush("Icons.WarningWithColor");
}

FText SStreamingStatus::GetStreamingStatusTooltipText() const
{
	return bCachedStreamingEnabledState ?  LOCTEXT("StreamingStatusEnabledToolTip", "CVD's Streaming System is enabled. Only objects within the configured range are visible. This is the recommended config to get the best performance, but you might want disabled it if you are dbugging Level Streaming related issues")
											: LOCTEXT("StreamingStatusDisabledTooltip", "CVD's Streaming system is disabled. Performance will be degraded");
}

FReply SStreamingStatus::OpenToggleStreamingStatePopup()
{
	UChaosVDGeneralSettings* GeneralSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGeneralSettings>();
	if (!GeneralSettings)
	{
		FMessageDialog::Open(EAppMsgCategory::Error, EAppMsgType::Ok, LOCTEXT("DisableStreamingErrorMessageText", "Failled to obtain the settings object. The streaming state cannot be changed at this moment."));
		return FReply::Handled();
	}
		
	if (bCachedStreamingEnabledState)
	{
		EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgCategory::Warning, EAppMsgType::OkCancel,
														LOCTEXT("DisableStreamingMessageText", "Do you want to disable CVD's Streaming system? \n\nDisabling the streaming system will degrade performance in large recordings \nand is not advised unless you are trying to debug level streaming related issues \nto see an accurate representation of what was loaded at the time the recording was made."),
														LOCTEXT("DisableStreamingTitleText", "CVD Streaming System"));

		if (Response == EAppReturnType::Ok)
		{
			FProperty* SaveToDiskProperty = FindFieldChecked<FProperty>(UChaosVDGeneralSettings::StaticClass(), "bStreamingSystemEnabled");
			GeneralSettings->PreEditChange(SaveToDiskProperty);
			GeneralSettings->bStreamingSystemEnabled = false;
			GeneralSettings->PostEditChange();
		}
	}
	else
	{
		EAppReturnType::Type Response = FMessageDialog::Open(EAppMsgCategory::Info, EAppMsgType::OkCancel,
												LOCTEXT("EnableStreamingMessageText", "Do you want to enable CVD's Streaming system? \n\nEnabling the streaming system is the recommended setting as it will improve performance significantly, \nbut if you are trying to debug level streaming related issues you might need to keep it disabled \nto see an accurate representation of what was loaded at the time the recording was made."),
												LOCTEXT("EnableStreamingTitleText", "CVD Streaming System"));

		if (Response == EAppReturnType::Ok)
		{
			FProperty* SaveToDiskProperty = FindFieldChecked<FProperty>(UChaosVDGeneralSettings::StaticClass(), "bStreamingSystemEnabled");
			GeneralSettings->PreEditChange(SaveToDiskProperty);
			GeneralSettings->bStreamingSystemEnabled = true;
			GeneralSettings->PostEditChange();
		}
	}
	
	return FReply::Handled();
}
}

#undef LOCTEXT_NAMESPACE
