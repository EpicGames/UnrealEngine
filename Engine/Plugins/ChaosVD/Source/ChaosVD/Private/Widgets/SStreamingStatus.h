// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"

namespace Chaos::VD
{
/**
 * Widget to be used as overlay in the main viewport to indicate the user the streaming system status
 */
class SStreamingStatus : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SStreamingStatus)
	{
	}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

private:

	void HandleSettingsChanged(UObject* SettingsObject);
	FText GetStreamingStatusText() const;
	FSlateColor GetStreamingStatusColor() const;
	const FSlateBrush* GetStreamingStatusImage() const;
	FText GetStreamingStatusTooltipText() const;
	FReply OpenToggleStreamingStatePopup();

	bool bCachedStreamingEnabledState = false;
};
}

