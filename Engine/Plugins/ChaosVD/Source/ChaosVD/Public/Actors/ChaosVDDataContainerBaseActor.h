// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "GameFramework/Actor.h"
#include "ChaosVDDataContainerBaseActor.generated.h"

struct FChaosVDSceneCompositionTestData;
struct FChaosVDFrameStageData;
class FChaosVDExtension;
class FChaosVDScene;

struct FChaosVDSolverFrameData;
struct FChaosVDGameFrameData;

namespace Chaos::VD::Test::SceneObjectTypes
{
	extern FName SolverID;
}

/** Base class for any CVD actor that will contain frame related data (either solver frame or game frame) */
UCLASS(MinimalAPI, Abstract, NotBlueprintable, NotPlaceable)
class AChaosVDDataContainerBaseActor : public AActor
{
	GENERATED_BODY()

public:
	CHAOSVD_API AChaosVDDataContainerBaseActor();
	CHAOSVD_API virtual ~AChaosVDDataContainerBaseActor() override;

	CHAOSVD_API virtual void UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData);

	CHAOSVD_API virtual void UpdateFromNewSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData);

	CHAOSVD_API virtual void UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData);

	CHAOSVD_API virtual void Destroyed() override;

	virtual void SetScene(TWeakPtr<FChaosVDScene> InScene)
	{
		SceneWeakPtr = InScene;
	}

	CHAOSVD_API virtual void HandleWorldStreamingLocationUpdated(const FVector& InLocation);
	
	CHAOSVD_API void SetSolverID(int32 InSolverID);

	int32 GetSolverID() const
	{
		return SolverDataID;
	}

	virtual const FTransform& GetSimulationTransform() const
	{
		return FTransform::Identity;
	}

	CHAOSVD_API virtual void UpdateVisibility(bool bIsVisible);

	virtual bool IsVisible() const
	{
		return true;
	}

	CHAOSVD_API virtual void PostActorCreated() override;
	
	TWeakPtr<FChaosVDScene> GetScene() const
	{
		return SceneWeakPtr;
	}

	/**
	 * Flags a CVD Data container actor for game frame data re-routing.
	 * Used by some data container actors to know when the GT data they are processing comes from
	 * a re-routing execution path.
	 */
	struct FScopedGameFrameDataReRouting
	{
		explicit FScopedGameFrameDataReRouting(AChaosVDDataContainerBaseActor* InDataContainerBaseActor)
		{
			check(InDataContainerBaseActor);
			DataContainerBaseActor = InDataContainerBaseActor;
			
			DataContainerBaseActor->bInternallyReRoutingGameFrameData = true;
		}

		~FScopedGameFrameDataReRouting()
		{
			DataContainerBaseActor->bInternallyReRoutingGameFrameData = false;
		}

	private:
		AChaosVDDataContainerBaseActor* DataContainerBaseActor = nullptr;
	};
	
#if WITH_EDITOR
	CHAOSVD_API void SetIsTemporarilyHiddenInEditor(bool bIsHidden) override;
#endif

	/**
	 * Appends Scene Data relevant for functional tests
	 * @param OutStateTestData Structure instance where to add the data
	 * @note If you change the data being added here, you need to re-generate the snapshots used by the
	 * scene integrity playback tests in the Simulation Tests Plugin
	 */
	CHAOSVD_API virtual void AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData);
	
protected:

	CHAOSVD_API void HandlePostInitializationExtensionRegistered(const TSharedRef<FChaosVDExtension>& NewExtension);
	
	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	int32 SolverDataID = INDEX_NONE;

	bool bInternallyReRoutingGameFrameData = false;
};
