// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "DataStorage/Handles.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"
#include "Templates/SharedPointer.h"

#include "ChaosVDBaseSceneObject.generated.h"

class AActor;

/** Base Class for any UStruct based object that has a representation in CVD's World and outliner*/
USTRUCT()
struct FChaosVDBaseSceneObject
{
	GENERATED_BODY()

	virtual ~FChaosVDBaseSceneObject() = default;

	/**
	 * Sets the name that will be used for this object in the scene outliner
	 * @param NewDisplayName New name
	 */
	void SetDisplayName(const FString& NewDisplayName)
	{
		DisplayName = NewDisplayName;
	}

	/**
	 * Sets the name of the icon that will be shown as part of the label for this object in the scene outliner
	 * @param NewIconName Registered name of the icon in an editor style
	 */
	void SetIconName(FName NewIconName)
	{
		IconName = NewIconName;
	}

	/**
	 * Sets a weak reference to another UStruct based scene object, that is the parent of this object
	 * @param NewParent Parent Object
	 */
	virtual void SetParent(const TSharedPtr<FChaosVDBaseSceneObject>& NewParent)
	{
		Parent = NewParent;
	}

	/**
	 * Sets a weak reference to an actor that is the parent of this object.
	 * @param NewParent Parent Actor
	 */
	CHAOSVD_API void SetParentParentActor(AActor* NewParent);


	/**
	 *  Returns a copy the name that will be used for this object in the scene outliner
	 */
	FString GetDisplayName() const
	{
		return DisplayName;
	}

	/**
	 *  Returns a reference to the name that will be used for this object in the scene outliner
	 */
	const FString& GetDisplayNameRef() const
	{
		return DisplayName;
	}

	/**
	 * Returns the registered name of the icon that will be shown as part of the label for this object in the scene outliner
	 */
	FName GetIconName() const
	{
		return IconName;
	}

	/**
	 * Returns a weak ptr to this object's parent
	 */
	TWeakPtr<FChaosVDBaseSceneObject> GetParent() const
	{
		return Parent;
	}

	/**
	 * Returns a weak ptr to this object's parent Actor
	 */
	AActor* GetParentActor() const
	{
		return ParentActor.Get();
	}

	/**
	 * Returns the handle for this object in TEDS
	 */
	UE::Editor::DataStorage::RowHandle GetTedsRowHandle() const
	{
		return CachedRowHandle;
	}

	/**
	 * Sets the handle for this object in TEDS
	 */
	void SetTedsRowHandle(UE::Editor::DataStorage::RowHandle Handle)
	{
		CachedRowHandle = Handle;
	}
	
	enum class EStreamingState
	{
		Visible,
		Hidden
	};

	void SetStreamingState(EStreamingState NewState)
	{
		StreamingState = NewState;
	}
	
	EStreamingState GetStreamingState() const
	{
		return StreamingState;
	}

	virtual void SyncStreamingState()
	{
	}

	virtual FBox GetStreamingBounds() const
	{
		return FBox(EForceInit::ForceInitToZero);
	}

	virtual int32 GetStreamingID() const
	{
		return INDEX_NONE;
	}

protected:
	TWeakPtr<FChaosVDBaseSceneObject> Parent = nullptr;

	TWeakObjectPtr<AActor> ParentActor = nullptr;

	FString DisplayName = TEXT("None");

	UE::Editor::DataStorage::RowHandle CachedRowHandle = UE::Editor::DataStorage::InvalidRowHandle;
	
	FName IconName = NAME_None;

	EStreamingState StreamingState = EStreamingState::Hidden;
};

