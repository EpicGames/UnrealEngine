// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ChaosVDCharacterGroundConstraintDataProviderInterface.generated.h"

struct FChaosVDCharacterGroundConstraint;

// This class does not need to be modified.
UINTERFACE()
class UChaosVDCharacterGroundConstraintDataProviderInterface : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface for any object that is able to provide CVD Character Ground Constraint Data
 */
class IChaosVDCharacterGroundConstraintDataProviderInterface
{
	GENERATED_BODY()

public:
	
	/**
	 * Gathers and populates the provided array with any existing character ground constraint data for this object (if any) 
	 * @param OutConstraintDataFound Array to populate with constraint data
	 */
	virtual void GetCharacterGroundConstraintData(TArray<TSharedPtr<FChaosVDCharacterGroundConstraint>>& OutConstraintsFound) PURE_VIRTUAL(IChaosVDCharacterGroundConstraintDataProviderInterface::GetCharacterGroundConstraintData)

	/**
	 * Checks if the object implementing the interface has any character ground constraint data available
	 * @return True if any constraint data is found 
	 */
	virtual bool HasCharacterGroundConstraintData() PURE_VIRTUAL(IChaosVDCharacterGroundConstraintDataProviderInterface::HasCharacterGroundConstraintData, return false;)

	/**
	 * Returns the name of the object providing the constraint data
	 */
	virtual FName GetProviderName() PURE_VIRTUAL(IChaosVDCharacterGroundConstraintDataProviderInterface::GetProviderName, return FName();)
};
