// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Ticker.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "ChaosVDEngine.generated.h"

namespace Chaos::VisualDebugger
{
struct FChaosVDOptionalDataChannel;
}

namespace UE::TraceBasedDebuggers
{
struct FTraceSessionDescriptor;
}

class FChaosVDPlaybackController;
class FChaosVDScene;
class FChaosVisualDebuggerMainUI;
struct FChaosVDRecording;

/** Enumeration of the available modes controlling how data is loaded into CVD */
UENUM()
enum class EChaosVDLoadRecordedDataMode : uint8
{
	/** This mode will unload any CVD recording currently loaded before loading the selected file */
	SingleSource,
	/** CVD will load and merge the data of the selected recording into the currently loaded recording */
	MultiSource
};

DECLARE_MULTICAST_DELEGATE_OneParam(FTraceSessionStateChangedDelegate, const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor)

PRAGMA_DISABLE_DEPRECATION_WARNINGS
UE_DEPRECATED(5.8, "Use FTraceSessionStateChangedDelegate taking UE::TraceBasedDebuggers::FTraceSessionDescriptor in parameter")
DECLARE_MULTICAST_DELEGATE_OneParam(FSessionStateChangedDelegate, const struct FChaosVDTraceSessionDescriptor& InSessionDescriptor)
PRAGMA_ENABLE_DEPRECATION_WARNINGS

/** Core Implementation of the visual debugger - Owns the systems that are not UI */
class FChaosVDEngine : public FTSTickerObjectBase, public TSharedFromThis<FChaosVDEngine>
{
public:
	CHAOSVD_API FChaosVDEngine();
	CHAOSVD_API virtual ~FChaosVDEngine();

	CHAOSVD_API void Initialize();
	CHAOSVD_API void CloseActiveTraceSessions();
	CHAOSVD_API void CloseSessionByRemoteSessionID(FGuid RemoteSessionID);

	void StopActiveTraceSessions();

	CHAOSVD_API void DeInitialize();

	CHAOSVD_API virtual bool Tick(float DeltaTime) override;

	const FGuid& GetInstanceGuid() const
	{
		return InstanceGUID;
	}

	TSharedPtr<FChaosVDScene>& GetCurrentScene()
	{
		return CurrentScene;
	}

	TSharedPtr<FChaosVDPlaybackController>& GetPlaybackController()
	{
		return PlaybackController;
	}

	bool HasAnyLiveSessionActive() const;

	CHAOSVD_API void LoadRecording(const FString& FilePath, EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource);
	void LoadCombinedMultiRecording(const FString& FilePath);

	bool ConnectToLiveSession(uint32 SessionID, const FString& InSessionAddress, EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource);
	bool ConnectToLiveSession_Direct(FGuid RemoteSessionID, EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource);
	bool ConnectToLiveSession_Relay(FGuid RemoteSessionID, EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource);

	void OpenSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& SessionDescriptor, EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource);

	bool SaveOpenSessionToCombinedFile(const FString& InTargetFilePath = FString());

	bool CanCombineOpenSessions() const;

PRAGMA_DISABLE_DEPRECATION_WARNINGS

	UE_DEPRECATED(5.8, "Use GetTraceSessionDescriptors")
	TArrayView<FChaosVDTraceSessionDescriptor> GetCurrentSessionDescriptors()
	{
		return {};
	}

	UE_DEPRECATED(5.8, "Please use the version that takes a remote Session ID")
	bool ConnectToLiveSession_Direct(EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource);

	UE_DEPRECATED(5.8, "Use the version taking UE::TraceBasedDebuggers::FTraceSessionDescriptor in parameter")
	void OpenSession(const FChaosVDTraceSessionDescriptor& SessionDescriptor, EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource)
	{
	}

	UE_DEPRECATED(5.8, "Use OnTraceSessionOpened instead")
	FSessionStateChangedDelegate& OnSessionOpened()
	{
		static FSessionStateChangedDelegate SessionOpenedDelegate;
		return SessionOpenedDelegate;
	}

	UE_DEPRECATED(5.8, "Use OnTraceSessionClosed instead")
	FSessionStateChangedDelegate& OnSessionClosed()
	{
		static FSessionStateChangedDelegate SessionClosedDelegate;
		return SessionClosedDelegate;
	}

	UE_DEPRECATED(5.8, "Use GetTraceSessionDescriptor")
	const FChaosVDTraceSessionDescriptor& GetSessionDescriptor(FGuid RemoteSessionID) const;

PRAGMA_ENABLE_DEPRECATION_WARNINGS

	FTraceSessionStateChangedDelegate& OnTraceSessionOpened()
	{
		return OnSessionOpenedDelegate;
	}

	FTraceSessionStateChangedDelegate& OnTraceSessionClosed()
	{
		return OnSessionClosedDelegate;
	}

	CHAOSVD_API const UE::TraceBasedDebuggers::FTraceSessionDescriptor& GetTraceSessionDescriptor(FGuid RemoteSessionID) const;

	CHAOSVD_API TArrayView<UE::TraceBasedDebuggers::FTraceSessionDescriptor> GetTraceSessionDescriptors();

private:

	void LoadRecording_Internal(const TFunction<FString(const TSharedPtr<FChaosVDRecording>&)>&, EChaosVDLoadRecordedDataMode LoadingMode = EChaosVDLoadRecordedDataMode::MultiSource);

	void RestoreDataChannelsEnabledStateFromSave();
	void UpdateSavedDataChannelsEnabledState(TWeakPtr<Chaos::VisualDebugger::FChaosVDOptionalDataChannel> DataChannelChanged);

	void UpdateRecentFilesList(const FString& InFilename);

	FGuid InstanceGUID;

	TArray<UE::TraceBasedDebuggers::FTraceSessionDescriptor> CurrentSessionDescriptors;

	TSharedPtr<FChaosVDScene> CurrentScene;
	TSharedPtr<FChaosVDPlaybackController> PlaybackController;
	
	bool bIsInitialized = false;

	FDelegateHandle LiveSessionStoppedDelegateHandle;

	FDelegateHandle DataChannelStateUpdatedHandle;
	
	FTraceSessionStateChangedDelegate OnSessionOpenedDelegate;
	FTraceSessionStateChangedDelegate OnSessionClosedDelegate;
};