// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

//#include "ChaosVDGeometryBuilder.h"
#include "Math/Transform.h"
#include "UObject/NameTypes.h"
#include "UObject/WeakObjectPtr.h"

class FChaosVDGeometryBuilder;
class UDynamicMesh;
class UStaticMesh;

namespace Chaos
{
	class FImplicitObject;
}

/** Structure that acts as a handle to access the generated static mesh of a implicit object, plus other data extracted from it */
struct FChaosVDExtractedGeometryDataHandle
{
	/** Key to be used to match this geometry with recorded Shape Data */
	uint32 GetDataComponentKey() const
	{
		return DataComponentKey;
	}
	void SetDataComponentKey(const uint32 Key)
	{
		DataComponentKey = Key;
	}

	/** Key representing the generated static mesh - Handles for different implicit objects can point to the same geometry key, if after being unpacked they result in essentially the same mesh */
	uint32 GetGeometryKey() const
	{
		return GeometryKey;
	}
	void SetGeometryKey(const uint32 Key);

	/** Name of the type this underlying implicit object this instance represents */
	FName GetTypeName() const;

	/** Extracted transform of the generated implicit object in component space */
	const FTransform& GetRelativeTransform() const
	{
		return Transform;
	}
	void SetTransform(const FTransform& InTransform)
	{
		Transform = InTransform;
	}

	/** Ptr to the implicit object this handle provides access to and from which the mesh was generated */
	const Chaos::FImplicitObject* GetImplicitObject() const
	{
		return ImplicitObject;
	}

	/** Returns the Index to correct the shape data instance in the shape instances array of the particle that owns this geometry */
	int32 GetShapeInstanceIndex() const
	{
		return ShapeInstanceIndex;
	}

	/** If the object was an union and this was a leaf, this getter returns the root union object, otherwise it will be the same as GetImplicitObject */
	const Chaos::FImplicitObject* GetRootImplicitObject() const
	{
		return RootImplicitObject;
	}
	
	/** Sets the Implicit Object Ptr from which the data was extracted and the mesh was generated */
	void SetImplicitObject(const Chaos::FImplicitObject* InImplicitObject)
	{
		ImplicitObject = InImplicitObject;
	}

	/** Sets the Index on which this Implicit object was in the Hierarchy of the root one */
	void SetShapeInstanceIndex(int32 InObjectIndex)
	{
		ShapeInstanceIndex = InObjectIndex;
	}
	
	/** Sets the Implicit Object Ptr from which the data was extracted and the mesh was generated */
	void SetRootImplicitObject(const Chaos::FImplicitObject* InImplicitObject)
	{
		RootImplicitObject = InImplicitObject;
	}

	bool operator==(const FChaosVDExtractedGeometryDataHandle& Other) const
	{
		return Other.GeometryKey == GeometryKey && Other.DataComponentKey == DataComponentKey && Other.Transform.Equals(Transform);
	}

	friend uint32 GetTypeHash(const FChaosVDExtractedGeometryDataHandle& Handle)
	{
		return HashCombine(HashCombine(Handle.GeometryKey, Handle.DataComponentKey), GetTypeHash(Handle.Transform));
	}

private:

	/** Key representing the generated static mesh */
	uint32 GeometryKey = 0;

	/** Key to be used to match this geometry with recorded Shape Data */
	uint32 DataComponentKey = 0;

	/** Index on which this Implicit object was in the Hierarchy of the root one */
	int32 ShapeInstanceIndex = INDEX_NONE;

	/** Extracted transform of the generated implicit object in component space */
	FTransform Transform;

	/** The Implicit Object Ptr from which the data was extracted and the mesh was generated */
	const Chaos::FImplicitObject* ImplicitObject = nullptr;

	/** If the object was an union and this was a leaf, this will be the root union object, otherwise it will be the same as ImplicitObject */
	const Chaos::FImplicitObject* RootImplicitObject = nullptr;
};
