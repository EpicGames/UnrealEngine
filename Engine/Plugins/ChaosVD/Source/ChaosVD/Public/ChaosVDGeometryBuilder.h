// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDExtractedGeometryDataHandle.h"
#include "ChaosVDGeometryDataComponent.h"
#include "ChaosVDHeightfieldMeshGenerator.h"
#include "ChaosVDMeshComponentPool.h"
#include "ChaosVDModule.h"
#include "ChaosVDObjectPool.h"
#include "ChaosVDScene.h"
#include "ObjectsWaitingGeometryList.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectType.h"
#include "Containers/Ticker.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Engine/StaticMesh.h"

#include "Components/ChaosVDInstancedStaticMeshComponent.h"
#include "Components/ChaosVDStaticMeshComponent.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "UObject/WeakObjectPtr.h"
#include "Tasks/Task.h"
#include "Templates/SharedPointer.h"
#include "UObject/GCObject.h"
#include "UObject/UObjectGlobals.h"

class FChaosVDGeometryGenerationTask;
class AActor;
class UDynamicMesh;
class UDynamicMeshComponent;

namespace UE
{
	namespace Geometry
	{
		class FMeshShapeGenerator;
		class FDynamicMesh3;
	}
}

typedef TWeakObjectPtr<UMeshComponent> FMeshComponentWeakPtr;
typedef TSharedPtr<FChaosVDExtractedGeometryDataHandle> FExtractedGeometryHandle;

/** Set of flags used to control how we generate a transform from implicit object data */
enum class EChaosVDGeometryTransformGeneratorFlags
{
	None = 0,
	/** When calculating the adjusted transform, it will generate a scale to represent the actual size of the implicit object */
	UseScaleForSize = 1 << 0,
};
ENUM_CLASS_FLAGS(EChaosVDGeometryTransformGeneratorFlags)

/*
 * Generates Dynamic mesh components and dynamic meshes based on Chaos implicit object data
 */
class FChaosVDGeometryBuilder : public FGCObject, public TSharedFromThis<FChaosVDGeometryBuilder>
{
public:

	FChaosVDGeometryBuilder()
	{
		FInstancedStaticMeshDelegates::OnInstanceIndexUpdated.AddRaw(this, &FChaosVDGeometryBuilder::HandleStaticMeshComponentInstanceIndexUpdated);
	}

	virtual ~FChaosVDGeometryBuilder() override;

	void Initialize(const TWeakPtr<FChaosVDScene>& ChaosVDScene);
	void DeInitialize();

	/** Creates Dynamic Mesh components for each object within the provided Implicit object
	 *	@param InImplicitObject : Implicit object to process
	 *	@param OutMeshDataHandles Array containing all the generated components paired with their corresponding implicit objects
	 *  @param Root ImplicitObjectView to attach children to
	 *	@param DesiredLODCount Number of LODs to generate for the mesh.
	 *	@param MeshIndex Index of the current component being processed. This is useful when this method is called recursively
	 *	@param InTransform to apply to the generated components/geometry
	 */
	void CreateMeshesFromImplicitObject(const Chaos::FImplicitObject* InImplicitObject, TArray<TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>>& OutMeshDataHandles, Chaos::VD::FChaosVDImplicitObjectView* Root, int32 AvailableShapeDataNum = 0, const int32 DesiredLODCount = 0, const Chaos::FRigidTransform3& InTransform = Chaos::FRigidTransform3(), const int32 MeshIndex = 0);

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FChaosVDGeometryBuilder");
	}

	/**
	 * Evaluates an Implicit objects and returns true if it contains an object of the specified type 
	 * @param InImplicitObject Object to evaluate
	 * @param ImplicitTypeToCheck Object type to compare against
	 * @return 
	 */
	static bool DoesImplicitContainType(const Chaos::FImplicitObject* InImplicitObject, const Chaos::EImplicitObjectType ImplicitTypeToCheck);

	/**
	 * Evaluates the provided transform's scale, and returns true if the scale has a negative component
	 * @param InTransform Transform to evaluate
	 */
	static bool HasNegativeScale(const Chaos::FRigidTransform3& InTransform);

private:
	
	void CreateMeshesFromImplicit_Internal(const Chaos::FImplicitObject* InRootImplicitObject,const Chaos::FImplicitObject* InLeafImplicitObject, TArray<TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>>& OutMeshDataHandles, Chaos::VD::FChaosVDImplicitObjectView* Root, const int32 DesiredLODCount = 0, const Chaos::FRigidTransform3& InTransform = Chaos::FRigidTransform3(), const int32 ParentShapeInstanceIndex = 0, int32 AvailableShapeDataNum = 0);
	
public:
	/**
	 * Return true if we have cached geometry for the provided Geometry Key
	 * @param GeometryKey Cache key for the geometry we are looking for
	 */
	bool HasGeometryInCache(uint32 GeometryKey);
	bool HasGeometryInCache_AssumesLocked(uint32 GeometryKey) const;

	/** Returns an already mesh for the provided implicit object if exists, otherwise returns null
	 * @param GeometryCacheKey Key to be used to find this geometry in the cache
	 */
	UStaticMesh* GetCachedMeshForImplicit(const uint32 GeometryCacheKey);

private:

	/** Creates a Dynamic Mesh for the provided Implicit object and generator, and then caches it to be reused later
	 * @param GeometryCacheKey Key to be used to find this geometry in the cache
	 * @param MeshGenerator Generator class with the data and rules to create the mesh
	 * @param LODsToGenerateNum Number of LODs to generate for this static mesh
	 */
	UStaticMesh* CreateAndCacheStaticMesh(const uint32 GeometryCacheKey, UE::Geometry::FMeshShapeGenerator& MeshGenerator, const int32 LODsToGenerateNum = 0);

	/** Takes a Mesh component ptr and initializes it to be used with the provided owner
	 * @param Owner Actor that will own the provided Mesh Component
	 * @param MeshComponent Component to initialize
	 */
	template <class ComponentType>
    bool InitializeMeshComponent(AActor* Owner, ComponentType* MeshComponent);

	/** Sets the correct material for the provided Geometry component based on its configuration
	 * @param GeometryComponent Component that needs its materia updated
	 */
	void SetMeshComponentMaterial(IChaosVDGeometryComponent* GeometryComponent);

public:

	void HandleNewGeometryData(const Chaos::FConstImplicitObjectPtr& Geometry, const uint32 GeometryID);

	/**
	 * Finds or creates a Mesh component for the geometry data handle provided, and add a new instance of that geometry to it
	 * @param InOwningParticleData Particle data from which the implicit object is from
	 * @param InExtractedGeometryDataHandle Handle to the extracted geometry data the new component will use
	 * */
	template<typename ComponentType>
	TSharedPtr<FChaosVDInstancedMeshData> CreateMeshDataInstance(const FChaosVDParticleDataWrapper& InOwningParticleData, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InExtractedGeometryDataHandle);

	/**
	 * Finds or creates a Mesh component compatible with the provided mesh data handle, and updates the handle to use that new component.
	 * This is used when data in the handle changed and becomes no longer compatible with the mesh component in use.
	 * @param HandleToUpdate Instance handle we need to update to a new component
	 * @param MeshAttributes Attributes of the mesh that the new component needs to be compativle with
	 * */
	template<typename ComponentType>
	void UpdateMeshDataInstance(const TSharedRef<FChaosVDInstancedMeshData>& InHandleToUpdate, EChaosVDMeshAttributesFlags MeshAttributes);

	/**
	 * Destroys a Mesh component that will not longer be used.
	 * If pooling is enabled, the component will be reset and added back to the pool
	 * @param MeshComponent Component to Destroy
	 * */
	void DestroyMeshComponent(UMeshComponent* MeshComponent);

	/** Enqueues a component to have its material updated based on its configuration
	 * @param MeshComponent Component to Update
	 */
	void RequestMaterialUpdate(UMeshComponent* MeshComponent);

private:
	
	void CachePreBuiltMeshes();

	/** Gets a ptr to a fully initialized Mesh component compatible with the provided geometry handle and mesh attribute flags, ready to accept a new mesh instance
	 * @param GeometryDataHandle Handle with the data of the geometry to be used for the new instance
	 * @param MeshAttributes Set of flags that the mesh component needs to be compatible with
	 */
	template<typename ComponentType>
	ComponentType* GetMeshComponentForNewInstance(const TSharedRef<FChaosVDExtractedGeometryDataHandle>& GeometryDataHandle, EChaosVDMeshAttributesFlags MeshAttributes);

	/** Gets a reference to the correct instanced static mesh component cache that is compatible with the provided mesh attribute flags
	 * @param MeshAttributeFlags GeometryHandle this component will render
	 */
	TMap<uint32, UChaosVDInstancedStaticMeshComponent*>& GetInstancedStaticMeshComponentCacheMap(EChaosVDMeshAttributesFlags MeshAttributeFlags);

	/** Gets any available instanced static mesh component that is compatible with the provided mesh attributes and component type
	 * @param InExtractedGeometryDataHandle GeometryHandle this component will render
	 * @param MeshComponentsContainerActor Actor that will owns the mesh component
	 * @param MeshComponentAttributeFlags Set of flags that the mesh component needs to be compatible with
	 * @param bOutIsNewComponent True if the component we are returning is new. False if it is an existing one but that can take a new mesh instance
	 */
	template <typename ComponentType>
	ComponentType* GetAvailableInstancedStaticMeshComponent(const TSharedPtr<FChaosVDExtractedGeometryDataHandle>& InExtractedGeometryDataHandle, AActor* MeshComponentsContainerActor, EChaosVDMeshAttributesFlags MeshComponentAttributeFlags, bool& bOutIsNewComponent);

	/** Gets any available mesh component that is compatible with the provided mesh attributes and component type
	 * @param InExtractedGeometryDataHandle GeometryHandle this component will render
	 * @param MeshComponentsContainerActor Actor that will owns the mesh component
	 * @param MeshComponentAttributeFlags Set of flags that the mesh component needs to be compatible with
	 * @param bOutIsNewComponent True if the component we are returning is new. False if it is an existing one but that can take a new mesh instance
	 */
	template <typename ComponentType>
	ComponentType* GetAvailableMeshComponent(const TSharedPtr<FChaosVDExtractedGeometryDataHandle>& InExtractedGeometryDataHandle, AActor* MeshComponentsContainerActor, EChaosVDMeshAttributesFlags MeshComponentAttributeFlags, bool& bOutIsNewComponent);

	/**
	 * Applies a mesh to a mesh component based on its type
	 * @param MeshComponent Mesh component to apply the mesh to
	 * @param GeometryKey Key to find the mesh to apply in the cache
	 */
	bool ApplyMeshToComponentFromKey(TWeakObjectPtr<UMeshComponent> MeshComponent, const uint32 GeometryKey);

public:
	/** Creates a mesh generator for the provided Implicit object which will be used to create a Static Mesh
	 * @param InImplicit ImplicitObject used a data source for the mesh generator
	 * @param SimpleShapesComplexityFactor Factor used to reduce or increase the complexity (number of triangles generated) of simple shapes (Sphere/Capsule)
	 */
	TSharedPtr<UE::Geometry::FMeshShapeGenerator> CreateMeshGeneratorForImplicitObject(const Chaos::FImplicitObject* InImplicit, float SimpleShapesComplexityFactor = 1.0f);

	/** Returns true if the implicit object if of one of the types we need to unpack before generating a mesh for it */
	bool ImplicitObjectNeedsUnpacking(const Chaos::FImplicitObject* InImplicitObject) const;

	/** Unwraps the provided Implicit object into the object it self so a mesh can be generated from it
	 * @param InImplicitObject Object to unwrap
	 * @param InOutTransform Transform that was extracted from the implicit object. Might be unchanged if the implicit proved was not transformed or scaled
	 */
	const Chaos::FImplicitObject* UnpackImplicitObject(const Chaos::FImplicitObject* InImplicitObject, Chaos::FRigidTransform3& InOutTransform) const;
	
	/** Re-adjust the provided transform if needed, so it can be visualized properly with its generated mesh
	 * @param InImplicit Source implicit object
	 * @param OutAdjustedTransform Updated transform
	 */
	void AdjustedTransformForImplicit(const Chaos::FImplicitObject* InImplicit, FTransform& OutAdjustedTransform, EChaosVDGeometryTransformGeneratorFlags Options = EChaosVDGeometryTransformGeneratorFlags::None);

private:

	/** Extracts data from an implicit object in a format CVD can use, and starts the Mesh generation process if needed
	 * @return Returns a handle to the generated data that can be used to access the generated mesh when ready
	 */
	TSharedPtr<FChaosVDExtractedGeometryDataHandle> ExtractGeometryDataForImplicit(const Chaos::FImplicitObject* InImplicitObject, const Chaos::FRigidTransform3& InTransform);

	/**
	 * Creates a Mesh from the provided Implicit object geometry data. This is a async operation, and the mesh will be assigned to the component once is ready
	 * @param GeometryCacheKey Key to be used to find this geometry in the cache
	 * @param ImplicitObject Implicit object to use a sdata source to create the static mesh
	 * @param LODsToGenerateNum Num of LODs to Generate. Not all mesh types support this
	 * */
	void DispatchCreateAndCacheMeshForImplicitAsync(const uint32 GeometryCacheKey, const Chaos::FImplicitObject* ImplicitObject, const int32 LODsToGenerateNum = 0);

	/* Process an FImplicitObject and returns de desired geometry type. Could be directly the shape or another version of the implicit */
	template <bool bIsInstanced, typename GeometryType>
	const GeometryType* GetGeometry(const Chaos::FImplicitObject* InImplicit, const bool bIsScaled, Chaos::FRigidTransform3& OutTransform) const;

	/** Process an FImplicitObject and returns de desired geometry type based on the packed object flags. Could be directly the shape or another version of the implicit */
	template<typename GeometryType>
	const GeometryType* GetGeometryBasedOnPackedType(const Chaos::FImplicitObject* InImplicitObject, Chaos::FRigidTransform3& Transform, const Chaos::EImplicitObjectType PackedType) const;

	/** Tick method of this Geometry builder. Used to do everything that needs to be performed in the GT, like applying the generated meshes to mesh component */
	bool GameThreadTick(float DeltaTime);

	/**
	 * Add a mesh component to the waiting list for Geometry. This needs to be called before dispatching a generation job for new Geometry
	 * @param GeometryKey Key of the geometry that is being generated (or will be generated)
	 * @param MeshComponent Component where the geometry needs to be applied
	 */
	void AddMeshComponentWaitingForGeometry(uint32 GeometryKey, TWeakObjectPtr<UMeshComponent> MeshComponent) const;

	/**
	 * Add a mesh component to the waiting list for Geometry. This needs to be called before dispatching a generation job for new Geometry
	 * @param GeometryKey Key of the geometry that is being generated (or will be generated)
	 * @param MeshComponent Component where the geometry needs to be applied
	 */
	void RemoveMeshComponentWaitingForGeometry(uint32 GeometryKey, TWeakObjectPtr<UMeshComponent> MeshComponent) const;

	void RequestMeshForComponent(const TSharedRef<FChaosVDExtractedGeometryDataHandle>& SourceGeometry, UMeshComponent* MeshComponent);

	/** Returns a reference to the Mesh components pool used by this builder */
	FChaosVDMeshComponentPool& GetMeshComponentDataPool()
	{
		return ComponentMeshPool;
	}

	/** Returns true if the provided Implicit object type can use a pre-built static mesh*
	 * @param ObjetType Type to evaluate
	 */
	bool UsesPreBuiltGeometry(Chaos::EImplicitObjectType ObjetType) const;

	/** Handles any changes to the indexes of created instanced mesh components we are managing, making corrections/updates as needed */
	void HandleStaticMeshComponentInstanceIndexUpdated(UInstancedStaticMeshComponent* InComponent, TArrayView<const FInstancedStaticMeshDelegates::FInstanceIndexUpdateData> InIndexUpdates);

	/** Map containing already generated static mesh for any given implicit object */
	TMap<uint32, TObjectPtr<UStaticMesh>> StaticMeshCacheMap;

	/** Set of all geometry keys of the Meshes that are being generated but not ready yet */
	TMap<uint32, TSharedPtr<FChaosVDGeometryGenerationTask>> GeometryBeingGeneratedByKey;
	
	/** Used to lock Read or Writes to the Geometry cache and in flight job tracking containers */
	FRWLock GeometryCacheRWLock;

	/** Handle to the ticker used to ticker the Geometry Builder in the game thread*/
	FTSTicker::FDelegateHandle GameThreadTickDelegate;

	/** Object containing all the meshes component waiting for geometry, by geometry key*/
	TUniquePtr<FObjectsWaitingGeometryList<FMeshComponentWeakPtr>> MeshComponentsWaitingForGeometry;

	TUniquePtr<FObjectsWaitingProcessingQueue<FMeshComponentWeakPtr>> MeshComponentsWaitingForMaterial;

	TUniquePtr<FObjectsWaitingProcessingQueue<TSharedPtr<FChaosVDGeometryGenerationTask>>> GeometryTasksPendingLaunch;

	/** Map containing already initialized Instanced static mesh components for any given geometry key */
	TMap<uint32, UChaosVDInstancedStaticMeshComponent*> InstancedMeshComponentByGeometryKey;

	/** Map containing already initialized Instanced static mesh components ready to be use with translucent materials, for any given geometry key */
	TMap<uint32, UChaosVDInstancedStaticMeshComponent*> TranslucentInstancedMeshComponentByGeometryKey;

	/** Map containing already initialized Instanced static mesh components for mesh instances that required a negative scale transform, for any given geometry key */
	TMap<uint32, UChaosVDInstancedStaticMeshComponent*> MirroredInstancedMeshComponentByGeometryKey;

	/** Map containing already initialized Instanced static mesh components for mesh instances that required a negative scale transform and use a translucent material, for any given geometry key */
	TMap<uint32, UChaosVDInstancedStaticMeshComponent*> TranslucentMirroredInstancedMeshComponentByGeometryKey;

	/** Components that need to be processed and added to the pool in the next frame */
	TArray<TObjectPtr<UMeshComponent>> MeshComponentsPendingDisposal;

	/** Instance of uninitialized mesh components pool */
	FChaosVDMeshComponentPool ComponentMeshPool;

	/** Weak Ptr to the CVD scene owning this geometry builder */
	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	TObjectPtr<UStaticMesh> BasicShapesMeshes[Chaos::ImplicitObjectType::ConcreteObjectCount];	

	bool bInitialized = false;

	struct FSourceGeometryHashCache
	{
		/** Returns the hash for the provided implicit object and caches it if it is the first time we see it.
		 * @param ImplicitObject Implicit Object from which get the hash from
		 */
		[[nodiscard]] uint32 GetAndCacheGeometryHash(const Chaos::FImplicitObject* ImplicitObject)
		{
			if (!ImplicitObject)
			{
				return 0;
			}

			{
				FReadScopeLock ReadLock(CacheLock);
				if (uint32* FoundHash = CachedGeometryHashes.Find(ImplicitObject))
				{
					return *FoundHash;
				}
			}

			const uint32 GeometryHash = ImplicitObject->GetTypeHash();
			CacheImplicitObjectHash(ImplicitObject, GeometryHash);

			return GeometryHash;
		}

		/** Returns true if we have the hash for the provided implicit object in cache
		 * @param ImplicitObject Implicit Object to cache
		 */
		bool HasGeometryInHashCache(const Chaos::FImplicitObject* ImplicitObject)
		{
			FReadScopeLock ReadLock(CacheLock);
			return CachedGeometryHashes.Contains(ImplicitObject);
		}

		/** Caches the provided hash linking it to the provided implicit object
		 * @param ImplicitObject Implicit Object from which the has was calculated from
		 * @param Hash Hash from the Implicit Object
		 */
		void CacheImplicitObjectHash(const Chaos::FImplicitObject* ImplicitObject, uint32 Hash)
		{
			if (!ImplicitObject)
			{
				return;
			}
	
			FWriteScopeLock WriteLock(CacheLock);
			CachedGeometryHashes.Add(ImplicitObject, Hash);
		}

		/** Clears the hash cache */
		void Reset()
		{
			FWriteScopeLock WriteLock(CacheLock);
			CachedGeometryHashes.Reset();
		}

		FRWLock CacheLock;
		TMap<const void*, uint32> CachedGeometryHashes;
	};

	FSourceGeometryHashCache SourceGeometryCache;
	
	friend class FChaosVDGeometryGenerationTask;
};

/** Used to execute each individual Geometry Generation task using the data with which was constructed.
 * It allows the task to skip the actual generation attempt if the Geometry builder instance goes away which happens when the tool is closed
 */
class FChaosVDGeometryGenerationTask
{
public:
	FChaosVDGeometryGenerationTask(const TWeakPtr<FChaosVDGeometryBuilder>& InBuilder, const uint32 GeometryKey, const Chaos::FImplicitObject* ImplicitObject,
		const int32 LODsToGenerateNum)
		: Builder(InBuilder),
		  ImplicitObject(ImplicitObject),
		  GeometryKey(GeometryKey),
		  LODsToGenerateNum(LODsToGenerateNum),
		  bIsCanceled(false)
	{
	}

	void GenerateGeometry();

	uint32 GetGeometryKey() const
	{
		return GeometryKey;
	}

	bool IsCanceled() { return bIsCanceled; }
	void CancelTask(){ bIsCanceled = true; }

	UE::Tasks::FTask TaskHandle;

private:
	TWeakPtr<FChaosVDGeometryBuilder> Builder;
	const Chaos::FImplicitObject* ImplicitObject;
	uint32 GeometryKey;
	int32 LODsToGenerateNum;
	std::atomic<bool> bIsCanceled;
};

template <typename ComponentType>
bool FChaosVDGeometryBuilder::InitializeMeshComponent(AActor* Owner, ComponentType* MeshComponent)
{
	if (!ensure(MeshComponent))
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed To Create mesh component | Component Is Null. ", ANSI_TO_TCHAR(__FUNCTION__));
		return false;
	}

	if (Owner)
	{
		Owner->AddOwnedComponent(MeshComponent);

		if (ensure(!MeshComponent->IsRegistered()))
		{
			MeshComponent->RegisterComponent();
		}
		
		MeshComponent->AttachToComponent(Owner->GetRootComponent(), FAttachmentTransformRules::SnapToTargetIncludingScale);

		MeshComponent->bSelectable = true;
	}
	else
	{
		UE_LOGF(LogChaosVDEditor, Error, "[%ls] Failed To Register Component | Owner Is Null. ", ANSI_TO_TCHAR(__FUNCTION__));
		return false;
	}

	return true;
}

template <typename ComponentType>
ComponentType* FChaosVDGeometryBuilder::GetAvailableInstancedStaticMeshComponent(const TSharedPtr<FChaosVDExtractedGeometryDataHandle>& InExtractedGeometryDataHandle, AActor* MeshComponentsContainerActor, EChaosVDMeshAttributesFlags MeshComponentAttributeFlags, bool& bOutIsNewComponent)
{
	// Get the correct Instanced Mesh Component from the existing cache
	TMap<uint32, UChaosVDInstancedStaticMeshComponent*>& InstancedMeshComponentMapToSearch = GetInstancedStaticMeshComponentCacheMap(MeshComponentAttributeFlags);

	if (UChaosVDInstancedStaticMeshComponent** FoundInstancedMeshComponent = InstancedMeshComponentMapToSearch.Find(InExtractedGeometryDataHandle->GetGeometryKey()))
	{
		bOutIsNewComponent = false;
		return Cast<ComponentType>(*FoundInstancedMeshComponent);
	}
	else
	{
		// If no existing component meets our requirements, get a new one form the pool
		ComponentType* Component = ComponentMeshPool.AcquireMeshComponent<ComponentType>(MeshComponentsContainerActor, InExtractedGeometryDataHandle->GetTypeName());

		bOutIsNewComponent = true;

		InstancedMeshComponentMapToSearch.Add(InExtractedGeometryDataHandle->GetGeometryKey(), Component);

		return Component;
	}
}

template <typename ComponentType>
ComponentType* FChaosVDGeometryBuilder::GetAvailableMeshComponent(const TSharedPtr<FChaosVDExtractedGeometryDataHandle>& InExtractedGeometryDataHandle, AActor* MeshComponentsContainerActor, EChaosVDMeshAttributesFlags MeshComponentAttributeFlags, bool& bOutIsNewComponent)
{
	ComponentType* MeshComponent = nullptr;
	if constexpr (std::is_base_of_v<UInstancedStaticMeshComponent, ComponentType>)
	{
		MeshComponent = GetAvailableInstancedStaticMeshComponent<ComponentType>(InExtractedGeometryDataHandle, MeshComponentsContainerActor, MeshComponentAttributeFlags, bOutIsNewComponent);
	}
	else
	{
		MeshComponent = ComponentMeshPool.AcquireMeshComponent<ComponentType>(MeshComponentsContainerActor, InExtractedGeometryDataHandle->GetTypeName());
		bOutIsNewComponent = true;
	}

	if (bOutIsNewComponent)
	{
		if (!InitializeMeshComponent<ComponentType>(MeshComponentsContainerActor, MeshComponent))
		{
			return nullptr;
		}
	}

	return MeshComponent;
}

template <typename ComponentType>
TSharedPtr<FChaosVDInstancedMeshData> FChaosVDGeometryBuilder::CreateMeshDataInstance(const FChaosVDParticleDataWrapper& InOwningParticleData, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InExtractedGeometryDataHandle)
{
	static_assert(std::is_base_of_v<UChaosVDStaticMeshComponent, ComponentType> || std::is_base_of_v<UChaosVDInstancedStaticMeshComponent, ComponentType>, "CreateMeshComponentsFromImplicit Only supports CVD versions of Static MeshComponent and Instanced Static Mesh Component");

	const FTransform ExtractedGeometryTransform = InExtractedGeometryDataHandle->GetRelativeTransform();

	EChaosVDMeshAttributesFlags MeshComponentAttributeFlags = EChaosVDMeshAttributesFlags::None;
	if (HasNegativeScale(ExtractedGeometryTransform))
	{
		EnumAddFlags(MeshComponentAttributeFlags, EChaosVDMeshAttributesFlags::MirroredGeometry);
	}

	TSharedPtr<FChaosVDInstancedMeshData> MeshComponentHandle;
	ComponentType* Component = GetMeshComponentForNewInstance<ComponentType>(InExtractedGeometryDataHandle, MeshComponentAttributeFlags);

	if (IChaosVDGeometryComponent* CVDGeometryComponent = Cast<IChaosVDGeometryComponent>(Component))
	{
		constexpr bool bIsWorldSpace = true;
		const FTransform OwningParticleTransform(InOwningParticleData.ParticlePositionRotation.MR, InOwningParticleData.ParticlePositionRotation.MX);
		MeshComponentHandle = CVDGeometryComponent->AddMeshInstance(OwningParticleTransform, bIsWorldSpace, InExtractedGeometryDataHandle, InOwningParticleData.ParticleIndex, InOwningParticleData.SolverID);
		
		MeshComponentHandle->SetGeometryBuilder(AsWeak());
	}

	return MeshComponentHandle;
}

template <typename ComponentType>
void FChaosVDGeometryBuilder::UpdateMeshDataInstance(const TSharedRef<FChaosVDInstancedMeshData>& InHandleToUpdate, EChaosVDMeshAttributesFlags MeshAttributes)
{
	static_assert(std::is_base_of_v<UChaosVDStaticMeshComponent, ComponentType> || std::is_base_of_v<UChaosVDInstancedStaticMeshComponent, ComponentType>, "CreateMeshComponentsFromImplicit Only supports CVD versions of Static MeshComponent and Instanced Static Mesh Component");

	ComponentType* Component = GetMeshComponentForNewInstance<ComponentType>(InHandleToUpdate->GetGeometryHandle(), MeshAttributes);

	if (IChaosVDGeometryComponent* CVDGeometryComponent = Cast<IChaosVDGeometryComponent>(Component))
	{
		CVDGeometryComponent->AddExistingMeshInstance(InHandleToUpdate);
	}
}

template <typename ComponentType>
ComponentType* FChaosVDGeometryBuilder::GetMeshComponentForNewInstance(const TSharedRef<FChaosVDExtractedGeometryDataHandle>& GeometryDataHandle, EChaosVDMeshAttributesFlags MeshAttributes)
{
	AActor* MeshComponentsContainerActor = nullptr;
	
	if (const TSharedPtr<FChaosVDScene> CVDScene = SceneWeakPtr.Pin())
	{
		MeshComponentsContainerActor = CVDScene->GetMeshComponentsContainerActor();
	}

	if (!MeshComponentsContainerActor)
	{
		return nullptr;
	}

	bool bIsNew = false;
	ComponentType* Component = GetAvailableMeshComponent<ComponentType>(GeometryDataHandle, MeshComponentsContainerActor, MeshAttributes, bIsNew);

	if (IChaosVDGeometryComponent* CVDGeometryComponent = Cast<IChaosVDGeometryComponent>(Component))
	{
		if (!CVDGeometryComponent->IsMeshReady())
		{
			RequestMeshForComponent(GeometryDataHandle, Component);
		}

		if (bIsNew)
		{
			CVDGeometryComponent->SetGeometryBuilder(AsWeak());
			CVDGeometryComponent->SetMeshComponentAttributeFlags(MeshAttributes);
			CVDGeometryComponent->Initialize();
			CVDGeometryComponent->OnComponentEmpty()->AddRaw(this, &FChaosVDGeometryBuilder::DestroyMeshComponent);			
		}
	}

	return Component;
}

template <bool bIsInstanced, typename GeometryType>
const GeometryType* FChaosVDGeometryBuilder::GetGeometry(const Chaos::FImplicitObject* InImplicitObject, const bool bIsScaled, Chaos::FRigidTransform3& OutTransform) const
{
	if (bIsScaled)
	{
		if (const Chaos::TImplicitObjectScaled<GeometryType, bIsInstanced>* ImplicitScaled = InImplicitObject->template GetObject<Chaos::TImplicitObjectScaled<GeometryType, bIsInstanced>>())
		{
			OutTransform.SetScale3D(ImplicitScaled->GetScale());
			return ImplicitScaled->GetUnscaledObject()->template GetObject<GeometryType>();
		}
	}
	else
	{
		if (bIsInstanced)
		{
			const Chaos::TImplicitObjectInstanced<GeometryType>* ImplicitInstanced = InImplicitObject->template GetObject<Chaos::TImplicitObjectInstanced<GeometryType>>();
			return ImplicitInstanced->GetInnerObject()->template GetObject<GeometryType>();
		}
		else
		{
			return InImplicitObject->template GetObject<GeometryType>();
		}
	}
	
	return nullptr;
}

template <typename GeometryType>
const GeometryType* FChaosVDGeometryBuilder::GetGeometryBasedOnPackedType(const Chaos::FImplicitObject* InImplicitObject, Chaos::FRigidTransform3& Transform,  const Chaos::EImplicitObjectType PackedType) const
{
	using namespace Chaos;

	const bool bIsInstanced = IsInstanced(PackedType);
	const bool bIsScaled = IsScaled(PackedType);

	if (bIsInstanced)
	{
		return GetGeometry<true,GeometryType>(InImplicitObject, bIsScaled, Transform);
	}
	else
	{
		return GetGeometry<false, GeometryType>(InImplicitObject, bIsScaled, Transform);
	}
}
