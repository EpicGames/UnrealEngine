// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDExtractedGeometryDataHandle.h"
#include "Chaos/ImplicitObject.h"
#include "Components/MeshComponent.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"
#include "HAL/Platform.h"
#include "InstancedStaticMeshDelegates.h"
#include "Templates/SharedPointer.h"
#include "UObject/Interface.h"
#include "UObject/StrongObjectPtr.h"

#include "ChaosVDGeometryDataComponent.generated.h"

class UChaosVDParticleVisualizationSettings;
class UChaosVDParticleVisualizationColorSettings;
struct FChaosVDSceneParticle;
class FChaosVDGeometryBuilder;
class FChaosVDScene;
class IChaosVDGeometryComponent;
class UChaosVDInstancedStaticMeshComponent;
class UChaosVDStaticMeshComponent;
class UMaterialInstanceDynamic;

struct FChaosVDExtractedGeometryDataHandle;
struct FChaosVDParticleDataWrapper;

namespace Chaos
{
	class FImplicitObject;
}

DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDMeshReadyDelegate, IChaosVDGeometryComponent&)
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDMeshComponentEmptyDelegate, UMeshComponent*)

UENUM()
enum class EChaosVDMaterialType
{
	SMOpaque,
	SMTranslucent,
	ISMCOpaque,
	ISMCTranslucent
};

UENUM()
enum class EChaosVDMeshComponent
{
	Invalid,
	Static,
	InstancedStatic,
	Dynamic
};

/** Struct holding the a minimum amount of data about a Implicit object to be shown in the details panel */
USTRUCT()
struct FChaosVDImplicitObjectBasicView
{
	GENERATED_BODY()

	/** Geometry type name*/
	UPROPERTY(VisibleAnywhere, Category="Recorded GeometryData")
	FName ImplicitObjectType;

	Chaos::EImplicitObjectType ImplicitObjectTypeEnum = Chaos::ImplicitObjectType::Unknown;

	/** Index of the Shape Instance data for this geometry in the Shape Instance data array */
	UPROPERTY(VisibleAnywhere, Category="Recorded GeometryData")
	int32 ShapeInstanceIndex = INDEX_NONE;

	/** True if this is the root implicit object */
	UPROPERTY(VisibleAnywhere, Category="Recorded GeometryData")
	bool bIsRootObject = false;

	/** If this is a transformed implicit, this will contain the recorded relative transform */
	UPROPERTY(VisibleAnywhere, Category="Recorded GeometryData")
	FTransform RelativeTransform;
};

/** Struct holding the state of a mesh instance - Is separated from the Mesh instance class so we can show the data in the Details panel */
USTRUCT()
struct FChaosVDMeshDataInstanceState
{
	GENERATED_BODY()

	/** Recorded Shape instance Data */
	UPROPERTY(VisibleAnywhere, Category="Recorded GeometryData")
	FChaosVDShapeCollisionData CollisionData;

	/** Minimum set of data about the recorded implicit object */
	UPROPERTY(VisibleAnywhere, Category="Recorded GeometryData")
	FChaosVDImplicitObjectBasicView ImplicitObjectInfo;

	/* CVD Debug - Current world transform used to render this Mesh */
	UPROPERTY(VisibleAnywhere, Category="CVD GeometryData", meta=(EditCondition=bShowCVDDebugData, EditConditionHides))
	FTransform CurrentWorldTransform;

	/* CVD Debug - Current mesh component type to render this Mesh */
	UPROPERTY(VisibleAnywhere, Category="CVD GeometryData", meta=(EditCondition=bShowCVDDebugData, EditConditionHides))
	EChaosVDMeshComponent MeshComponentType = EChaosVDMeshComponent::Invalid;

	/* CVD Debug - Pointer to the mesh component used to render this Mesh */
	UPROPERTY()
	TObjectPtr<UMeshComponent> MeshComponent;

	/* CVD Debug - Instance index of mesh component used to render this Mesh */
	UPROPERTY(VisibleAnywhere, Category="CVD GeometryData", meta=(EditCondition=bShowCVDDebugData, EditConditionHides))
	int32 MeshInstanceIndex = INDEX_NONE;

	/* CVD Debug - Color used to render this mesh */
	UPROPERTY(VisibleAnywhere, Category="CVD GeometryData", meta=(EditCondition=bShowCVDDebugData, EditConditionHides))
	FLinearColor CurrentGeometryColor = FLinearColor(ForceInitToZero);

	/* CVD Debug - Id of the particle this geometry belongs */
	UPROPERTY(VisibleAnywhere, Category="CVD GeometryData", meta=(EditCondition=bShowCVDDebugData, EditConditionHides))
	int32 OwningParticleID = INDEX_NONE;

	/* CVD Debug - Id of the solver this geometry belongs */
	UPROPERTY(VisibleAnywhere, Category="CVD GeometryData", meta=(EditCondition=bShowCVDDebugData, EditConditionHides))
	int32 OwningSolverID = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category="CVD GeometryData", meta=(EditCondition=bShowCVDDebugData, EditConditionHides))
	bool bIsVisible = true;

	UPROPERTY(VisibleAnywhere, Category="CVD GeometryData", meta=(EditCondition=bShowCVDDebugData, EditConditionHides))
	bool bIsSelected = false;

	UPROPERTY(EditAnywhere, Category="CVD GeometryData")
	bool bShowCVDDebugData = false;
};

/** Structure that represents a specific mesh instance on a CVD Mesh component (instanced or static) */
class FChaosVDInstancedMeshData : public TSharedFromThis<FChaosVDInstancedMeshData>
{
public:
	explicit FChaosVDInstancedMeshData(int32 InInstanceIndex, UMeshComponent* InMeshComponent, int32 InParticleID, int32 InSolverID, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InSourceGeometryHandle);

	/** Returns the Particle ID of the particle owning this mesh instance */
	int32 GetOwningParticleID() const
	{
		return InstanceState.OwningParticleID;
	}

	/** Returns the Solver ID of the particle owning this mesh instance */
	int32 GetOwningSolverID() const
	{
		return InstanceState.OwningSolverID;
	}

	/** Applies the provided world transform to the mesh instance this handle represents
	 * @param InTransform Transform to apply
	 */
	void SetWorldTransform(const FTransform& InTransform);

	/** Returns the world transform of the mesh instance this handle represents */
	const FTransform& GetWorldTransform() const
	{
		return InstanceState.CurrentWorldTransform;
	}

	/** Returns the geometry handle used to create the mesh instance this handle represents */
	const TSharedRef<FChaosVDExtractedGeometryDataHandle>& GetGeometryHandle() const
	{
		return ExtractedGeometryHandle;
	}

	/** Applies to the provided color to the mesh instance this handle represents */
	void SetInstanceColor(const FLinearColor& NewColor);

	/** Returns the current color of the mesh instance this handle represents */
	FLinearColor GetInstanceColor() const
	{
		return InstanceState.CurrentGeometryColor;
	}
	
	/** Applies to the provided shape collision data to the mesh instance this handle represents */
	void UpdateMeshComponentForCollisionData(const FChaosVDShapeCollisionData& InCollisionData);

	/** Returns the mesh component used to render the mesh instance this handle represents */
	UMeshComponent* GetMeshComponent() const
	{
		return InstanceState.MeshComponent;
	}

	/** Returns the instance index of the mesh instance this handle represents */
	int32 GetMeshInstanceIndex() const
	{
		return InstanceState.MeshInstanceIndex;
	}

	/** Returns the type of the component used to render the mesh instance this handle represents */
	EChaosVDMeshComponent GetMeshComponentType() const
	{
		return InstanceState.MeshComponentType;
	}

	/** Sets a Ptr to the geometry builder used to generate and manage the geometry/mesh components this handle represents */
	void SetGeometryBuilder(const TWeakPtr<FChaosVDGeometryBuilder>& InGeometryBuilder)
	{
		GeometryBuilderInstance = InGeometryBuilder;
	}
	TWeakPtr<FChaosVDGeometryBuilder> GetGeometryBuilder()
	{
		return GeometryBuilderInstance;
	}

	/** Marks this mesh instance as selected. Used to handle Selection in Editor */
	void SetIsSelected(bool bInIsSelected);

	/** Sets the visibility of this mesh instance */
	void SetVisibility(bool bInIsVisible);

	/** Returns the current visibility state this mesh instance */
	bool GetVisibility() const
	{
		return InstanceState.bIsVisible;
	}

	/** Applies a new shape collision data to this mesh instance */
	void SetGeometryCollisionData(const FChaosVDShapeCollisionData&& InCollisionData);

	/** Returns the current shape collision data to this mesh instance */
	FChaosVDShapeCollisionData& GetGeometryCollisionData()
	{
		return InstanceState.CollisionData;
	}

	FChaosVDMeshDataInstanceState& GetState()
	{
		return InstanceState;
	}

	/** Used only for debugging purposes - It will be set to true if we received new Shape Instance data but the Shape Index for the implicit object we represent is not valid */
	bool bFailedToUpdateShapeInstanceData = false;
	
	/** Sets the mesh instance index of the mesh instance this handle represents */
	void SetMeshInstanceIndex(int32 NewIndex)
	{
		InstanceState.MeshInstanceIndex = NewIndex;
	}

	/** Returns true if this instance is queued to be destroyed */
	bool IsPendingDestroy() const
	{
		return bIsPendingDestroy;
	}

	/** Marks a instance that is queued to be destroyed at the end of the frame */
	void MarkPendingDestroy()
	{
		bIsPendingDestroy = true;
	}

private:

	/** Sets the mesh component used to render the mesh instance this handle represents */
	void SetMeshComponent(UMeshComponent* NewComponent)
	{
		InstanceState.MeshComponent = NewComponent;
	}

	FChaosVDMeshDataInstanceState InstanceState;

	bool bIsPendingDestroy = false;

	TSharedRef<FChaosVDExtractedGeometryDataHandle> ExtractedGeometryHandle;

	TWeakPtr<FChaosVDGeometryBuilder> GeometryBuilderInstance = nullptr;

	friend UChaosVDInstancedStaticMeshComponent;
	friend UChaosVDStaticMeshComponent;
};


namespace Chaos::VD
{
	/** Class that represents an Implicit Object to be shown in the Geometry Hierarchy Viewer */
	class FChaosVDImplicitObjectView : public TSharedFromThis<FChaosVDImplicitObjectView>
	{
	public:
		Chaos::EImplicitObjectType ImplicitObjectType;
		FChaosVDImplicitObjectBasicView ImplicitObjectInfo;

		void AddChild(TSharedPtr<FChaosVDImplicitObjectView> Child)
		{
			Children->Add(Child);
		}

		TSharedPtr<TArray<TSharedPtr<FChaosVDImplicitObjectView>>> GetChildren()
		{
			return Children;
		}

		void SetMeshData(TSharedPtr<FChaosVDInstancedMeshData> InMeshData)
		{
			MeshData = InMeshData;
		}

		TSharedPtr<FChaosVDInstancedMeshData> GetMeshData()
		{
			return MeshData;
		}

	private:
		TSharedPtr<TArray<TSharedPtr<FChaosVDImplicitObjectView>>> Children = MakeShared<TArray<TSharedPtr<FChaosVDImplicitObjectView>>>();
		TSharedPtr<FChaosVDInstancedMeshData> MeshData;
	};
}

enum class EChaosVDMeshAttributesFlags : uint8
{
	None = 0,
	MirroredGeometry = 1 << 0,
	TranslucentGeometry = 1 << 1
};

ENUM_CLASS_FLAGS(EChaosVDMeshAttributesFlags)

UINTERFACE()
class UChaosVDGeometryComponent : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface with a default implementation for any Geometry component that
 * contains CVD data
 */
class IChaosVDGeometryComponent
{
	GENERATED_BODY()

public:

	/** Returns the Geometry Handle used to identify the geometry data this component represents */
	virtual uint32 GetGeometryKey() const PURE_VIRTUAL(IChaosVDGeometryDataComponent::GetGeometryKey, return 0;);

	/** Returns the CVD Mesh Data Instance handle for the provided Instance index */
	virtual TSharedPtr<FChaosVDInstancedMeshData> GetMeshDataInstanceHandle(int32 InstanceIndex) const PURE_VIRTUAL(IChaosVDGeometryDataComponent::GetMeshDataInstanceHandle, return nullptr;);

	/**
	 * Add a new instance to this mesh component
	 * @param InstanceTransform The transform the new instance will use
	 * @param bIsWorldSpace True if the provide transform is in world space
	 * @param InGeometryHandle The CVD Geometry Handle with data about the geometry instantiate
	 * @param ParticleID Particle ID owning this geometry
	 * @param SolverID Solver ID of the particle owning this geometry
	 * @return CVD Mesh instance handle that provides access to this component and specific instance, allowing manipulation of it
	 */
	virtual TSharedPtr<FChaosVDInstancedMeshData> AddMeshInstance(const FTransform InstanceTransform, bool bIsWorldSpace, const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InGeometryHandle, int32 ParticleID, int32 SolverID) PURE_VIRTUAL(IChaosVDGeometryDataComponent::AddMeshInstance, return nullptr;);

	/**
	 * Adds a new instance to this mesh component, but using an existing Mesh Data Handle instead of creating a new one
	 * @param InMeshDataHandle Handle that will provide access to this mesh instance
	 */
	virtual void AddExistingMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InMeshDataHandle) PURE_VIRTUAL(IChaosVDGeometryDataComponent::AddExistingMeshInstance);

	enum class ERemovalMode
	{
		Instant,
		Deferred
	};

	/**
	 * Removes the instance the provided handle represents
	 * @param InHandleToRemove 
	 */
	virtual void RemoveMeshInstance(const TSharedRef<FChaosVDInstancedMeshData>& InHandleToRemove, ERemovalMode Mode = ERemovalMode::Deferred) PURE_VIRTUAL(IChaosVDGeometryDataComponent::RemoveMeshInstance);

	/** True if the mesh this component represents is ready for use */
	virtual bool IsMeshReady() const  PURE_VIRTUAL(IChaosVDGeometryDataComponent::IsMeshReady, return false;);

	/** Sets if the mesh this component represents is ready for use or not */
	virtual void SetIsMeshReady(bool bIsReady) PURE_VIRTUAL(IChaosVDGeometryDataComponent::SetIsMeshReady);
	
	/** Triggers when the mesh this component represents is ready */
	virtual FChaosVDMeshReadyDelegate* OnMeshReady() PURE_VIRTUAL(IChaosVDGeometryDataComponent::OnMeshReady, return nullptr;);

	/** Triggers when the component does not have any instance to render. Used to allow it to return to the mesh component tool for future re-use*/
	virtual FChaosVDMeshComponentEmptyDelegate* OnComponentEmpty() PURE_VIRTUAL(IChaosVDGeometryDataComponent::OnComponentEmpty, return nullptr;);

	/** Updates the visibility of this component based on the stored CVD data */
	virtual void UpdateVisibilityForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) PURE_VIRTUAL(IChaosVDGeometryDataComponent::UpdateVisibilityForInstance);

	/** Changes the selection state of the provided instance - Used for Selection in Editor */
	virtual void UpdateSelectionStateForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) PURE_VIRTUAL(IChaosVDGeometryDataComponent::UpdateSelectionStateForInstance);

	/** Updates the colors of this component based on the stored CVD data */
	virtual void UpdateColorForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) PURE_VIRTUAL(IChaosVDGeometryDataComponent::UpdateColorForInstance);

	/** Updates the colors of this component based on the stored CVD data */
	virtual void UpdateWorldTransformForInstance(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle) PURE_VIRTUAL(IChaosVDGeometryDataComponent::UpdateWorldTransformForInstance);

	/** Sets the CVD Mesh Attribute flags this component is compatible with*/
	virtual void SetMeshComponentAttributeFlags(EChaosVDMeshAttributesFlags Flags) PURE_VIRTUAL(IChaosVDGeometryDataComponent::SetMeshComponentAttributeFlags);
	
	/** Returns the CVD Mesh Attribute flags this component is compatible with*/
	virtual EChaosVDMeshAttributesFlags GetMeshComponentAttributeFlags() const PURE_VIRTUAL(IChaosVDGeometryDataComponent::GetMeshComponentAttributeFlags, return EChaosVDMeshAttributesFlags::None;);

	/** Resets the state of this mesh component, so it can be re-used later on */
	virtual void Reset() PURE_VIRTUAL(IChaosVDGeometryDataComponent::Reset);

	virtual void Initialize() PURE_VIRTUAL(IChaosVDGeometryDataComponent::Initialize);
	
	/** Sets a Ptr to the geometry builder used to generate and manage the geometry/mesh components */
	virtual void SetGeometryBuilder(TWeakPtr<FChaosVDGeometryBuilder> GeometryBuilder) PURE_VIRTUAL(IChaosVDGeometryDataComponent::SetGeometryBuilder);

	virtual EChaosVDMaterialType GetMaterialType() const PURE_VIRTUAL(IChaosVDGeometryDataComponent::GetMaterialType,  return EChaosVDMaterialType::SMOpaque;);

	virtual bool GetIsDestroyed() const PURE_VIRTUAL(IChaosVDGeometryDataComponent::GetIsDestroyed,  return false;);

	virtual void SetIsDestroyed(bool bNewIsDestroyed) PURE_VIRTUAL(IChaosVDGeometryDataComponent::SetIsDestroyed);
};

class FChaosVDGeometryComponentUtils
{
public:
	/** Finds and updates the Shape data using the provided array as source*/
	static void UpdateCollisionDataFromShapeArray(TArrayView<FChaosVDShapeCollisionData> InShapeArray, const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle);

	/** Calculates and updates the color used to render the mesh represented by the provided handle, based on the particle state */
	static void UpdateMeshColor(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle, const FChaosVDParticleDataWrapper& InParticleData, bool bIsServer);

	/** Calculates the correct visibility state based on the particle state, and applies it to the mesh instance the provided handle represents */
	static void UpdateMeshVisibility(const TSharedRef<FChaosVDInstancedMeshData>& InInstanceHandle, const FChaosVDParticleDataWrapper& InParticleData, bool bIsActive);

	/** Returns the material to use as a base to create material instances for the provided type */
	static UMaterialInterface* GetBaseMaterialForType(EChaosVDMaterialType Type);

	/** Returns the correct material type to use based on the provided Component type and Mesh Attributes */
	template<typename TComponent>
	static EChaosVDMaterialType GetMaterialTypeForComponent(EChaosVDMeshAttributesFlags MeshAttributes);

private:
	/** Returns the color that needs to be used to present the provided particle data based on its state and current selected options */
	static FLinearColor GetGeometryParticleColor(const TSharedRef<FChaosVDExtractedGeometryDataHandle>& InGeometryHandle, const FChaosVDParticleDataWrapper& InParticleData, bool bIsServer);

	static const UChaosVDParticleVisualizationColorSettings* GetParticleColorSettings();
	static const UChaosVDParticleVisualizationSettings* GetParticleVisualizationSettings();

	static TObjectPtr<UChaosVDParticleVisualizationColorSettings> CachedParticleColorSettings;
	static TObjectPtr<UChaosVDParticleVisualizationSettings> CachedParticleVisualizationSettings;
};

template <typename TComponent>
EChaosVDMaterialType FChaosVDGeometryComponentUtils::GetMaterialTypeForComponent(EChaosVDMeshAttributesFlags MeshAttributes)
{
	constexpr bool bIsInstancedMeshComponent = std::is_base_of_v<UInstancedStaticMeshComponent, TComponent>;
	if (EnumHasAnyFlags(MeshAttributes, EChaosVDMeshAttributesFlags::TranslucentGeometry))
	{
		return bIsInstancedMeshComponent ? EChaosVDMaterialType::ISMCTranslucent : EChaosVDMaterialType::SMTranslucent;
	}
	else
	{
		return bIsInstancedMeshComponent ? EChaosVDMaterialType::ISMCOpaque : EChaosVDMaterialType::SMOpaque;
	}
}

namespace Chaos::VisualDebugger
{
	void SelectParticleWithGeometryInstance(const TSharedRef<FChaosVDScene>& InScene, FChaosVDSceneParticle* Particle, const TSharedPtr<FChaosVDInstancedMeshData>& InMeshDataHandle);
}
