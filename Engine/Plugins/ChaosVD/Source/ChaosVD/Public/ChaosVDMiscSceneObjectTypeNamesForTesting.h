// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

namespace Chaos::VD::Test::SceneObjectTypes
{
	extern FName SceneQuery;
	extern FName CollisionConstraint;
	extern FName JointConstraint;
	extern FName CharacterGroundConstraint;
}
