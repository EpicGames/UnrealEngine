// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "Modules/ModuleManager.h"
#include "ExtensionsSystem/ChaosVDExtension.h"
#include "Features/IModularFeatures.h"
#include "Misc/Guid.h"

class SWidget;
class FChaosVDEngine;
class SChaosVDMainTab;
class SDockTab;
class FSpawnTabArgs;
struct FGuid;

namespace UE::TraceBasedDebuggers
{
class FTraceSessionsManager;
}

/** Main module class for the Chaos Visual Debugger editor */
DECLARE_LOG_CATEGORY_EXTERN(LogChaosVDEditor, Log, VeryVerbose);
class FChaosVDModule : public IModuleInterface
{
public:

	using ToolkitHostType = SChaosVDMainTab;

	CHAOSVD_API static FChaosVDModule& Get();

	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	/** Returns the Trace manager instance used by CVD */
	TSharedPtr<UE::TraceBasedDebuggers::FTraceSessionsManager>& GetTraceSessionsManager()
	{
		return ChaosVDTraceManager;
	}

	UE_DEPRECATED(5.8, "Use GetTraceSessionsManager")
	TSharedPtr<class FChaosVDTraceManager> GetTraceManager()
	{
		return nullptr;
	}

	/** Spawns a new instance of CVD */
	void SpawnCVDTab();

	static bool IsStandaloneChaosVisualDebugger();

	/** Re-instantiates CVD's widgets without restarting the underlying CVD engine instance */
	void ReloadInstanceUI(FGuid InstanceGUID);

	void TakePlaybackEngineStateSnapshot();

private:

	TSharedRef<SDockTab> SpawnMainTab(const FSpawnTabArgs& Args);

	void HandleTabClosed(TSharedRef<SDockTab> ClosedTab, FGuid InstanceGUID);

	void RegisterChaosVDEngineInstance(const FGuid& InstanceGuid, TSharedPtr<FChaosVDEngine> Instance);
	void DeregisterChaosVDEngineInstance(const FGuid& InstanceGuid);

	void RegisterChaosVDTabInstance(const FGuid& InstanceGuid, TSharedPtr<SDockTab> Instance);
	void DeregisterChaosVDTabInstance(const FGuid& InstanceGuid);
	
	void CloseActiveInstances();
	
	TMap<FGuid, TSharedPtr<FChaosVDEngine>> ActiveChaosVDInstances;

	TMap<FGuid, TWeakPtr<SDockTab>> ActiveCVDTabs;

	TSharedPtr<UE::TraceBasedDebuggers::FTraceSessionsManager> ChaosVDTraceManager;

	static FString ChaosVisualDebuggerProgramName;

	bool bIsShuttingDown = false;
};


