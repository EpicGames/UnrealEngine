// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDSceneSelectionObserver.h"
#include "Templates/SharedPointer.h"
#include "ChaosVDTabSpawnerBase.h"
#include "Delegates/IDelegateInstance.h"
#include "Widgets/SChaosVDDetailsView.h"
#include "ChaosVD/Private/Widgets/SChaosVDGeometryTree.h"

struct FChaosVDSolverDataSelectionHandle;
struct FChaosVDParticleDebugData;

class AActor;
class FName;
class FSpawnTabArgs;
class FTabManager;
class SChaosVDDetailsView;
class SDockTab;

/** Spawns and handles an instance for a selection independent details panel */
class FChaosVDStandAloneObjectDetailsTab : public FChaosVDTabSpawnerBase, public TSharedFromThis<FChaosVDStandAloneObjectDetailsTab>
{
public:

	FChaosVDStandAloneObjectDetailsTab(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget) : FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
	{
	}
	
	TSharedPtr<SChaosVDDetailsView> GetDetailsPanel()
	{
		return DetailsPanelView;
	}

	CHAOSVD_API void AddUnsupportedStruct(const UStruct* Struct);
	
	/** Updates the current object this view details is viewing */
	template<typename TStruct>
	void SetStructToInspect(TStruct* NewStruct);

	/** Updates the label shown on the docked tab header */
	CHAOSVD_API void UpdateTabLabel(const FText& NewLabel);

protected:

	CHAOSVD_API virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	CHAOSVD_API virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

	CHAOSVD_API bool IsSupportedStruct(const TWeakObjectPtr<const UStruct>& InWeakStructPtr);

	void UpdateTreeDetailsSplit(const FChaosVDSceneParticle* Particle);

	TSharedPtr<SChaosVDDetailsView> DetailsPanelView;
	TSharedPtr<SChaosVDGeometryTree> GeometryTreeWidget;
	TSharedPtr<SSplitter> DetailsSplitter;
	float SavedSplitSize = 0.2f;

	TSet<TWeakObjectPtr<const UStruct>> UnsupportedStructs;

	TWeakPtr<SDockTab> SpawnedTab;
};

template <typename TStruct>
void FChaosVDStandAloneObjectDetailsTab::SetStructToInspect(TStruct* NewStruct)
{
	DetailsPanelView->SetSelectedStruct(NewStruct);
}

/** Spawns and handles an instance for the visual debugger details panel */
class FChaosVDObjectDetailsTab : public FChaosVDStandAloneObjectDetailsTab, public FChaosVDSceneSelectionObserver
{
public:

	FChaosVDObjectDetailsTab(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget) : FChaosVDStandAloneObjectDetailsTab(InTabID, InTabManager, InOwningTabWidget)
	{
	}

protected:

	CHAOSVD_API virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	CHAOSVD_API virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;

	CHAOSVD_API virtual void HandlePostSelectionChange(const UTypedElementSelectionSet* ChangedSelectionSet) override;

	CHAOSVD_API void HandleActorsSelection(TArrayView<AActor*> SelectedActors);

	CHAOSVD_API virtual void HandleSolverDataSelectionChange(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& SelectionHandle);

	FDelegateHandle SelectionDelegateHandle;
	TWeakObjectPtr<UObject> CurrentSelectedObject = nullptr;
};
