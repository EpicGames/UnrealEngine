// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDRecording.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "Templates/SharedPointer.h"
#include "Delegates/DelegateCombinations.h"
#include "Delegates/Delegate.h"
#include "ChaosVDPlaybackController.generated.h"

struct FChaosVDPlaybackEngineSnapshot;
class IChaosVDPlaybackControllerInstigator;
class UChaosVDCoreSettings;
struct FChaosVDTrackInfo;
class FChaosVDScene;
struct FChaosVDRecording;
class FChaosVDPlaybackController;
class FString;

namespace UE::TraceBasedDebuggers
{
struct FTraceSessionDescriptor;
}

enum class EChaosVDPlaybackButtonsID : uint8;

DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDInitialFirstFramePlayed, TWeakPtr<FChaosVDPlaybackController>)
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDPlaybackControllerUpdated, TWeakPtr<FChaosVDPlaybackController>)
DECLARE_MULTICAST_DELEGATE_ThreeParams(FChaosVDPlaybackControllerFrameUpdated, TWeakPtr<FChaosVDPlaybackController>, TWeakPtr<const FChaosVDTrackInfo>, FGuid);

/** Enum with the available game track types*/
enum class EChaosVDTrackType : int32
{
	Invalid,
	Game,
	Solver,
	/** Used mostly for search */
	All
};

/** Data that represents the current state of a track and ID info*/
struct FChaosVDTrackInfo
{
	int32 TrackID = INDEX_NONE;
	int32 TrackSlot = INDEX_NONE;
	EChaosVDTrackType TrackType = EChaosVDTrackType::Invalid;
	int32 CurrentFrame = INDEX_NONE;
	int32 CurrentStage = INDEX_NONE;
	int32 LockedOnStage = INDEX_NONE;
	int32 MaxFrames = INDEX_NONE;
	FName TrackName;
	TArray<FStringView> CurrentStageNames;
	bool bIsReSimulated = false;
	bool bIsPlaying = false;
	bool bTrackSyncEnabled = true;
	bool bIsServer = false;
	bool bHasNetworkSyncData = false;
	bool bSupportsVisibilityChange = true;
	bool bCanShowTrackControls = true;

	bool operator==(const FChaosVDTrackInfo& Other) const;

	static bool AreSameTrack(const TSharedRef<const FChaosVDTrackInfo>& TrackA,  const TSharedRef<const FChaosVDTrackInfo>& TrackB);

	bool IsValidTrack() const
	{
		return TrackID != INDEX_NONE;
	}
};

struct FChaosVDQueuedTrackInfoUpdate
{
	TWeakPtr<const FChaosVDTrackInfo> TrackInfo;
	FGuid InstigatorID;
};

struct FChaosVDGeometryDataUpdate
{
	Chaos::FConstImplicitObjectPtr NewGeometry;
	uint32 GeometryID;
};

/** Flags used to control how the unload of a recording is performed */
enum class EChaosVDUnloadRecordingFlags : uint8
{
	None = 0, 
	BroadcastChanges = 1 << 0,
	Silent = 1 << 1
};
ENUM_CLASS_FLAGS(EChaosVDUnloadRecordingFlags)

/** Available sync modes that determine how tracks will sync between each other during playback */
UENUM()
enum class EChaosVDSyncTimelinesMode : uint8
{
	None UMETA(Hidden),
	/** Syncs all tracks using the recorded timestamp */
	RecordedTimestamp,
	/** Syncs all tracks using the recorded network ticks offset of the Predictive networked physics system*/
	NetworkTick,
	/** No-Auto sync is performed and all available solver tracks inside the recording are visible at once and can be scrubbed
	 * independently
	 */
	Manual
};

typedef TMap<int32, TSharedPtr<FChaosVDTrackInfo>> TrackInfoByIDMap;

/** Loads,unloads and owns a Chaos VD recording file */
class FChaosVDPlaybackController : public TSharedFromThis<FChaosVDPlaybackController>, public FTSTickerObjectBase
{
public:

	/** ID used for the Game Track */
	static constexpr int32 GameTrackID  = 0;
	static constexpr int32 GameTrackSlot  = 0;
	static constexpr int32 InvalidFrameRateOverride  = -1;
	static constexpr float FallbackFrameTime = 1.0f / 60.0f;

	static inline FGuid PlaybackSelfInstigatorID = FGuid::NewGuid();

	FChaosVDPlaybackController(const TWeakPtr<FChaosVDScene>& InSceneToControl);
	virtual ~FChaosVDPlaybackController() override;

	void Initialize();
	void DeInitialize();

	/** Loads a recording using a CVD Trace Session Descriptor */
	bool LoadChaosVDRecordingFromTraceSession(const UE::TraceBasedDebuggers::FTraceSessionDescriptor& InSessionDescriptor);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use the version taking UE::TraceBasedDebuggers::FTraceSessionDescriptor in parameter")
	bool LoadChaosVDRecordingFromTraceSession(const struct FChaosVDTraceSessionDescriptor& InSessionDescriptor);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** Unloads the currently loaded recording
	 * @param UnloadOptions Options flags to change the steps performed during the unload
	 */
	void UnloadCurrentRecording(EChaosVDUnloadRecordingFlags UnloadOptions = EChaosVDUnloadRecordingFlags::BroadcastChanges);

	/** Returns true if the controller has a valid recording loaded*/
	bool IsRecordingLoaded() const
	{
		return LoadedRecording.IsValid();
	}

	/** Returns a weak ptr to the Scene this controller is controlling during playback */
	TWeakPtr<FChaosVDScene> GetControllerScene()
	{
		return SceneToControl;
	}

	/**
	 * Moves a track of the recording to the specified stage and frame numbers
	 * @param InstigatorID ID of the system that requested the move
	 * @param TrackType Type of the track to move
	 * @param InTrackID ID of the track to move
	 * @param FrameNumber Frame number to go to
	 * @param StageNumber Stage number to go to
	 */
	CHAOSVD_API void GoToTrackFrame(FGuid InstigatorID, EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber, int32 StageNumber);
	void GoToTrackFrame_AssumesLocked(FGuid InstigatorID, EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber, int32 StageNumber);

	/**
	 * Moves a track of the recording to the specified stage and frame numbers. And then syncs every other track to it
	 * @param InstigatorID ID of the system that requested the move
	 * @param TrackType Type of the track to move
	 * @param InTrackID ID of the track to move
	 * @param FrameNumber Frame number to go to
	 * @param StageNumber Stage number to go to
	 */
	CHAOSVD_API void GoToTrackFrameAndSync(FGuid InstigatorID, EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber, int32 StageNumber);
	void GoToTrackFrame_AssumesLockedAndSync(FGuid InstigatorID, EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber, int32 StageNumber);

	/**
	 * Gets the number of available stages in a track at the specified frame
	 * @param TrackType Type of the track to evaluate
	 * @param InTrackID ID of the track to evaluate
	 * @param FrameNumber Frame number to evaluate
	 * @return Number of available Stages
	 */
	int32 GetTrackStagesNumberAtFrame_AssumesLocked(EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber) const;

	/**
	 * Gets the available stages container in a track at the specified frame
	 * @param TrackType Type of the track to evaluate
	 * @param InTrackID ID of the track to evaluate
	 * @param FrameNumber Frame number to evaluate
	 * @return Ptr to the stages data container
	 */
	const FChaosVDFrameStagesContainer* GetTrackStagesDataAtFrame_AssumesLocked(EChaosVDTrackType TrackType, int32 InTrackID, int32 FrameNumber) const;

	/**
	 * Gets the number of available frames for the specified track
	 * @param TrackType Type of the track to evaluate
	 * @param InTrackID ID of the track to evaluate
	 * @return Number of available frames
	 */
	int32 GetTrackFramesNumber(EChaosVDTrackType TrackType, int32 InTrackID) const;
	int32 GetTrackFramesNumber_AssumesLocked(EChaosVDTrackType TrackType, int32 InTrackID) const;
	
	/**
	 * Gets the current frame number at which the specified track is at
	 * @param TrackType Type of the track to evaluate
	 * @param InTrackID ID of the track to evaluate
	 * @return Current frame number for the track
	 */
	int32 GetTrackCurrentFrame(EChaosVDTrackType TrackType, int32 InTrackID) const;

	/**
	 * Gets the number of available frames for the specified track
	 * @param TrackType Type of the track to evaluate
	 * @param InTrackID ID of the track to evaluate
	 * @return Number of available frames
	 */
	int32 GetTrackCurrentStage(EChaosVDTrackType TrackType, int32 InTrackID) const;

	/**
	 * Gets the index number of the last stage available (available stages -1)
	 * @param TrackType Type of the track to evaluate
	 * @param InTrackID ID of the track to evaluate
	 * @return Number of the last stages
	 */
	int32 GetTrackLastStageAtFrame(EChaosVDTrackType TrackType, int32 InTrackID, int32 InFrameNumber) const;
	int32 GetTrackLastStageAtFrame_AssumesLocked(EChaosVDTrackType TrackType, int32 InTrackID, int32 InFrameNumber) const;

	/** Converts the current frame number of a track, to a frame number in other tracks space time
	 * @param InFromTrack Track info with the current frame number we want to convert
	 * @param InToTrack Track info we want to use to convert the frame to
	 * @param TrackSyncMode Criteria or "mode" that should be used to sync a track frame with another
	 * @return Converted Frame Number
	 */
	int32 ConvertCurrentFrameToOtherTrackFrame_AssumesLocked(const TSharedRef<const FChaosVDTrackInfo>& InFromTrack, const TSharedRef<const FChaosVDTrackInfo>& InToTrack, EChaosVDSyncTimelinesMode TrackSyncMode = EChaosVDSyncTimelinesMode::RecordedTimestamp);

	/**
	 * Gets all the ids of the tracks, of the specified type, that are available available on the loaded recording
	 * @param TrackType Type of the tracks we are interested in
	 * @param OutTrackInfo Array where any found track info data will be added
	 */

	void GetAvailableTracks(EChaosVDTrackType TrackType, TArray<TSharedPtr<const FChaosVDTrackInfo>>& OutTrackInfo);
	void GetMutableAvailableTracks(EChaosVDTrackType TrackType, TArray<TSharedPtr<FChaosVDTrackInfo>>& OutTrackInfo);
	
	/**
	 * Gets all the ids of the tracks, of the specified type, that are available available on the loaded recording, at a specified frame
	 * @param TrackTypeToFind Type of the tracks we are interested in
	 * @param OutTrackInfo Array where any found track info data will be added
	 * @param InFromTrack Ptr to the track info with the current frame to evaluate
	 */
	void GetAvailableTrackInfosAtTrackFrame(EChaosVDTrackType TrackTypeToFind, const TSharedRef<const FChaosVDTrackInfo>& InFromTrack, TArray<TSharedPtr<const FChaosVDTrackInfo>>& OutTrackInfo);
	void GetAvailableTrackInfosAtTrackFrame_AssumesLocked(EChaosVDTrackType TrackTypeToFind, const TSharedRef<const FChaosVDTrackInfo>& InFromTrack, TArray<TSharedPtr<const FChaosVDTrackInfo>>& OutTrackInfo);

	/**
	 * Gets the track info of the specified type with the specified ID
	 * @param TrackType Type of the track to find
	 * @param TrackID ID of the track to find
	 * @return Ptr to the found track info data - Null if nothing was found
	 */
	TSharedPtr<const FChaosVDTrackInfo> GetTrackInfo(EChaosVDTrackType TrackType, int32 TrackID);

	/**
	 * Gets the track info of the specified type with the specified ID
	 * @param TrackType Type of the track to find
	 * @param TrackID ID of the track to find
	 * @return Ptr to the found track info data - Null if nothing was found.
	 */
	TSharedPtr<FChaosVDTrackInfo> GetMutableTrackInfo(EChaosVDTrackType TrackType, int32 TrackID);

	/**
	 * Locks the stages timeline of a given track so each time you move between frames, it will automatically scrub to the locked in stage
	 * @param TrackType Type of the track to find
	 * @param TrackID ID of the track to find
	 */
	void LockTrackInCurrentStage(EChaosVDTrackType TrackType, int32 TrackID);

	/**
	 * UnLocks the stages timeline of a given track so each time you move between frames, it will automatically scrub to the default stage
	 * @param TrackType Type of the track to find
	 * @param TrackID ID of the track to find
	 */
	void UnlockTrackStage(EChaosVDTrackType TrackType, int32 TrackID);

	/** Returns a weak ptr pointer to the loaded recording */
	TWeakPtr<FChaosVDRecording> GetCurrentRecording()
	{
		return LoadedRecording;
	}

	/** Called when data on the recording being controlled gets updated internally or externally (for example, during Trace Analysis)*/
	FChaosVDPlaybackControllerUpdated& OnDataUpdated()
	{
		return ControllerUpdatedDelegate;
	}

	/** Called when a frame on a track is updated */
	FChaosVDPlaybackControllerFrameUpdated& OnTrackFrameUpdated()
	{
		return ControllerFrameUpdatedDelegate;
	}

	FChaosVDInitialFirstFramePlayed& OnFirstFramePlayed()
	{
		return ControllerFirstFramePlayedDelegate;
	}
	
	CHAOSVD_API virtual bool Tick(float DeltaTime) override;

	/** Returns true if we are playing a live debugging session */
	bool IsPlayingLiveSession() const;

	/** Updates the loaded recording state to indicate is not longer receiving live updates */
	void HandleDisconnectedFromSession();

	/**
	 * Stops the playback (if active)
	 * @param InstigatorGUID ID of the system that is making the stop attempt
	 */
	void StopPlayback(const FGuid& InstigatorGUID);

	/** Returns if we are playing back at a custom framerate instead of the recorded framerate */
	bool IsUsingFrameRateOverride() const
	{
		return bUseFrameRateOverride;
	}

	/** Toggles on or off the framerate override feature */
	bool ToggleUseFrameRateOverride()
	{
		return bUseFrameRateOverride =  !bUseFrameRateOverride;
	}

	/** Returns the frame rate override value, as frame time */
	float GetFrameTimeOverride() const;
	/** Returns the frame rate override value */
	int32 GetFrameRateOverride() const;

	/** Sets the desired framerate override value */
	void SetFrameRateOverride(float NewFrameRateOverride);

	/**
	 * Returns the the recorded frame time for the provided track info
	 * @param TrackType Type of the track (Game or Solver)
	 * @param TrackID ID of the track
	 * @param InTrackInfo Track info structure used to identify data as what frame we are currently in
	 * @return 
	 */
	float GetFrameTimeForTrack(EChaosVDTrackType TrackType, int32 TrackID, const TSharedRef<const FChaosVDTrackInfo>& InTrackInfo) const;
	float GetFrameTimeForTrack_AssumesLocked(EChaosVDTrackType TrackType, int32 TrackID, const TSharedRef<const FChaosVDTrackInfo>& InTrackInfo) const;

	/**
	 * Changes the visibility of a specific track
	 * @param Type Type of the track (Game or Solver)
	 * @param TrackID ID of the track
	 * @param bNewVisibility New visibility value
	 */
	void UpdateTrackVisibility(EChaosVDTrackType Type, int32 TrackID, bool bNewVisibility);

	/**
	 * Return the current visibility state of the specified track
	 * @param Type Type of the track (Game or Solver)
	 * @param TrackID ID of the track
	 * @return true if the track is visible
	 */
	bool IsTrackVisible(EChaosVDTrackType Type, int32 TrackID);

	/**
	 * Handles an external playback control input (usually from the UI) for a frame
	 * @param ButtonID ID determinig the playback action
	 * @param InTrackInfoRef RefPtr to the track the actions should be applied to
	 * @param Instigator ID of the system that is making the input attempt
	 */
	void HandleFramePlaybackControlInput(EChaosVDPlaybackButtonsID ButtonID, const TSharedRef<const FChaosVDTrackInfo>& InTrackInfoRef, FGuid Instigator);
	
	/**
	 * Handles an external playback control input (usually from the UI) for a solver stage
	 * @param ButtonID ID determinig the playback action
	 * @param InTrackInfoRef RefPtr to the track the actions should be applied to
	 * @param Instigator ID of the system that is making the input attempt
	 */
	void HandleFrameStagePlaybackControlInput(EChaosVDPlaybackButtonsID ButtonID, const TSharedRef<const FChaosVDTrackInfo>& InTrackInfoRef, FGuid Instigator);

	/** Advances the playback
	 * @param DeltaTime Current Delta Time
	 */
	void TickPlayback(float DeltaTime);

	/** Returns the state of the current active track. */
	TSharedRef<FChaosVDTrackInfo> GetActiveTrackInfo() const
	{
		return ActiveTrack;
	}

	/**
	 *  Gathers all track states of the specified type
	 * @param Type Track type (Gamne or Solver)
	 * @param OutTracks Outs a list of matching track states
	 */
	void GetTracksByType(EChaosVDTrackType Type, TArray<TSharedPtr<FChaosVDTrackInfo>>& OutTracks);

	/**
	 * Scrubs all tracks to the correct frame to be in sync with the provided track
	 * @param FromTrack Track that every other track needs to sync to
	 * @param InstigatorID ID of the system that is instigating the playback action
	 * @param TrackSyncMode Sync mode determining how the sync is calculated (Timestamp, Network tick, etc)
	 */
	void SyncTracks(const TSharedRef<const FChaosVDTrackInfo>& FromTrack, FGuid InstigatorID, EChaosVDSyncTimelinesMode TrackSyncMode = EChaosVDSyncTimelinesMode::RecordedTimestamp);
	void SyncTracks_AssumesLocked(const TSharedRef<const FChaosVDTrackInfo>& FromTrack, FGuid InstigatorID, EChaosVDSyncTimelinesMode TrackSyncMode = EChaosVDSyncTimelinesMode::RecordedTimestamp);

	/**
	 * Toggles the sync functionality on the provided track
	 * @param InTrackInfoRef RefPtr to the track we want to change 
	 */
	void ToggleTrackSyncEnabled(const TSharedRef<const FChaosVDTrackInfo>& InTrackInfoRef);

	/** Returns true if we are in continuous playback mode */
	bool IsPlaying() const;

	bool IsScrubbingTimeline() const
	{
		return bIsScrubbingTimeline;
	}

	void SetScrubbingTimeline(bool bNewIsScrubbingTimeline);

	/** Returns the current sync mode for tracks */
	EChaosVDSyncTimelinesMode GetTimelineSyncMode() const
	{
		return CurrentSyncMode;
	}

	/**
	 * Finds and returns a track instance that can be used as sync point for the new timeline mode change
	 * @param SyncMode New timeline Sync Mode
	 * */
	TSharedPtr<const FChaosVDTrackInfo> GetTrackToReSyncFromOnModeChange(EChaosVDSyncTimelinesMode SyncMode);
	TSharedPtr<const FChaosVDTrackInfo> GetTrackToReSyncFromOnModeChange_AssumesLocked(EChaosVDSyncTimelinesMode SyncMode);

	/**
	 * Checks if the provided track is compatible with a specific sync mode
	 * @param InTrackRef Track to check against
	 * @param SyncMode Timeline sync mode to verify if the provided track is compatible with
	 * @return true if the provided track is compatible with the provided sync mode
	 */
	bool IsCompatibleWithSyncMode(const TSharedRef<const FChaosVDTrackInfo>& InTrackRef, EChaosVDSyncTimelinesMode SyncMode);

	/** Sets a new sync mode to be used between tracks */
	CHAOSVD_API void SetTimelineSyncMode(EChaosVDSyncTimelinesMode SyncMode);

	/** Attempts to set the track on the specified slot as active.
	 * An active track is the track to all key shortcuts will be applied
	 * @param SlotIndex Slot of the track we want to make active
	 */
	void TrySetActiveTrack(int32 SlotIndex);

	/** Attempts to set the provided track as active.
	 * An active track is the track to all key shortcuts will be applied
	 * @param NewActiveTrack Slot of the track we want to make active
	 */
	void TrySetActiveTrack(const TSharedRef<const FChaosVDTrackInfo>& NewActiveTrack);

	/**
	 * Creates and populates an object containing information about the state of the playback engine and scene.
	 * This is used by functional tests to compare a known good state agains ones generated in future versions
	 * @return Playback engine state data
	 * @note If you change the data being added here, you need to re-generate the snapshots used by the
	 * scene integrity playback tests in the Simulation Tests Plugin
	 */
	CHAOSVD_API FChaosVDPlaybackEngineSnapshot GeneratePlaybackEngineSnapshot();

protected:
	/**
	 * Broadcast from the game thread data in this controller was updated
	 * @param InControllerWeakPtr A pre created weak ptr instance to this controller
	 */
	void BroadcastGTDataUpdate(const TWeakPtr<FChaosVDPlaybackController>& InControllerWeakPtr);

	/**
	 * Process and broadcast from the game thread track info updates
	 * @param InControllerWeakPtr A pre created weak ptr instance to this controller
	 */
	void ProcessQueuedTrackInfoUpdates(const TWeakPtr<FChaosVDPlaybackController>& InControllerWeakPtr);
	
	/** Updates (or adds) solvers data from the loaded recording to the solver tracks */
	void UpdateSolverTracksData();

	/** Updates the controlled scene with the loaded data at specified game frame */
	void GoToRecordedGameFrame_AssumesLocked(int32 FrameNumber, FGuid InstigatorID);

	/** Updates the controlled scene with the loaded data at specified solver frame and solver Stage */
	void GoToRecordedSolverStage_AssumesLocked(int32 InTrackID, int32 FrameNumber, int32 StageNumber, FGuid InstigatorID);

	/** Handles any data changes on the loaded recording - Usually called during Trace analysis */
	void HandleCurrentRecordingUpdated();

	/** Finds the closest Key frame to the provided frame number, and plays all the following frames until the specified frame number (no inclusive) */
	void PlayFromClosestKeyFrame_AssumesLocked(int32 InTrackID, int32 FrameNumber, FChaosVDScene& InSceneToControl);

	/** Add the provided track info update to the queue. The update will be broadcast in the game thread */
	void EnqueueTrackInfoUpdate(const TSharedRef<const FChaosVDTrackInfo>& InTrackInfo, FGuid InstigatorID);

	void PlaySolverStageData(int32 TrackID, const TSharedRef<FChaosVDScene>& InSceneToControlSharedPtr, const FChaosVDSolverFrameData& InSolverFrameData, int32 StageIndex);

	template <typename TVisitorCallback>
	void VisitAvailableTracks(const TVisitorCallback& VisitorCallback);

	TSharedRef<FChaosVDTrackInfo> CreateTrackInfo(int32 SlotIndex = INDEX_NONE);

	void HandleGeneralSettingsUpdated(UObject* SettingsObject);

	bool bIsScrubbingTimeline = false;

	/** Map containing all track info, by track type*/
	TMap<EChaosVDTrackType, TrackInfoByIDMap> TrackInfoPerType;

	/** Ptr to the loaded recording */
	TSharedPtr<FChaosVDRecording> LoadedRecording;

	/**Ptr to the current Chaos VD Scene this controller controls*/
	TWeakPtr<FChaosVDScene> SceneToControl;

	/** Delegate called when the data on the loaded recording changes */
	FChaosVDPlaybackControllerUpdated ControllerUpdatedDelegate;

	/** Delegate called when the data in a track changes */
	FChaosVDPlaybackControllerFrameUpdated ControllerFrameUpdatedDelegate;

	/** Delegate called when the first frame is played on recording load */
	FChaosVDInitialFirstFramePlayed ControllerFirstFramePlayedDelegate;

	/** Set to true when the recording data controlled by this Playback Controller is updated, the update delegate will be called on the GT */
	std::atomic<bool> bHasPendingGTUpdateBroadcast;

	/** Last seen Platform Cycle on which the loaded recording was updated */
	uint64 RecordingLastSeenTimeUpdatedAsCycle = 0;

	/** Queue with a copy of all Track Info Updates that needs to be done in the Game thread */
	TQueue<FChaosVDQueuedTrackInfoUpdate, EQueueMode::Mpsc> TrackInfoUpdateGTQueue;

	bool bPlayedFirstFrame = false;

	int32 MaxFramesLaggingBehindDuringLiveSession = 50;
	int32 MinFramesLaggingBehindDuringLiveSession = 5;

	int32 CurrentFrameRateOverride = 60;

	bool bUseFrameRateOverride = false;

	bool bPauseRequested = false;

	FDelegateHandle RecordingStoppedHandle;

	TSharedRef<FChaosVDTrackInfo> ActiveTrack;

	float CurrentPlaybackTime = 0.0f;

	/** Counter used to create Track Slot IDs. Game track is always Slot 0 */
	int32 LastAssignedTrackSlot = 0;

	EChaosVDSyncTimelinesMode CurrentSyncMode = EChaosVDSyncTimelinesMode::RecordedTimestamp;

	bool bStartLiveSessionPlaybackAutomatically = true;

	bool bInitialized = false;
};

template <typename TVisitorCallback>
void FChaosVDPlaybackController::VisitAvailableTracks(const TVisitorCallback& VisitorCallback)
{
	for (const TPair<EChaosVDTrackType, TrackInfoByIDMap>& TracksByType : TrackInfoPerType)
	{
		for (const TPair<int32, TSharedPtr<FChaosVDTrackInfo>>& TracksById : TracksByType.Value )
		{
			if (!VisitorCallback(TracksById.Value))
			{
				return;
			}
		}
	}
}
