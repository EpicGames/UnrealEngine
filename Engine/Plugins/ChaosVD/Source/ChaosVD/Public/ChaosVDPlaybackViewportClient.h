// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "EditorViewportClient.h"

class SChaosVDMainTab;
class FComponentVisualizer;
class UChaosVDCoreSettings;
struct FChaosVDGameFrameData;
class FChaosVDScene;
enum class EChaosVDActorTrackingMode;

/** Client viewport class used for to handle a Chaos Visual Debugger world Interaction/Rendering.
 * It re-routes interaction events to our Chaos VD scene
 */
class FChaosVDPlaybackViewportClient : public FEditorViewportClient
{
public:

	FChaosVDPlaybackViewportClient(const TSharedPtr<FEditorModeTools>& InModeTools, const TSharedPtr<SEditorViewport>& InEditorViewportWidget);
	virtual ~FChaosVDPlaybackViewportClient() override;

	virtual void ProcessClick(FSceneView& View, HHitProxy* HitProxy, FKey Key, EInputEvent Event, uint32 HitX, uint32 HitY) override;

	void SetScene(TWeakPtr<FChaosVDScene> InScene);

	virtual UWorld* GetWorld() const override
	{
		return CVDWorld;
	}

	virtual void Draw(const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	virtual void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas) override;

	CHAOSVD_API void ToggleObjectTrackingIfSelected();

	bool IsAutoTrackingSelectedObject() const
	{
		return bAutoTrackSelectedObject;
	}

	CHAOSVD_API void SetAutoTrackingViewDistance(float NewDistance);

	float GetAutoTrackingViewDistance() const
	{
		return TrackingViewDistance;
	}

	void GoToLocation(const FVector& InLocation);

	void UpdateObjectTracking();

	void FocusOnSelectedObject();

	FBox GetSelectionBounds() const;
	
	virtual void UpdateMouseDelta() override;

	void HandleCVDSceneUpdated();

	bool GetCanSelectTranslucentGeometry() const
	{
		return bAllowTranslucentHitProxies;
	}

	void SetCanSelectTranslucentGeometry(bool bCanSelect);
	void ToggleCanSelectTranslucentGeometry();

protected:

	virtual bool Internal_InputKey(const FInputKeyEventArgs& EventArgs) override;

private:

	void ExecuteVisualizationCallback(const TSharedRef<SChaosVDMainTab>& InMainTabToolkitHostRef,
		const TSharedRef<FChaosVDScene>& SceneRef, const TFunction<void(const UActorComponent*, const TSharedRef<FComponentVisualizer>&)>& VisitorCallback);

	void HandleFocusRequest(FBox BoxToFocusOn);
	void HandleActorMoving(AActor* MovedActor) const;

	FDelegateHandle FocusRequestDelegateHandle;
	UWorld* CVDWorld;
	TWeakPtr<FChaosVDScene> CVDScene;

	bool bAutoTrackSelectedObject = false;
	float TrackingViewDistance = 120.0f;

	bool bAllowTranslucentHitProxies = true;
};
