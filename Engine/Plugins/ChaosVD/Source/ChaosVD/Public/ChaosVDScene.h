// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDRecording.h"
#include "ChaosVDSolverDataSelection.h"
#include "Components/ChaosVDSolverDataComponent.h"
#include "Containers/UnrealString.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "Elements/Framework/TypedElementListFwd.h"
#include "Elements/Framework/TypedElementSelectionSet.h"
#include "Elements/Interfaces/TypedElementAssetDataInterface.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "ChaosVDScene.generated.h"

struct FChaosVDPlaybackEngineSnapshot;
class AChaosVDDataContainerBaseActor;
class AChaosVDGeometryContainer;
class IChaosVDGeometryOwnerInterface;
class FChaosVDSelectionCustomization;
class ITypedElementSelectionInterface;
struct FTypedElementSelectionOptions;
class AChaosVDGameFrameInfoActor;
class AChaosVDSolverInfoActor;
class AChaosVDSceneCollisionContainer;
class UChaosVDCoreSettings;
class FChaosVDGeometryBuilder;
class FReferenceCollector;
class UChaosVDSceneQueryDataComponent;
class UObject;
class UMaterial;
class USelection;
class UTypedElementSelectionSet;
class UWorld;

struct FChaosVDSceneParticle;

struct FTypedElementHandle;

typedef TMap<int32, AChaosVDSolverInfoActor*> FChaosVDSolverInfoByIDMap;

DECLARE_MULTICAST_DELEGATE(FChaosVDSceneUpdatedDelegate)
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDOnObjectSelectedDelegate, UObject*)
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDFocusRequestDelegate, FBox)
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDSolverInfoActorCreatedDelegate, AChaosVDSolverInfoActor*)

DECLARE_MULTICAST_DELEGATE_TwoParams(FChaosVDSolverVisibilityChangedDelegate, int32 SolverID, bool bNewVisibility)

class IDetailLayoutBuilder;

/** Fired by FChaosVDSceneParticleCustomization when building details for a selected particle.
 *  Subscribers (e.g. UChaosVDParticleExtraDataComponent) can append additional categories and
 *  struct entries to the details panel via the provided IDetailLayoutBuilder. */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FChaosVDParticleDetailsExtraDataDelegate, int32 /*SolverID*/, int32 /*ParticleID*/, IDetailLayoutBuilder& /*DetailBuilder*/)

/** Lightweight delegate broadcast on every frame change for an already-selected particle.
 *  Subscribers update their cached struct scope memory in-place; they set bOutNeedsFullRebuild=true
 *  if the data layout has changed and a full IDetailLayoutBuilder refresh is required. */
DECLARE_MULTICAST_DELEGATE_ThreeParams(FChaosVDParticleDetailsExtraDataRefreshDelegate, int32 /*SolverID*/, int32 /*ParticleID*/, bool& /*bOutNeedsFullRebuild*/)

UENUM()
enum class EChaosVDSceneCleanUpOptions
{
	None = 0,
	ReInitializeGeometryBuilder = 1 << 0,
	CollectGarbage = 1 << 1
};

ENUM_CLASS_FLAGS(EChaosVDSceneCleanUpOptions)

/** Recreates a UWorld from a recorded Chaos VD Frame */
class FChaosVDScene : public FGCObject , public TSharedFromThis<FChaosVDScene>, public FTSTickerObjectBase
{
public:

	FChaosVDScene();
	virtual ~FChaosVDScene() override;

	void Initialize();
	void DeInitialize();

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override
	{
		return TEXT("FChaosVDScene");
	}

	/** Called each time this Scene is modified */
	FChaosVDSceneUpdatedDelegate& OnSceneUpdated()
	{
		return SceneUpdatedDelegate;
	}

	/** Fired when the particle details panel is being built for a selected particle.
	 *  Subscribers can append additional categories/structs to the panel via IDetailLayoutBuilder. */
	FChaosVDParticleDetailsExtraDataDelegate& OnParticleDetailsExtraDataRequested()
	{
		return ParticleDetailsExtraDataDelegate;
	}

	/** Fired on every frame change while a particle remains selected.
	 *  Subscribers should update their cached struct scope memory in-place (no layout rebuild). */
	FChaosVDParticleDetailsExtraDataRefreshDelegate& OnParticleDetailsExtraDataRefreshRequested()
	{
		return ParticleDetailsExtraDataRefreshDelegate;
	}

	/** Updates, Adds and Remove actors to match the provided Solver Stage Data */
	void UpdateFromRecordedSolverStageData(const int32 SolverID, const FChaosVDFrameStageData& InRecordedStepData, const FChaosVDSolverFrameData& InFrameData);

	/**
	 * Handles the Playback switching to a new Game Thread Frame
	 * @param FrameNumber Number of the GT Frame
	 * @param AvailableSolversIds List of Solver IDs available for this GT Frame data
	 * @param InNewGameFrameData New Frame Data
	 * @param OutRemovedSolversIds Out List of Solver IDs that are no longer available in this GT frame
	 */
	void HandleEnterNewGameFrame(int32 FrameNumber, const TArray<int32, TInlineAllocator<16>>& AvailableSolversIds, const FChaosVDGameFrameData& InNewGameFrameData, TArray<int32, TInlineAllocator<16>>& OutRemovedSolversIds);

	/**
	 * Handles the playback switching to a new solver frame
	 * @param FrameNumber Number of the solver frame
	 * @param InFrameData New Solver frame data
	 */
	void HandleEnterNewSolverFrame(int32 FrameNumber, const FChaosVDSolverFrameData& InFrameData);


	/**
	 * Deletes all actors of the Scene and underlying UWorld 
	 * @param Options Flags controlling how the cleanup is performed
	 */
	void CleanUpScene(EChaosVDSceneCleanUpOptions Options = EChaosVDSceneCleanUpOptions::None);

	/** Returns a ptr to the UWorld used to represent the current recorded frame data */
	UWorld* GetUnderlyingWorld() const
	{
		return PhysicsVDWorld;
	}

	/** Returns true if the scene is initialized and ready to use */
	bool IsInitialized() const
	{
		return bIsInitialized;
	}

	/** Returns a weak ptr to the geometry builder object handling geometry generation and caching for this scene */
	TWeakPtr<FChaosVDGeometryBuilder> GetGeometryGenerator()
	{
		return GeometryGenerator;
	}

	/** Returns an instance to the loaded implicit object for the provided id
	 * @param GeometryID ID of the geometry
	 */
	CHAOSVD_API Chaos::FConstImplicitObjectPtr GetUpdatedGeometry(int32 GeometryID) const;

	CHAOSVD_API const Chaos::FImplicitObject* UnpackGeometry(const Chaos::FImplicitObject* InImplicitObject, Chaos::FRigidTransform3& InOutTransform) const;
	CHAOSVD_API bool DoesGeometryNeedUnpacking(const Chaos::FImplicitObject* InImplicitObject) const;
	
	/** Adds an object to the selection set if it was not selected already, making it selected in practice */
	void SetSelectedObject(UObject* SelectedObject);
	CHAOSVD_API void SetSelected(const FTypedElementHandle& InElementHandle);

	/** Evaluates an object and returns true if it is selected */
	bool IsObjectSelected(const UObject* Object);

	/** Evaluates an object and returns true if it is selected */
	bool IsSelected(const FTypedElementHandle& InElementHandle) const;

	/** Returns a ptr to the current selection set object */
	UTypedElementSelectionSet* GetElementSelectionSet() const
	{
		return SelectionSet;
	}
	
	USelection* GetActorSelectionObject() const
	{
		return ActorSelection;
	}

	USelection* GetComponentsSelectionObject() const
	{
		return ComponentSelection;
	}

	USelection* GetObjectsSelectionObject() const
	{
		return ObjectSelection;
	}
	
	/** Event triggered when an object is focused in the scene (double-click in the scene outliner)*/
	FChaosVDFocusRequestDelegate& OnFocusRequest()
	{
		return FocusRequestDelegate;
	}

	/** Returns a ptr to the particle actor representing the provided Particle ID
	 * @param SolverID ID of the solver owning the Particle
	 * @param ParticleID ID of the particle
	 */
	CHAOSVD_API TSharedPtr<FChaosVDSceneParticle> GetParticleInstance(int32 SolverID, int32 ParticleID);

	/** Returns a const reference for all Solver Data info actor currently available */
	const FChaosVDSolverInfoByIDMap& GetSolverInfoActorsMap()
	{
		return SolverDataContainerBySolverID;
	}

	/** Returns a ptr of a Solver info actor instance for the provided solver ID, if exists
	 * @param SolverID ID of the solver
	 */
	CHAOSVD_API AChaosVDSolverInfoActor* GetSolverInfoActor(int32 SolverID);

	/** Event called when a solver info actor is created */
	FChaosVDSolverInfoActorCreatedDelegate& OnSolverInfoActorCreated()
	{
		return SolverInfoActorCreatedDelegate;
	}

	/** Event called when a solver visibility has changed */
	FChaosVDSolverVisibilityChangedDelegate& OnSolverVisibilityUpdated()
	{
		return SolverVisibilityChangedDelegate;
	}

	/** Updates the render state of the hit proxies of an array of actors. This used to update the selection outline state */
	void UpdateSelectionProxiesForActors(TArrayView<AActor*> SelectedActors);

	/** Returns the generic selection data manager object for this scene */
	TWeakPtr<FChaosVDSolverDataSelection> GetSolverDataSelectionObject()
	{
		return SolverDataSelectionObject ? SolverDataSelectionObject : nullptr;
	}

	/** Returns an array view with all available Data info actors */
	TConstArrayView<TObjectPtr<AChaosVDDataContainerBaseActor>> GetDataContainerActorsView() const
	{
		return AvailableDataContainerActors;
	}

	TSharedPtr<FChaosVDRecording> GetLoadedRecording() const
	{
		return LoadedRecording;
	}

	// Returns the currently selected elements in the scene, which can then be further inspected with GetStructDataFromTypedElementHandle
	CHAOSVD_API TArray<FTypedElementHandle> GetSelectedElementHandles();

	FName GetTEDSSelectionSetName();

	void ClearSelectionAndNotify();

	void RequestUpdate();

	CHAOSVD_API virtual bool Tick(float DeltaTime) override;

	void UpdateWorldStreamingLocation(const FVector& InLocation);

	const FVector& GetWorldStreamingLocation() const
	{
		return WorldStreamingLocation;
	}

	/**
	 * Appends Scene Data relevant for functional tests
	 * @param OutStateTestData Structure instance where to add the data
	 * @note If you change the data being added here, you need to re-generate the snapshots used by the
	 * scene integrity playback tests in the Simulation Tests Plugin
	 */
	void AppendSceneCompositionTestData(FChaosVDPlaybackEngineSnapshot& OutStateTestData);

private:

	FVector WorldStreamingLocation = FVector::ZeroVector; 

	void AddFromCVDWorldTagToActor(AActor* Actor);

	FName TEDSSelectionSetName;
	
	void SetLoadedRecording(const TSharedPtr<FChaosVDRecording>& NewRecordingInstance);
	
	TSharedPtr<FChaosVDRecording> LoadedRecording;
	
	AActor* GetMeshComponentsContainerActor() const;

	AActor* GetSkySphereActor() const
	{
		return SkySphere;
	}

	void PerformGarbageCollection();

	void CreateBaseLights(UWorld* TargetWorld);

	void CreatePostProcessingVolumes(UWorld* TargetWorld);

	/** Creates an actor that will contain all solver data for the provided Solver ID*/
	AChaosVDSolverInfoActor* GetOrCreateSolverInfoActor(int32 SolverID);

	/** Creates an actor that will contain all non-solver data for recorded from any thread*/
	AChaosVDGameFrameInfoActor* GetOrCreateGameFrameInfoActor();

	AActor* CreateMeshComponentsContainer(UWorld* TargetWorld);

	/** Creates the instance of the World which will be used the recorded data*/
	UWorld* CreatePhysicsVDWorld();

	/** Map of SolverID-ChaosVDSolverInfo Actor. Used to keep track of active solvers representations and be able to modify them as needed*/
	FChaosVDSolverInfoByIDMap SolverDataContainerBySolverID;

	/** Returns the correct TypedElementHandle based on an object type so it can be used with the selection set object */
	FTypedElementHandle GetSelectionHandleForObject(const UObject* Object) const;

	void HandleDeSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);
	void HandleSelectElement(const TTypedElement<ITypedElementSelectionInterface>& InElementSelectionHandle, FTypedElementListRef InSelectionSet, const FTypedElementSelectionOptions& InSelectionOptions);

	void InitializeSelectionSets();
	void DeInitializeSelectionSets();

	void HandleActorDestroyed(AActor* ActorDestroyed);

	/** UWorld instance used to represent the recorded debug data */
	TObjectPtr<UWorld> PhysicsVDWorld = nullptr;

	FChaosVDSceneUpdatedDelegate SceneUpdatedDelegate;
	FChaosVDParticleDetailsExtraDataDelegate ParticleDetailsExtraDataDelegate;
	FChaosVDParticleDetailsExtraDataRefreshDelegate ParticleDetailsExtraDataRefreshDelegate;

	TSharedPtr<FChaosVDGeometryBuilder> GeometryGenerator;

	FChaosVDGeometryDataLoaded NewGeometryAvailableDelegate;

	FChaosVDFocusRequestDelegate FocusRequestDelegate;

	/** Selection set object holding the current selection state */
	TObjectPtr<UTypedElementSelectionSet> SelectionSet;

	TObjectPtr<USelection> ActorSelection = nullptr;
	TObjectPtr<USelection> ComponentSelection = nullptr;
	TObjectPtr<USelection> ObjectSelection = nullptr;

	/** Array of actors with hit proxies that need to be updated */
	TArray<AActor*> PendingActorsToUpdateSelectionProxy;

	/** Scene Streamable manager that we'll use to async load any assets we depend on */
	TSharedPtr<struct FStreamableManager> StreamableManager;

	mutable AActor* SkySphere = nullptr;

	AChaosVDGeometryContainer* MeshComponentContainerActor = nullptr;

	AChaosVDGameFrameInfoActor* GameFrameDataInfoActor = nullptr;

	bool bIsInitialized = false;

	FDelegateHandle ActorDestroyedHandle;

	FChaosVDSolverInfoActorCreatedDelegate SolverInfoActorCreatedDelegate;

	FChaosVDSolverVisibilityChangedDelegate SolverVisibilityChangedDelegate;
	
	TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelectionObject;

	TArray<TObjectPtr<AChaosVDDataContainerBaseActor>> AvailableDataContainerActors;

	bool bPendingUpdateRequest = false;

	friend class FChaosVDGeometryBuilder;
	friend class FChaosVDSelectionCustomization;
	friend class FChaosVDPlaybackViewportClient;
	friend class FChaosVDPlaybackController;
};

