// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "UObject/ObjectMacros.h"
#include "UObject/NameTypes.h"
#include "ChaosVDPlaybackController.h"

#include "ChaosVDSceneCompositionReport.generated.h"

USTRUCT()
struct FChaosVDFrameIndexTestData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category=SceneComposition)
	int32 FrameNumber = INDEX_NONE;

	UPROPERTY(VisibleAnywhere, Category=SceneComposition)
	int32 StageNumber = INDEX_NONE;
};

/**
 * Structure containing data about how a CVD scene is composed.
 * This is used for functional testing, to verify if a loaded file at a specific frame keeps the expected composition
 */
USTRUCT()
struct FChaosVDSceneCompositionTestData
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category=SceneComposition)
	TMap<FName, int32> ObjectsCountByType;
};

/**
 * Structure containing data about the playback engine state at the time when this struct is created.
 * This is used for functional testing, to verify if a loaded file at a specific frame keeps the expected composition
 *
 * @note If you change the composition of this structure or of the structures used by it, you need to re-generate the snapshots used by the
 * scene integrity playback tests in the Simulation Tests Plugin
 */
USTRUCT()
struct FChaosVDPlaybackEngineSnapshot
{
	GENERATED_BODY()

	UPROPERTY(VisibleAnywhere, Category=SceneComposition)
	TArray<FName> InstalledExtensionsNames;

	UPROPERTY(VisibleAnywhere, Category=SceneComposition)
	TMap<int32, FChaosVDFrameIndexTestData> FrameIndexDataByTrackID;

	UPROPERTY(VisibleAnywhere, Category=SceneComposition)
	EChaosVDSyncTimelinesMode TimelineSyncMode = EChaosVDSyncTimelinesMode::RecordedTimestamp;

	UPROPERTY(VisibleAnywhere, Category=SceneComposition)
	FChaosVDSceneCompositionTestData SceneComposition;
};
