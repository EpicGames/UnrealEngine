// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDBaseSceneObject.h"
#include "Chaos/Core.h"
#include "ChaosVDCharacterGroundConstraintDataProviderInterface.h"
#include "ChaosVDGeometryDataComponent.h"
#include "ChaosVDSceneParticleFlags.h"
#include "DataWrappers/ChaosVDParticleDataWrapper.h"
#include "DataWrappers/ChaosVDCollisionDataWrappers.h"

#include "ChaosVDSceneParticle.generated.h"

namespace UE::Editor::DataStorage
{
	class ICompatibilityProvider;
	class IEditorDataStorageProvider;
} // UE::Editor::DataStorage

UENUM()
enum class EChaosVDParticleVisibilityUpdateFlags
{
	None = 0,
	DirtyScene = 1 << 0,
};
ENUM_CLASS_FLAGS(EChaosVDParticleVisibilityUpdateFlags)

USTRUCT()
struct FChaosVDSceneParticle : public FChaosVDBaseSceneObject 
{

GENERATED_BODY()

public:
	virtual ~FChaosVDSceneParticle() override;

	FChaosVDSceneParticle();

	void PreUpdateFromRecordedParticleData(const TSharedPtr<FChaosVDParticleDataWrapper>& InRecordedData, const Chaos::FRigidTransform3& SimulationTransform);
	void ProcessPendingParticleDataUpdates();

	TSharedPtr<const FChaosVDParticleDataWrapper> GetParticleData() const
	{
		return ParticleDataPtr;
	}

	virtual void SetParent(const TSharedPtr<FChaosVDBaseSceneObject>& NewParent) override;

	void UpdateGeometryComponentsVisibility(EChaosVDParticleVisibilityUpdateFlags Flags = EChaosVDParticleVisibilityUpdateFlags::None);

	void UpdateGeometryColors();

	/** Changes the active state of this CVD Particle Actor */
	void SetIsActive(bool bNewActive);

	void AddHiddenFlag(EChaosVDHideParticleFlags Flag);

	void RemoveHiddenFlag(EChaosVDHideParticleFlags Flag);

	/** Performs all the required steps to hide a particle and update the viewport / scene outliner.
	 * @param Flag Context flags about what is hiding this particle
	 * @note Don't use when performing operation on multiple particles or other performance sensitive operations.
	 * See UChaosVDParticleDataComponent for examples of batch visibility update operations
	 */
	void HideImmediate(EChaosVDHideParticleFlags Flag);

	/** Performs all the required steps to show a particle and update the viewport / scene outliner.
	 * @note Don't use when performing operation on multiple particles or other performance sensitive operations.
	 * See UChaosVDParticleDataComponent for examples of batch visibility update operations
	 */
	void ShowImmediate();

	EChaosVDHideParticleFlags GetHideFlags() const
	{
		return HideParticleFlags;
	}

	bool IsVisible() const
	{
		return HideParticleFlags == EChaosVDHideParticleFlags::None;
	}

	/** Returns true if this particle actor is active - Inactive Particle actors are still in the world but with outdated data
	 * and hidden from the viewport and outliner. They represent particles that were destroyed.
	 */
	bool IsActive() const
	{
		return bIsActive;
	}

	CHAOSVD_API FBox GetBoundingBox() const;
	FBox GetInflatedBoundingBox() const;
	Chaos::TAABB<double, 3> GetChaosBoundingBox() const;

	TConstArrayView<TSharedPtr<FChaosVDParticlePairMidPhase>> GetCollisionData();
	bool HasCollisionData();

	// BEGIN IChaosVDCharacterGroundConstraintDataProviderInterface
	void GetCharacterGroundConstraintData(TArray<TSharedPtr<FChaosVDCharacterGroundConstraint>>& OutConstraintsFound);
	bool HasCharacterGroundConstraintData();
	// END IChaosVDCharacterGroundConstraintDataProviderInterface

	void SetIsServerParticle(bool bNewIsServer)
	{
		bIsServer = bNewIsServer;
	}

	bool GetIsServerParticle() const
	{
		return bIsServer;
	}

	void UpdateMeshInstancesSelectionState();

	// BEGIN IChaosVDGeometryOwner Interface
	TConstArrayView<TSharedRef<FChaosVDInstancedMeshData>> GetMeshInstances() const
	{
		return MeshDataHandles;
	}

	virtual void SetSelectedMeshInstance(const TWeakPtr<FChaosVDInstancedMeshData>& GeometryInstanceToSelect);

	virtual TWeakPtr<FChaosVDInstancedMeshData> GetSelectedMeshInstance() const
	{
		return CurrentSelectedGeometryInstance;
	}

	// END IChaosVDGeometryOwner Interface

	void HandleDeSelected();
	void HandleSelected();

	bool IsSelected();

	void SetScene(const TWeakPtr<FChaosVDScene>& NewScene)
	{
		SceneWeakPtr = NewScene;
	}
	
	TWeakPtr<FChaosVDScene> GetScene() const
	{
		return SceneWeakPtr;
	}

	TWeakPtr<FChaosVDSceneParticle> GetParentParticle()
	{
		return ParentParticleInstance;
	}

	TWeakPtr<Chaos::VD::FChaosVDImplicitObjectView> GetRootImplicitView() const
	{
		return RootImplicitView.ToWeakPtr();
	}

	EChaosVDSceneParticleDirtyFlags GetDirtyFlags() const
	{
		return DirtyFlags;
	}

	void RemoveAllGeometry();

	virtual FBox GetStreamingBounds() const override;

	virtual void SyncStreamingState() override;

	virtual int32 GetStreamingID() const override;

protected:
	
	void UpdateGeometry(const Chaos::FConstImplicitObjectPtr& InImplicitObject, EChaosVDActorGeometryUpdateFlags OptionsFlags = EChaosVDActorGeometryUpdateFlags::None);

	void CalculateAndCacheBounds() const;

	void UpdateParent(const TSharedPtr<FChaosVDParticleDataWrapper>& InRecordedData);

	void ProcessUpdatedAndRemovedHandles(TArray<TPair<TSharedPtr<FChaosVDExtractedGeometryDataHandle>, TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView>>>& OutExtractedGeometryDataHandles);

	const TArray<TSharedPtr<FChaosVDParticlePairMidPhase>>* GetCollisionMidPhasesArray() const;

	const TArray<TSharedPtr<FChaosVDConstraintDataWrapperBase>>* GetCharacterGroundConstraintArray() const;

	void UpdateShapeDataComponents();

	void ApplyPendingTransformData();

	template<typename TTaskCallback>
	void VisitGeometryInstances(const TTaskCallback& VisitorCallback);

	uint8 bIsGeometryDataGenerationStarted : 1 = false;
	uint8 bIsActive: 1 = false;
	uint8 bIsServer: 1 = false;
	EChaosVDHideParticleFlags HideParticleFlags = EChaosVDHideParticleFlags::None;
	EChaosVDSceneParticleDirtyFlags DirtyFlags = EChaosVDSceneParticleDirtyFlags::None;

	Chaos::FConstImplicitObjectPtr CurrentRootGeometry = nullptr;

	TWeakPtr<FChaosVDSceneParticle> ParentParticleInstance;

	TSharedPtr<FChaosVDParticleDataWrapper> ParticleDataPtr;

	FDelegateHandle GeometryUpdatedDelegate;

	TArray<TSharedRef<FChaosVDInstancedMeshData>> MeshDataHandles;

	TWeakPtr<FChaosVDInstancedMeshData> CurrentSelectedGeometryInstance;

	TSharedPtr<Chaos::VD::FChaosVDImplicitObjectView> RootImplicitView;


	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	FTransform PendingParticleTransform;
	FTransform CachedSimulationTransform;

	mutable FBox CachedBounds = FBox(ForceInitToZero);

	/** Called when this particle is destroyed. This is intentionally private and only intended to be used by
	 * the customization class for this object
	 */
	FSimpleDelegate ParticleDestroyedDelegate;

	friend class FChaosVDSceneParticleCustomization;
	friend class UChaosVDParticleDataComponent;
};

template <typename TVisitorCallback>
void FChaosVDSceneParticle::VisitGeometryInstances(const TVisitorCallback& VisitorCallback)
{
	for (TSharedRef<FChaosVDInstancedMeshData>& MeshDataHandle : MeshDataHandles)
	{
		VisitorCallback(MeshDataHandle);
	}
}
