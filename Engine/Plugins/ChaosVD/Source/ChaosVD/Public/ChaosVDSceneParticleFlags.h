// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Misc/EnumClassFlags.h"

#include "ChaosVDSceneParticleFlags.generated.h"

/** Options flags to control how geometry is updated in a ChaosVDActor */
UENUM()
enum class EChaosVDActorGeometryUpdateFlags : uint8
{
	None = 0,
	ForceUpdate = 1 << 0
};
ENUM_CLASS_FLAGS(EChaosVDActorGeometryUpdateFlags)

UENUM()
enum class EChaosVDHideParticleFlags : uint8
{
	None = 0,
	HiddenByVisualizationFlags = 1 << 0,
	HiddenBySceneOutliner = 1 << 1,
	HiddenByActiveState = 1 << 2,
	HiddenBySolverVisibility = 1 << 3,
};
ENUM_CLASS_FLAGS(EChaosVDHideParticleFlags)

/** Set of flags used to indicate which data was modified and needs to be evaluated again by other systems */
UENUM()
enum class EChaosVDSceneParticleDirtyFlags : uint16
{
	None =			0,
	Visibility =	1 << 0,
	Coloring =		1 << 1,
	Active =		1 << 2,
	Transform =		1 << 3,
	Parent =		1 << 4,
	Geometry =		1 << 5,
	CollisionData =	1 << 6,
	PreUpdatePass =	1 << 7,
	TEDS =	1 << 8,
	StreamingBounds = 1 << 9,
};
ENUM_CLASS_FLAGS(EChaosVDSceneParticleDirtyFlags)
