// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/WeakObjectPtr.h"

class FChaosVDScene;
class UTypedElementSelectionSet;

/** Base class that provided access to selection events from the Chaos visual debugger
 * local selection system.
 */
class FChaosVDSceneSelectionObserver
{
public:

	CHAOSVD_API FChaosVDSceneSelectionObserver();
	CHAOSVD_API virtual ~FChaosVDSceneSelectionObserver();

protected:

	/** Stores a weak ptr to the selection set object containing the current selection state */
	CHAOSVD_API void RegisterSelectionSetObject(UTypedElementSelectionSet* SelectionSetObject);
	
	const UTypedElementSelectionSet* GetSelectionSetObject() const { return ObservedSelection.IsValid() ? ObservedSelection.Get() : nullptr; };

	/** Called when the current selection might change
	 * @param SelectionSetPreChange the pointer to the selection set object that might be about to change
	 */
	virtual void HandlePreSelectionChange(const UTypedElementSelectionSet* SelectionSetPreChange){}
	
	/** Called when the current selection has changed
	 * @param ChangesSelectionSet the pointer to the selection set object that has changed
	 */
	virtual void HandlePostSelectionChange(const UTypedElementSelectionSet* ChangesSelectionSet){}

	/** Weak Ptr to the selection set object we are observing */
	TWeakObjectPtr<UTypedElementSelectionSet> ObservedSelection;
};
