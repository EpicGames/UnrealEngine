// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Chaos/AABBTree.h"
#include "Chaos/ISpatialAcceleration.h"
#include "Containers/ContainersFwd.h"
#include "Containers/Queue.h"
#include "Containers/Ticker.h"
#include "Containers/Set.h"
#include "HAL/CriticalSection.h"
#include "Math/Box.h"
#include "Misc/Optional.h"
#include "Templates/SharedPointer.h"
#include "Tasks/Task.h"

#include "ChaosVDSceneStreaming.generated.h"

class FChaosVDScene;
struct FChaosVDBaseSceneObject;

UENUM()
enum class EChaosVDStreamingDirtyFlags : uint32
{
	None = 0,
	StreamingExtents = 1 << 0,
	StreamingEnabled = 1 << 1,
	StreamingSourceLocation = 1 << 2,
	AccelerationStructure = 1 << 3
};
ENUM_CLASS_FLAGS(EChaosVDStreamingDirtyFlags)

class IChaosVDStreamingDataSource
{
public:
	virtual ~IChaosVDStreamingDataSource() = default;

	virtual TConstArrayView<TSharedRef<FChaosVDBaseSceneObject>> GetStreamableSceneObjects() const = 0;
	virtual FRWLock& GetObjectsLock() const = 0;
};

/** Simple pseudo level streaming system that works with a collection of ChaosVDSceneObjects -
 * This system only updates a desired streaming state (from multiple threads) and issues a sync request in the GT.
 * How these actions are executed depends on the implementations of each ChaosVDSceneObjects derived object
 */
class FChaosVDSceneStreaming : public FTSTickerObjectBase
{
public:

	template<uint32 Alignment = DEFAULT_ALIGNMENT> class TMemStackSetAllocator : public TSetAllocator<TSparseArrayAllocator<TMemStackAllocator<Alignment>, TMemStackAllocator<Alignment>>, TMemStackAllocator<Alignment>> {};

	FChaosVDSceneStreaming();
	virtual ~FChaosVDSceneStreaming() override;

	/** Relevant streaming system data that needs processing */
	struct FPendingTrackingOperation
	{
		enum class EType
		{
			None,
			AddOrUpdate,
			Remove
		};

		FBox Bounds = FBox(EForceInit::ForceInitToZero);
		int32 ObjectID = INDEX_NONE;
		EType OperationType = EType::None;
	};

	void Initialize();
	void DeInitialize();

	virtual bool Tick(float DeltaTime) override;

	/** Adds an object tracking operation to the queue */
	void EnqueuePendingTrackingOperation(const FPendingTrackingOperation&& Operation);
	
	/**
	 * Creates the request object tracking operation for the provided object and adds it to the queue
	 * @param InSceneObject Object from which create the operation from
	 * @param Type Type of operation to perform (AddOrUpdate, Remove)
	 */
	void EnqueuePendingTrackingOperation(const TSharedRef<FChaosVDBaseSceneObject>& InSceneObject, FPendingTrackingOperation::EType Type);

	/**
	 * Updates the current location used to calculate what objects should be fully loaded or not
	 * @param NewLocation New streaming source location
	 */
	void UpdateStreamingSourceLocation(const FVector& NewLocation);

	/**
	 * Returns true if the provided bounds are within the current calculated streaming volume
	 * @param Bounds Bounds to evaluate
	 */
	bool IsInStreamingRange(const FBox& Bounds) const;

	/**
	 * Sets the correct streaming state for the provided object based on its bounds
	 * @param InSceneObject Object to evaluate
	 */
	void UpdateStreamingStateForObject(const TSharedRef<FChaosVDBaseSceneObject>& InSceneObject) const;

	/**
	 * Creates an object tracking operation structure with the necessary data to update the streaming system state for this object
	 * @param InSceneObject Object from which get the necessary data
	 * @param Type Operation type (AddOrUpdate, Remove)
	 */
	FPendingTrackingOperation CreateStreamingTrackingOperation(const TSharedRef<FChaosVDBaseSceneObject>& InSceneObject, FPendingTrackingOperation::EType Type);

	/**
	 * Resets the streaming system state
	 */
	void Reset();

	/**
	 * Sets a ptr to an array with all the objects managed by this streaming system
	 * @param InStreamingDataSource Object that provides access to the data this system will manage
	 */
	void SetStreamingDataSource(IChaosVDStreamingDataSource* InStreamingDataSource);

	/** Changes the enabled state of this streaming system, and updates the world acordingly
	 * @param bNewEnabled New enabled state
	 */
	void SetStreamingEnabled(bool bNewEnabled);

	/**
	 * Sets a weak ptr to the CVD scene where all the objects managed by this system live
	 * @param InSceneWeakPtr 
	 */
	void SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr)
	{
		SceneWeakPtr = InSceneWeakPtr;
	}

	/**
	 * Handles any streaming settings changes done by the user and updates the world accordingly
	 * @param SettingsObject Object containing the new settings data
	 */
	void HandleSettingsChanged(UObject* SettingsObject);


	/** Returns true if this system is enabled */
	bool IsEnabled() const
	{
		return bIsStreamingSystemEnabled;
	}

protected:

	void SetDirtyFlags(EChaosVDStreamingDirtyFlags Flag);
	void RemoveDirtyFlag(EChaosVDStreamingDirtyFlags Flag);

	void ProcessPendingOperations();

	void UpdateStreamingState();

	void SetStreamingExtent(float NewExtent);

	void UpdateStreamingQueryShape();

	void ReBuildAccelStructureFromSourceDataArray();

	void MakeEverythingVisible();

	float StreamingExtent = 5000.0f;

	float MovementThreshold = 10.0f;

	FVector CurrentStreamingSourceLocation = FVector::ZeroVector;

	TOptional<FVector> LastStreamingLocationUpdate;

	FBox SteamingViewBox;

	Chaos::TAABBTree<int32, Chaos::TAABBTreeLeafArray<int32>> StreamingAccelerationStructure;

	IChaosVDStreamingDataSource* StreamingDataSource = nullptr;

	TQueue<FPendingTrackingOperation, EQueueMode::Mpsc> PendingOperationsPerParticle;

	EChaosVDStreamingDirtyFlags DirtyFlags;

	bool bIsStreamingSystemEnabled = true;

	bool bProcessPendingOperationsQueueInWorkerThread = true;

	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	UE::Tasks::FTask CurrentProcessingTaskHandle;

	mutable FRWLock StreamingAccelerationOperationsLock;

	mutable FRWLock DirtyFlagsLock;

	struct FQueryVisitor
	{
		FQueryVisitor(TSet<int32>& InResults) : CollectedResults(InResults)
		{
		
		}

		bool VisitOverlap(const Chaos::TSpatialVisitorData<int32>& Instance)
		{
			CollectedResults.Add(Instance.Payload);
			return true;
		}
		bool VisitSweep(const Chaos::TSpatialVisitorData<int32>& Instance, Chaos::FQueryFastData& CurData)
		{
			check(false);
			return true;
		}
		bool VisitRaycast(const Chaos::TSpatialVisitorData<int32>& Instance, Chaos::FQueryFastData& CurData)
		{
			check(false);
			return true;
		}

		UE_DEPRECATED(5.8, "This was unused and is being removed")
		const void* GetQueryData() const
		{
			return nullptr;
		}

		UE_DEPRECATED(5.8, "This was unused and is being removed")
		const void* GetSimData() const
		{
			return nullptr;
		}

		const void* GetQueryPayload() const 
		{ 
			return nullptr; 
		}

		bool HasBlockingHit() const
		{
			return false;
		}

		UE_DEPRECATED(5.8, "This was unused and is being removed")
		bool ShouldIgnore(const Chaos::TSpatialVisitorData<int32>& Instance) const
		{
			return false;
		}
		
		const bool PrePreFilter(int) const
		{
			return false;
		}

		TSet<int32>& CollectedResults;
	};
};
