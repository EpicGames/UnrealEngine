// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"

class UChaosVDSettingsObjectBase;

/** Manager class that handles all available CVD settings objects*/
class FChaosVDSettingsManager : public FGCObject
{
public:
	FChaosVDSettingsManager();

	virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override;

	CHAOSVD_API static FChaosVDSettingsManager& Get();
	/** Returns nullptr if the singleton has not been created yet or has already been torn down. */
	CHAOSVD_API static FChaosVDSettingsManager* TryGet();
	static void TearDown();

	/** Returns the setting object of the specified class - If the object does not exist yet, it will be created during this call */
	template<typename SettingsClass>
	SettingsClass* GetSettingsObject();

	/** Returns the setting object of the specified class - If the object does not exist yet, it will be created during this call */
	CHAOSVD_API UChaosVDSettingsObjectBase* GetSettingsObject(UClass* SettingsClass);

	/** Deletes any saved config for the setting object of the specified class, and restores its values to be the same as the CDO */
	template<typename SettingsClass>
	void ResetSettings();

	/** Deletes any saved config for the setting object of the specified class, and restores its values to be the same as the CDO */
	CHAOSVD_API void ResetSettings(UClass* SettingsClass);

protected:

	void RestoreConfigPropertiesValuesFromCDO(UChaosVDSettingsObjectBase* TargetSettingsObject);
	
	TMap<TObjectPtr<UClass>, TObjectPtr<UChaosVDSettingsObjectBase>> AvailableSettingsObject;
	TObjectPtr<UChaosVDSettingsObjectsOuter> SettingsOuter;
};

template <typename SettingsClass>
SettingsClass* FChaosVDSettingsManager::GetSettingsObject()
{
	return Cast<SettingsClass>(GetSettingsObject(SettingsClass::StaticClass()));
}

template <typename SettingsClass>
void FChaosVDSettingsManager::ResetSettings()
{
	UClass* ObjectClass = SettingsClass::StaticClass();
	ResetSettings(ObjectClass);
}
