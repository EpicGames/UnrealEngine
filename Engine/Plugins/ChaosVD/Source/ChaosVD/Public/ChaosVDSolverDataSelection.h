// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Templates/SharedPointer.h"
#include "UObject/StructOnScope.h"
#include "ChaosVDSolverDataSelection.generated.h"

class FChaosVDSolverDataSelection;

/** Base struct type used for any context data we want to add for a selection handle */
USTRUCT()
struct FChaosVDSelectionContext
{
	GENERATED_BODY()
};

/** Struct used to create a combined view of multiple structs to be used in a vanilla details panel.
 * This struct type has a customization that will show each data entry as an individual property.
 * Entries may optionally carry a group name; entries sharing the same name are displayed under
 * a single collapsible category with that name. Entries with no group go into "Recorded Data".
 */
USTRUCT()
struct FChaosVDSelectionMultipleView
{
	GENERATED_BODY()

	struct FDataEntry
	{
		TSharedPtr<FStructOnScope> StructOnScope;
		FName GroupName = NAME_None;
	};

	template<typename StructType>
	void AddData(StructType* Struct, FName GroupName = NAME_None);

	void AddData(TSharedPtr<FStructOnScope> StructOnScope, FName GroupName = NAME_None)
	{
		DataInstances.Add({ StructOnScope, GroupName });
	}

	void Clear()
	{
		DataInstances.Empty();
	}

protected:
	TArray<FDataEntry> DataInstances;

	friend class FChaosVDSelectionMultipleViewCustomization;
};

template <typename StructType>
void FChaosVDSelectionMultipleView::AddData(StructType* Struct, FName GroupName)
{
	if (!Struct)
	{
		return;
	}

	DataInstances.Add({ MakeShared<FStructOnScope>(StructType::StaticStruct(), reinterpret_cast<uint8*>(Struct)), GroupName });
}

/**
 * Selection handle that holds a reference to the selected solver data
 */
struct FChaosVDSolverDataSelectionHandle : public TSharedFromThis<FChaosVDSolverDataSelectionHandle>
{
	virtual ~FChaosVDSolverDataSelectionHandle() = default;

	/** Set the data this handle points to */
	template<typename DataStructType>
	void SetHandleData(const TSharedPtr<DataStructType>& Data);

	/** Set the data that acts as context for this handle */
	template<typename ContextDataStructType>
	void SetHandleContext(ContextDataStructType&& ContextData);

	/** Sets the selection system instance that owns this handle */
	CHAOSVD_API void SetOwner(const TSharedPtr<FChaosVDSolverDataSelection>& InOwner);

	/** Returns tru if the data from this selection handle is currently selected */
	CHAOSVD_API virtual bool IsSelected();

	/** Return true if this handle is valid */
	CHAOSVD_API bool IsValid() const;

	/** Returns true if the data referenced by this handle is from the specified type*/
	template<typename DataStructType>
	bool IsA() const;

	/** Returns a raw ptr to the data this handled references */
	template<typename DataStructType>
	DataStructType* GetData() const;

	/** Returns a shared ptr to the data this handled references */
	template<typename DataStructType>
	TSharedPtr<DataStructType> GetDataAsShared() const;

	/** Returns a raw ptr to the context data this handled references */
	template<typename ContextDataStructType>
	ContextDataStructType* GetContextData() const;

	/** Returns the referenced data as FStructOnScope, so it can be feeded directly to a struct details panel */
	TSharedPtr<FStructOnScope> GetDataAsStructScope()
	{
		return SelectedDataStruct;
	}

	/** Returns a struct on Scope view that can be fed into a CVD details panel -
	 * Usually used to combine data and context into a single read only struct that can be inspected
	 */
	virtual TSharedPtr<FStructOnScope> GetCustomDataReadOnlyStructViewForDetails()
	{
		return SelectedDataStruct;
	}

	bool operator==(const FChaosVDSolverDataSelectionHandle& Other) const
	{
		return DataSharedPtr.Get() == Other.DataSharedPtr.Get();
	}

	bool operator!=(const FChaosVDSolverDataSelectionHandle& Other) const
	{
		return DataSharedPtr.Get() != Other.DataSharedPtr.Get();
	}

private:

	template<typename DataStructType>
	bool IsA_Internal(const TSharedPtr<FStructOnScope>& InStructOnScope) const;

	TSharedPtr<FStructOnScope> SelectedDataStruct;
	TSharedPtr<FStructOnScope> SelectedDataContext;

	TSharedPtr<void> DataSharedPtr;
	TSharedPtr<void> SelectedDataContextSharedPtr;

protected:
	TWeakPtr<FChaosVDSolverDataSelection> Owner;
};

template <typename DataStructType>
void FChaosVDSolverDataSelectionHandle::SetHandleData(const TSharedPtr<DataStructType>& Data)
{
	if (Data)
	{
		SelectedDataStruct = MakeShared<FStructOnScope>(DataStructType::StaticStruct(), reinterpret_cast<uint8*>(Data.Get()));
		DataSharedPtr = Data;
	}
	else
	{
		SelectedDataStruct->Reset();
		DataSharedPtr = nullptr;
	}
}

template <typename ContextDataStructType>
void FChaosVDSolverDataSelectionHandle::SetHandleContext(ContextDataStructType&& ContextData)
{
	SelectedDataContextSharedPtr = MakeShared<ContextDataStructType>();
	ContextDataStructType* ContextDataRawPtr = StaticCastSharedPtr<ContextDataStructType>(SelectedDataContextSharedPtr).Get();
	*ContextDataRawPtr = MoveTemp(ContextData);

	SelectedDataContext = MakeShared<FStructOnScope>(ContextDataStructType::StaticStruct(), reinterpret_cast<uint8*>(ContextDataRawPtr));		
}

template <typename DataStructType>
bool FChaosVDSolverDataSelectionHandle::IsA() const
{
	return IsA_Internal<DataStructType>(SelectedDataStruct);
}

template <typename DataStructType>
DataStructType* FChaosVDSolverDataSelectionHandle::GetData() const
{
	if (IsA_Internal<DataStructType>(SelectedDataStruct))
	{
		return reinterpret_cast<DataStructType*>(SelectedDataStruct->GetStructMemory());
	}
	
	return nullptr;
}

template <typename DataStructType>
TSharedPtr<DataStructType> FChaosVDSolverDataSelectionHandle::GetDataAsShared() const
{
	if (IsA_Internal<DataStructType>(SelectedDataStruct))
	{
		return StaticCastSharedPtr<DataStructType>(DataSharedPtr);
	}
	return nullptr;
}

template <typename ContextDataStructType>
ContextDataStructType* FChaosVDSolverDataSelectionHandle::GetContextData() const
{
	if (IsA_Internal<ContextDataStructType>(SelectedDataContext))
	{
		return reinterpret_cast<ContextDataStructType*>(SelectedDataContext->GetStructMemory());
	}

	return nullptr;
}

template <typename DataStructType>
bool FChaosVDSolverDataSelectionHandle::IsA_Internal(const TSharedPtr<FStructOnScope>& InStructOnScope) const
{
	if (InStructOnScope)
	{
		const UStruct* HandleStruct = InStructOnScope->GetStruct();
		return HandleStruct && (DataStructType::StaticStruct() == HandleStruct || HandleStruct->IsChildOf(DataStructType::StaticStruct()));
	}

	return false;
}


DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDSolverDataSelectionChangedDelegate, const TSharedPtr<FChaosVDSolverDataSelectionHandle>& SelectionHandle)

/** Generic Solver Data selection system. The data this selection system can use must be UStructs */
class FChaosVDSolverDataSelection : public TSharedFromThis<FChaosVDSolverDataSelection>
{
public:

	/** Selects the data in provided selection handle
	 * @param InSelectionHandle Handle that references the data we want to select
	 */
	CHAOSVD_API void SelectData(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle);

	/** Creates a selection handle for the provided data instance. The handle will be owned by this selection systems */
	template<typename SolverDataType, typename HandleType = FChaosVDSolverDataSelectionHandle>
	TSharedPtr<FChaosVDSolverDataSelectionHandle> MakeSelectionHandle(const TSharedPtr<SolverDataType>& InSolverData);

	/** Event that is called when the selection in this system changes */
	FChaosVDSolverDataSelectionChangedDelegate& GetDataSelectionChangedDelegate()
	{
		return SolverDataSelectionChangeDelegate;
	}

	/** Returns the selection handle for the currently selected data */
	TSharedPtr<FChaosVDSolverDataSelectionHandle> GetCurrentSelectionHandle()
	{
		return CurrentSelectedSolverDataHandle;
	}

	/** Returns true if the provided solver data instance is selected
	 * @param InSolverData Solver data instance to evaluate
	 */
	template <typename SolverDataType>
	bool IsDataSelected(const TSharedPtr<SolverDataType>& InSolverData);
	
	CHAOSVD_API bool IsSelectionHandleSelected(const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InSelectionHandle) const;

private:
	FChaosVDSolverDataSelectionChangedDelegate SolverDataSelectionChangeDelegate;
	TSharedPtr<FChaosVDSolverDataSelectionHandle> CurrentSelectedSolverDataHandle;
};

template <typename SolverDataType, typename HandleType>
TSharedPtr<FChaosVDSolverDataSelectionHandle> FChaosVDSolverDataSelection::MakeSelectionHandle(const TSharedPtr<SolverDataType>& InSolverData)
{
	static_assert(std::is_base_of_v<FChaosVDSolverDataSelectionHandle, HandleType>, "MakeSelectionHandle only supports handles derived from FChaosVDSolverDataSelectionHandle");

	TSharedPtr<HandleType> NewSelectionHandle = MakeShared<HandleType>();
	NewSelectionHandle->SetHandleData(StaticCastSharedPtr<SolverDataType>(InSolverData));
	NewSelectionHandle->SetOwner(AsShared());

	return StaticCastSharedPtr<HandleType>(NewSelectionHandle);
}

template <typename SolverDataType>
bool FChaosVDSolverDataSelection::IsDataSelected(const TSharedPtr<SolverDataType>& InSolverData)
{
	TSharedPtr<FChaosVDSolverDataSelectionHandle> SelectionHandle = MakeSelectionHandle(InSolverData);
	return SelectionHandle->IsSelected();
}