// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Delegates/DelegateCombinations.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"
#include "Widgets/Docking/SDockTab.h"

class FChaosVDScene;
class SChaosVDMainTab;
class FName;
class FTabManager;
class SDockTab;
class FSpawnTabArgs;

DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDTabSpawned, TSharedRef<SDockTab>)
DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDTabDestroyed, TSharedRef<SDockTab>)

/** Base class for any tab of the Chaos Visual Debugger tool*/
class FChaosVDTabSpawnerBase
{
public:
	virtual ~FChaosVDTabSpawnerBase() = default;

	CHAOSVD_API FChaosVDTabSpawnerBase(const FName& InTabID, TSharedPtr<FTabManager> InTabManager, TWeakPtr<SChaosVDMainTab> InOwningTabWidget);

	/**
	 * Handles a spawn request for this tab. It controls how and what contents this tab will have
	 * @param Args Arguments about the spawn request
	 */
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) = 0;

	/**
	 * Handles a tab being spawned by this spawner being closed
	 * @param InTabClosed RefPtr to the tab that was closed
	 */
	CHAOSVD_API virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed);
	
	/**
	 * Handles a tab being spawned by this spawner being closed
	 * @param InTabClosed RefPtr to the tab that was closed
	 */
	CHAOSVD_API virtual void HandleTabSpawned(TSharedRef<SDockTab> InTabSpawned);

	/** Event called when the tab is spawned */
	FChaosVDTabSpawned& OnTabSpawned()
	{
		return TabSpawnedDelegate;
	}
	
	/** Event called when the tab this spawner created is destroyed */
	FChaosVDTabDestroyed& OnTabDestroyed()
	{
		return TabDestroyedDelegate;
	}

	/**
	 * Returns the name uses as ID for this tab
	 */
	FName GetTabID() const
	{
		return TabID;
	}

protected:

	/**
	 * Generates a generic error widget to indicate that this tab was created, but something went wrong
	 */
	CHAOSVD_API TSharedRef<SWidget> GenerateErrorWidget();

	/** Ptr to the main tab of the owning visual debugger tool instance */
	TWeakPtr<SChaosVDMainTab> OwningTabWidget;

	FChaosVDTabSpawned TabSpawnedDelegate;
	FChaosVDTabDestroyed TabDestroyedDelegate;

	UWorld* GetChaosVDWorld() const;

	TWeakPtr<FChaosVDScene> GetChaosVDScene() const;

	FName TabID;
};
