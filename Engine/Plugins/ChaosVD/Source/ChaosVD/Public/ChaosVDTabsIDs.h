// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/NameTypes.h"

#define UE_API CHAOSVD_API

/** Built-in CVD tab IDs */
class FChaosVDTabID
{
public:

	static UE_API const FName ChaosVisualDebuggerTab;
	static UE_API const FName PlaybackViewport;
	static UE_API const FName WorldOutliner;
	static UE_API const FName DetailsPanel;
	static UE_API const FName IndependentDetailsPanel1;
	static UE_API const FName IndependentDetailsPanel2;
	static UE_API const FName IndependentDetailsPanel3;
	static UE_API const FName IndependentDetailsPanel4;
	static UE_API const FName OutputLog;
	static UE_API const FName SolversTrack;
	static UE_API const FName StatusBar;
	static UE_API const FName CollisionDataDetails;
	static UE_API const FName SceneQueryDataDetails;
	static UE_API const FName ConstraintsInspector;
	static UE_API const FName SceneQueryBrowser;
	static UE_API const FName RecordedOutputLog;
};

#undef UE_API
