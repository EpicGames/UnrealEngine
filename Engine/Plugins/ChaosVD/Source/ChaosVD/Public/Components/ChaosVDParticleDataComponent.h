// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDScene.h"
#include "ChaosVDSceneParticle.h"
#include "ChaosVDSceneStreaming.h"
#include "Chaos/AABBTree.h"
#include "Components/ChaosVDSolverDataComponent.h"
#include "Math/GenericOctree.h"
#include "TEDS/ChaosVDSelectionInterface.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "ChaosVDParticleDataComponent.generated.h"

class FChaosVDSceneStreaming;
struct FChaosVDParticleDataWrapper;
struct FChaosVDSceneParticle;

namespace Chaos::VD::Test::SceneObjectTypes
{
	extern FName ActiveParticles;
}

/**
 * Component that references all particle data for a specific solver for the current frame, and handles how the visualization
 * is updated based on that data
 */
UCLASS()
class UChaosVDParticleDataComponent : public UChaosVDSolverDataComponent, public IChaosVDStreamingDataSource
{
public:

	UChaosVDParticleDataComponent();

	typedef TArray<UE::Editor::DataStorage::RowHandle, TInlineAllocator<256, TMemStackAllocator<>>> FTedsHandlesForBatchArray;

private:
	GENERATED_BODY()

public:

	virtual void SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr) override;
	virtual void ClearData() override;

	virtual void UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData) override;
	virtual void UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData) override;
	
	template <class TCallback>
	void VisitSelectedParticleData(TCallback VisitCallback) const;

	template <class TCallback>
	void VisitAllParticleInstances(TCallback VisitCallback) const;

	template <class TCallback>
	void VisitAllParticleData(TCallback VisitCallback) const;

	virtual void SetVisibility(bool bNewIsVisible) override;

	TSharedPtr<FChaosVDSceneParticle> GetParticleInstanceByID(int32 ParticleID) const;
	TSharedPtr<FChaosVDSceneParticle> GetParticleInstanceByID_AssumesLocked(int32 ParticleID) const;

	TSharedPtr<FChaosVDBaseSceneObject> GetParticleContainerByType(EChaosVDParticleType ParticleType);
	TSharedPtr<FChaosVDBaseSceneObject> GetParticleContainerByType_AssumesLocked(EChaosVDParticleType ParticleType);

	const TSortedMap<EChaosVDParticleType, TSharedPtr<FChaosVDBaseSceneObject>>& GetParticleSceneContainersByType()
	{
		return ParticleSceneContainersByType;
	}

	CHAOSVD_API FChaosVDSceneParticle* GetSelectedParticle() const;

	virtual void HandleWorldStreamingLocationUpdated(const FVector& InLocation) override;

	// BEGIN IChaosVDStreamingDataSource
	virtual TConstArrayView<TSharedRef<FChaosVDBaseSceneObject>> GetStreamableSceneObjects() const override
	{
		return SolverParticlesArray;
	}
	
	virtual FRWLock& GetObjectsLock() const override
	{
		return ParticleSceneObjectsLock;
	}
	// END IChaosVDStreamingDataSource

	virtual void AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData) override;

protected:

	int32 GetMaxElementCountForBatchTEDSUpdate_AssumesLocked() const;

	void BatchTEDSSyncFromWorldTag(FTedsHandlesForBatchArray& ParticlesPendingTEDSTagSync,
		const UE::Editor::DataStorage::ICompatibilityProvider* CompatibilityStorage,
		UE::Editor::DataStorage::ICoreProvider* DataStorage);

	void BatchTEDSCombinedSyncFromWorldAndActiveTag(FTedsHandlesForBatchArray& ParticlesPendingCombinedTEDSTagSync,
	const UE::Editor::DataStorage::ICompatibilityProvider* CompatibilityStorage,
	UE::Editor::DataStorage::ICoreProvider* DataStorage);

	void BatchTEDSAddRemoveTags(UE::Editor::DataStorage::ICoreProvider* EditorDataStorage, FTedsHandlesForBatchArray& ParticlesPendingTEDSTagSync,
		TConstArrayView<const UScriptStruct*> ColumnsToAdd, TConstArrayView<const UScriptStruct*> ColumnsToRemove);

	void ProcessRemovedParticles_AssumessLocked(const TSharedRef<FChaosVDScene>& SceneRef, const FChaosVDSolverFrameData& InSolverFrameData,
		const FChaosVDFrameStageData& InSolverFrameStageData, const TSet<int32>& ParticlesUpdatedIDsForKeyFrameDiff, FTedsHandlesForBatchArray& OutRemovedTedsHandles);

	void BatchApplySolverVisibilityToParticle(bool bNewIsVisible);

	void HandleVisibilitySettingsUpdated(UObject* SettingsObject);
	void HandleColorsSettingsUpdated(UObject* SettingsObject);

	bool IsServerData();
	const FString& GetSolverName();

	/** Creates an ChaosVDParticle actor for the Provided recorded Particle Data */
	TSharedRef<FChaosVDSceneParticle> CreateSceneParticle_AssumesLocked(const TSharedRef<FChaosVDParticleDataWrapper>& InParticleData, const FChaosVDSolverFrameData& InFrameData);

	TOptional<bool> bCachedIsServerData;
	TOptional<FString> bCachedSolverName;

	TSortedMap<EChaosVDParticleType, TSharedPtr<FChaosVDBaseSceneObject>> ParticleSceneContainersByType;

	TMap<int32, TSharedRef<FChaosVDSceneParticle>> SolverParticlesByID;
	TArray<TSharedRef<FChaosVDBaseSceneObject>> SolverParticlesArray;

	mutable FRWLock ParticleSceneObjectsLock;

	FChaosVDSceneStreaming StreamingSystem;
};

template <typename TCallback>
void UChaosVDParticleDataComponent::VisitSelectedParticleData(TCallback VisitCallback) const
{
	TSharedPtr<FChaosVDScene> ScenePtr = SceneWeakPtr.Pin();
	if (!ScenePtr)
	{
		return;
	}
	using namespace Chaos::VD::TypedElementDataUtil;

	TArray<FTypedElementHandle, TInlineAllocator<8>> SelectedParticlesHandles;
	ScenePtr->GetElementSelectionSet()->GetSelectedElementHandles(SelectedParticlesHandles, UChaosVDSelectionInterface::StaticClass());

	for (const FTypedElementHandle& SelectedParticleHandle : SelectedParticlesHandles)
	{
		if (FChaosVDSceneParticle* ParticleInstance = GetStructDataFromTypedElementHandle<FChaosVDSceneParticle>(SelectedParticleHandle))
		{
			TSharedPtr<const FChaosVDParticleDataWrapper> ParticleDataViewer = ParticleInstance ? ParticleInstance->GetParticleData() : nullptr;
	
			if (!ensure(ParticleDataViewer))
			{
				continue;
			}

			if (ParticleDataViewer->SolverID != SolverID)
			{
				continue;
			}

			if (!VisitCallback(ParticleDataViewer))
			{
				return;
			}
		}
	}
}

template <typename TCallback>
void UChaosVDParticleDataComponent::VisitAllParticleInstances(TCallback VisitCallback) const
{
	FReadScopeLock ReadLock(ParticleSceneObjectsLock);

	for (const TPair<int32, TSharedRef<FChaosVDSceneParticle>>& ParticleWithIDPair : SolverParticlesByID)
	{
		if (!VisitCallback(ParticleWithIDPair.Value))
		{
			return;
		}
	}
}

template <typename TCallback>
void UChaosVDParticleDataComponent::VisitAllParticleData(TCallback VisitCallback) const
{
	FReadScopeLock ReadLock(ParticleSceneObjectsLock);

	for (const TPair<int32, TSharedRef<FChaosVDSceneParticle>>& ParticleWithIDPair : SolverParticlesByID)
	{
		TSharedPtr<const FChaosVDParticleDataWrapper> ParticleDataViewer = ParticleWithIDPair.Value->GetParticleData();
		if (!ensure(ParticleDataViewer))
		{
			continue;
		}

		if (!VisitCallback(ParticleDataViewer))
		{
			return;
		}
	}
}
