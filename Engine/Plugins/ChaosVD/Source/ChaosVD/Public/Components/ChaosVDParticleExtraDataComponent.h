// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChaosVDSolverDataComponent.h"
#include "DataWrappers/ChaosVDParticleExtraDataContainer.h"
#include "ChaosVDParticleExtraDataComponent.generated.h"

class IDetailLayoutBuilder;
class FChaosVDScene;

/** Holds a deserialized struct scope together with enough metadata to match it
 *  against fresh frame data for in-place memory updates. */
struct FChaosVDExtraDataScopeEntry
{
	FName CategoryName;
	FName StructTypePath;
	TSharedPtr<FStructOnScope> Scope;
};

/**
 * Solver data component that reads FChaosVDParticleExtraDataContainer from the current
 * solver frame and injects the recorded structs into the CVD particle details panel.
 *
 * Added to AChaosVDSolverInfoActor as a default subobject. It binds to
 * FChaosVDScene::OnParticleDetailsExtraDataRequested and for each selected particle
 * adds the traced categories to the IDetailLayoutBuilder, so the extra data appears
 * alongside "Particle Data" in the details panel (and in "Show Data in New Panel").
 *
 * On subsequent frame changes (OnParticleDetailsExtraDataRefreshRequested), it updates
 * the cached FStructOnScope memory in-place so Slate picks up the new values without
 * requiring a full layout rebuild.
 */
UCLASS(MinimalAPI)
class UChaosVDParticleExtraDataComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()

public:

	CHAOSVD_API virtual void ClearData() override;
	CHAOSVD_API virtual void UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData) override;
	CHAOSVD_API virtual void SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr) override;

private:

	void HandleParticleDetailsExtraData(int32 InSolverID, int32 InParticleID, IDetailLayoutBuilder& DetailBuilder);
	void HandleParticleDetailsRefresh(int32 InSolverID, int32 InParticleID, bool& bOutNeedsFullRebuild);
	void UnbindFromScene(const TSharedPtr<FChaosVDScene>& Scene);

	/** Extra data container for the current frame, looked up once per frame update. */
	TSharedPtr<FChaosVDParticleExtraDataContainer> CurrentFrameContainer;

	/** Stable FStructOnScope instances added to the details panel for the currently selected particle.
	 *  Kept alive so we can update their memory in-place on frame changes without rebuilding the layout. */
	TArray<FChaosVDExtraDataScopeEntry> CachedScopeEntries;

	/** Solver/particle for which CachedScopeEntries were built. */
	int32 CachedSolverID = INDEX_NONE;
	int32 CachedParticleID = INDEX_NONE;

	FDelegateHandle ExtraDataDelegateHandle;
	FDelegateHandle RefreshDelegateHandle;
};
