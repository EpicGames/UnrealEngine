// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ActorComponent.h"
#include "ChaosVDSolverDataComponent.generated.h"

struct FChaosVDSceneCompositionTestData;
struct FChaosVDFrameStageData;
struct FChaosVDSolverFrameData;
class FChaosVDScene;
struct FChaosVDGameFrameData;
/**
 * Base class for all components that stores recorded solver data
 */
UCLASS(MinimalAPI, Abstract)
class UChaosVDSolverDataComponent : public UActorComponent
{
	GENERATED_BODY()
public:
	virtual void ClearData() PURE_VIRTUAL(UChaosVDSolverDataComponent::ClearData);
	CHAOSVD_API virtual void SetScene(const TWeakPtr<FChaosVDScene>& InSceneWeakPtr);

	virtual void UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData) {}
	virtual void UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData) {}
	virtual void UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData) {}

	bool IsVisible() const
	{
		return bIsVisible;
	}

	CHAOSVD_API virtual void SetVisibility(bool bNewIsVisible);

	virtual void HandleWorldStreamingLocationUpdated(const FVector& InLocation) {}
	
	void SetSolverID(int32 InSolverID)
	{
		SolverID = InSolverID;
	}

	/**
	 * Appends Scene Data relevant for functional tests
	 * @param OutStateTestData Structure instance where to add the data
	 * @note If you change the data being added here, you need to re-generate the snapshots used by the
	 * scene integrity playback tests in the Simulation Tests Plugin
	 */
	virtual void AppendSceneCompositionTestData(FChaosVDSceneCompositionTestData& OutStateTestData) {}

protected:

	TWeakPtr<FChaosVDScene> SceneWeakPtr;

	int32 SolverID = INDEX_NONE;

	bool bIsVisible = true;
};

