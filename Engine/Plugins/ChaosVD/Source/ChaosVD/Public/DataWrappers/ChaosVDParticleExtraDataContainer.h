// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVisualDebugger/ChaosVDParticleExtraData.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "Containers/Map.h"
#include "Templates/SharedPointer.h"

#include "ChaosVDParticleExtraDataContainer.generated.h"

/**
 * Custom data container stored in FChaosVDSolverFrameData::CustomData.
 *
 * Maps SolverID -> ParticleID -> extra data payload for O(1) lookup at display time.
 * Must be a USTRUCT so FChaosVDCustomFrameData::GetOrAddDefaultData<T>() can key it
 * via StaticStruct()->GetFName()
 *
 * The NameTable is set by FChaosVDParticleExtraDataProcessor at load time so that
 * UChaosVDParticleExtraDataComponent can deserialize inner struct bytes without
 * needing a separate route to the trace provider.
 */
USTRUCT()
struct FChaosVDParticleExtraDataContainer
{
	GENERATED_BODY()

	TMap<int32, TMap<int32, TSharedPtr<Chaos::VisualDebugger::FChaosVDParticleExtraData>>> DataBySolverAndParticleID;

	/** Recording-session name table, cached here by the processor so the component can resolve FName IDs in inner struct bytes. */
	TSharedPtr<Chaos::VisualDebugger::FChaosVDSerializableNameTable> NameTable;

	/** Struct type paths for entries that used NativeSerialization mode.
	 *  Populated by FChaosVDParticleExtraDataProcessor at load time. */
	TSet<FName> NativeSerializedStructTypes;
};
