// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Features/IModularFeature.h"
#include "Misc/Guid.h"
#include "Templates/SharedPointer.h"
#include "Trace/ChaosVDTraceProvider.h"
#include "UObject/NameTypes.h"

class FChaosVDComponentVisualizerBase;
class FChaosVDDataProcessorBase;
class FChaosVDPlaybackController;
class FChaosVDTabSpawnerBase;
class FTabManager;
class IDetailsView;
class SChaosVDMainTab;
class UActorComponent;

struct FChaosVDTrackInfo;

/** Base class for any CVD Extension. This object will auto register itself with the extensions system, and receive any relevant CVD callbacks */
class FChaosVDExtension : public IModularFeature
{
public:

	FChaosVDExtension() = default;

	virtual ~FChaosVDExtension() = default;

	/** FName used as Type within CVD Extension system */
	FName GetExtensionType() const
	{
		return ExtensionName;	
	}

	/** Returns an array view of all available data processors instances in this extension */
	virtual void RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider) {}
	
	/** Returns an array view of all available solver data component instances in this extension, if any */
	virtual TConstArrayView<TSubclassOf<UActorComponent>> GetSolverDataComponentsClasses()
	{
		return TConstArrayView<TSubclassOf<UActorComponent>>();
	}

	/** Registers all available component visualizer instances in this extension, if any */
	virtual void RegisterComponentVisualizers(const TSharedRef<SChaosVDMainTab>& InCVDToolKit) {}

	/** Registers all available Tab Spawner instances in this extension, if any */
	virtual void RegisterCustomTabSpawners(const TSharedRef<SChaosVDMainTab>& InParentTabWidget) {}

	/** Initialization by Extension after MainTab is constructed */
	virtual void PostMainTabInitialization(const TSharedRef<SChaosVDMainTab>& InParentTabWidget) {}

	/** Injects any customization implementations for CVD's details panels*/
	virtual void SetCustomPropertyLayouts(IDetailsView* DetailsView, TSharedRef<SChaosVDMainTab> InCVDToolKit) {}

	/** Handles new data being loaded into a CVD Instance */
	virtual void HandlePlaybackControllerDataUpdated(TWeakPtr<FChaosVDPlaybackController> InController) {}

	/** Handles playback state changes on a CVD Instance */
	virtual void HandleControllerTrackFrameUpdated(TWeakPtr<FChaosVDPlaybackController> InController, TWeakPtr<const FChaosVDTrackInfo> UpdatedTrackInfo, FGuid InstigatorGuid) {}

	/** Handles initial first frame playback after a recording is loading */
	virtual void HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController) {}

protected:
	FName ExtensionName;
};
