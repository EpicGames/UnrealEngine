// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDExtension.h"

class FName;

/** Manager class where all CVD extensions are registered to*/
class FChaosVDExtensionsManager
{
public:
	FChaosVDExtensionsManager();
	~FChaosVDExtensionsManager();

	/** Iterates through all registered extensions, and calls the provided callback with them
	 * @param ExtensionVisitor Callback to call for each extension
	 */
	template <class VisitorType>
	void EnumerateExtensions(const VisitorType& ExtensionVisitor);

	/** Returns an instance to CVD's extensions manager*/
	CHAOSVD_API static FChaosVDExtensionsManager& Get();

	/** De-initializes CVD extenisons manager*/
	static void TearDown();

	/** Registers a CVD extension instance
	 * @param InExtension Instances of the extension to register
	 */
	CHAOSVD_API void RegisterExtension(const TSharedRef<FChaosVDExtension>& InExtension);
	
	/** Unregisters a CVD extension instance. Usually called during editor shutdown
	 * @param InExtension Instances of the extension to unregister
	 */
	CHAOSVD_API void UnRegisterExtension(const TSharedRef<FChaosVDExtension>& InExtension);

	DECLARE_EVENT_OneParam(FChaosVDExtensionsManager, FOnExtensionChanged, const TSharedRef<FChaosVDExtension>&);

	/** Event called each time a new CVD extension is registered */
	FOnExtensionChanged& OnExtensionRegistered()
	{
		return ExtensionRegisteredEvent;
	}

	/** Event called each time a new CVD extension is unregistered. Usually during editor shutdown */
	FOnExtensionChanged& OnExtensionUnRegistered()
	{
		return ExtensionUnRegisteredEvent;
	}

private:

	FOnExtensionChanged ExtensionRegisteredEvent;
	FOnExtensionChanged ExtensionUnRegisteredEvent;

	TMap<FName, TSharedRef<FChaosVDExtension>> AvailableExtensions;
};

template <typename VisitorType>
void FChaosVDExtensionsManager::EnumerateExtensions(const VisitorType& ExtensionVisitor)
{
	for (const TPair<FName, TSharedRef<FChaosVDExtension>>& ExtensionWithName : AvailableExtensions)
	{
		if (!ExtensionVisitor(ExtensionWithName.Value))
		{
			return;
		}
	}
}
