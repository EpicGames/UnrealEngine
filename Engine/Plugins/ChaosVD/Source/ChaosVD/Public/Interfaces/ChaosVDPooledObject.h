// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ChaosVDPooledObject.generated.h"

UINTERFACE(MinimalAPI)
class UChaosVDPooledObject : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface used by any object that can be pooled
 */
class IChaosVDPooledObject
{
	GENERATED_BODY()

public:
	virtual void OnAcquired() PURE_VIRTUAL(IChaosVDPooledObject::OnAcquired)
	virtual void OnDisposed() PURE_VIRTUAL(IChaosVDPooledObject::OnDisposed)
};
