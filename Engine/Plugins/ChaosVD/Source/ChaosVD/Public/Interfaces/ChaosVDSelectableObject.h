// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Interface.h"
#include "ChaosVDSelectableObject.generated.h"

UINTERFACE(MinimalAPI)
class UChaosVDSelectableObject : public UInterface
{
	GENERATED_BODY()
};

/**
 * Interface used for any object that can be selected in CVD and need to process selection events performed on it.
 */
class IChaosVDSelectableObject
{
	GENERATED_BODY()

public:
	virtual void HandleSelected() PURE_VIRTUAL(IChaosVDSelectionAware::HandleSelected);
	virtual void HandleDeSelected() PURE_VIRTUAL(IChaosVDSelectionAware::HandleUnSelected);
};
