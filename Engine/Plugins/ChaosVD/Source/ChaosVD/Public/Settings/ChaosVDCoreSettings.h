// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/World.h"
#include "Engine/StaticMesh.h"
#include "UObject/Object.h"
#include "UObject/SoftObjectPtr.h"

#include "ChaosVDCoreSettings.generated.h"

class UChaosVDCoreSettings;
class UMaterial;
class UTextureCube;

DECLARE_MULTICAST_DELEGATE_OneParam(FChaosVDSettingChanged, UObject* SettingsObject)

UCLASS(MinimalAPI)
class UChaosVDSettingsObjectsOuter : public UObject
{
	GENERATED_BODY()
};

/** Base class to be used by any CVD settings.
 * Contains the base logic to make these settings work with CVD's options save system
 */
UCLASS(config = ChaosVD, MinimalAPI)
class UChaosVDSettingsObjectBase : public UObject
{
public:
	CHAOSVD_API UChaosVDSettingsObjectBase();

private:
	GENERATED_BODY()

public:

	CHAOSVD_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

	FChaosVDSettingChanged& OnSettingsChanged() { return SettingsChangedDelegate; }

	CHAOSVD_API virtual void PostEditUndo() override;

	CHAOSVD_API virtual void OverridePerObjectConfigSection(FString& SectionName) override;

	FStringView GetConfigSectionName()
	{
		return OverrideConfigSectionName;
	}

protected:
	CHAOSVD_API virtual void BroadcastSettingsChanged();
	
private:

	FString OverrideConfigSectionName;
	FChaosVDSettingChanged SettingsChangedDelegate;

	friend class FChaosVDSettingsManager;
};

/** Base class to be used by any CVD settings related to visualization.
 * Contains the base logic to make these settings work with CVD's options save system
 * And makes sure that the viewport gets re-draw when a setting changes
 * 
 */
UCLASS(config = ChaosVD, MinimalAPI)
class UChaosVDVisualizationSettingsObjectBase : public UChaosVDSettingsObjectBase
{
	GENERATED_BODY()
protected:
	CHAOSVD_API virtual void BroadcastSettingsChanged() override;

public:
	/** Returns true if a visualization flag option of this setting option should be enabled in the UI */
	CHAOSVD_API virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag);
};

/** Core settings class for CVD */
UCLASS(config = Engine)
class UChaosVDCoreSettings : public UChaosVDSettingsObjectBase
{
	GENERATED_BODY()
public:

	UPROPERTY(Config, Transient)
	TSoftObjectPtr<UMaterial> QueryOnlyMeshesMaterial;

	UPROPERTY(Config, Transient)
	TSoftObjectPtr<UMaterial> SimOnlyMeshesMaterial;

	UPROPERTY(Config, Transient)
	TSoftObjectPtr<UMaterial> InstancedMeshesMaterial;

	UPROPERTY(Config, Transient)
	TSoftObjectPtr<UMaterial> InstancedMeshesQueryOnlyMaterial;

	UPROPERTY(Config)
	FSoftClassPath SkySphereActorClass;

	UPROPERTY(Config)
	TSoftObjectPtr<UTextureCube> AmbientCubeMapTexture;

	UPROPERTY(Config)
	TSoftObjectPtr<UStaticMesh> BoxMesh;

	UPROPERTY(Config)
	TSoftObjectPtr<UStaticMesh> SphereMesh;
};
