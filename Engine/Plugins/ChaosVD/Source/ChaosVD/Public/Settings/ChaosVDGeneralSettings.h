// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "ChaosVDRecordingDetails.h"
#include "ChaosVDGeneralSettings.generated.h"

/** Controls how FChaosVDParticleExtraDataProcessor handles particle extra data at load time.
 *
 *  Reflection-based entries use property-tagged serialization: properties are matched by name at
 *  load time against the *current* struct layout, so adds/removes/renames are handled gracefully
 *  (missing props show defaults; added props are lost). No crash risk.
 *
 *  Natively-serialized entries invoke the struct's native Serialize function, which reads bytes
 *  at fixed offsets. When the struct layout matches, values are exact. When it does not match,
 *  the result is a crash or silently corrupt values -- there is no graceful degradation.
 */
UENUM()
enum class EChaosVDParticleExtraDataLoadingMode : uint8
{
	/** Skip the processor entirely — no extra data is loaded or displayed. The only mode that
	 *  guarantees neither incorrect values nor crashes regardless of how much struct definitions
	 *  have changed since recording. */
	SkipAll          UMETA(DisplayName = "Skip All (Safe)"),

	/** Load only reflection-based entries; natively-serialized entries are stripped in the processor
	 *  before reaching memory. No crash risk, but displayed values may be inaccurate if struct
	 *  definitions changed since recording. */
	SerializeBinOnly UMETA(DisplayName = "Reflection-Based Only"),

	/** Load all entries including natively-serialized ones. Displays exact recorded values when the
	 *  struct layout matches the recording. If the layout has changed, natively-serialized entries
	 *  may crash or display corrupt values. Only use when the recording and viewer binaries are
	 *  known to match. */
	LoadAll          UMETA(DisplayName = "Load All (Unsafe)"),
};

/**
 * General settings that controls how CVD behaves
 */
UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDGeneralSettings : public UChaosVDSettingsObjectBase
{
	GENERATED_BODY()

public:

	/** Returns the configured transport mode for the provided target type
	 * @param TargetType Build target type to evaluate
	 */
	CHAOSVD_API UE::TraceBasedDebuggers::ETraceTransportMode GetTransportModeForTargetType(EBuildTargetType TargetType) const;

	/** Sets a specific transport mode for the provided target type
	 * @param TargetType Build target type to evaluate
	 * @param NewTransportMode New transport mode to use for the provided target
	 */
	CHAOSVD_API void SetTransportModeForTargetType(EBuildTargetType TargetType, UE::TraceBasedDebuggers::ETraceTransportMode NewTransportMode);

PRAGMA_DISABLE_DEPRECATION_WARNINGS
	UE_DEPRECATED(5.8, "Use GetTransportModeForTargetType")
	CHAOSVD_API EChaosVDTransportMode GetTransportModeForTarget(EBuildTargetType TargetType) const;

	UE_DEPRECATED(5.8, "Use SetTransportModeForTargetType")
	CHAOSVD_API void SetTransportModeForTarget(EBuildTargetType TargetType, EChaosVDTransportMode NewTransportMode);
PRAGMA_ENABLE_DEPRECATION_WARNINGS

	/** If true, CVD will only load frames that have solver data in them - Only takes effect before loading a file */
	UPROPERTY(Config, EditAnywhere, Category = "Session Data Loading")
	bool bTrimEmptyFrames = true;

	/** How many Game thread frames CVD should queue internally before making them available in the visualization and timeline controls - Only takes effect before loading a file */
	UPROPERTY(Config, EditAnywhere, Category = "Session Data Loading")
	int32 MaxGameThreadFramesToQueueNum = 10;

	/** If true, CVD will only load collision geometry that is visible */
	UPROPERTY(Config, EditAnywhere, Category = "Performance | Streaming")
	bool bStreamingSystemEnabled = true;

	/** Extent size of the box used for calculate what should be streamed in */
	UPROPERTY(Config, EditAnywhere, Category = "Performance | Streaming")
	float StreamingBoxExtentSize = 10000.0f;

	/** If set to true CVD will process any updates to the streaming accel structure in worker threads, in between streaming updates */
	UPROPERTY(Config, EditAnywhere, Category = "Performance | Streaming")
	bool bProcessPendingOperationsQueueInWorkerThread = true;

	/** If set to true CVD will keep the scene outliner up to date as the recording is played. If during the recording a
	 * significant amount of objects are loaded/unloaded, the performance impact will be significant enough to degrade the playback stability. */
	UPROPERTY(Config, EditAnywhere, Category = "Performance | Scene Outliner")
	bool bUpdateSceneOutlinerDuringPlayback = true;

	/** How many times CVD will attempt to connect to a live trace or load from file session if the first attempt failed */
	UPROPERTY(Config, EditAnywhere, Category = Connection)
	int32 MaxConnectionRetries = 14;

	/** If True, any traces done to memory will also be saved to disk at the same time in the user's document folder (or a user specified folder). This is not saved by design */
	UPROPERTY(EditAnywhere, Category = Connection)
	bool bSaveRecordingsToDisk = true;

	/** Custom path where recordings will be saved from. By default, recordings are saved in a Chaos Visual Debugger folder in the documents folder */
	UPROPERTY(Config, EditAnywhere, Category = Connection)
	FFilePath DefaultSavePathOverride;

	/** If True, playback will start automatically after a recording started, and we successfully connect to it */
	UPROPERTY(Config, EditAnywhere, Category = Connection)
	bool bAutoStartLiveSessionsPlayback = true;

	/**
	 * If true, CVD will skip data whose binary format is not guaranteed to be backwards compatible
	 * (e.g. raw-serialized structs such as generic particle extra data).
	 * This prevents crashes when opening a recording made with a different executable than the viewer.
	 *
	 * When enabled, any skipped data types are reported in a dialog after loading.
	 * Disable this only when you are certain the recording was made with the same binary as the viewer.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Backwards Compatibility")
	bool bSafeLoadingMode = true;

	/**
	 * Controls how particle extra data is loaded. Particle extra data is not backwards compatible
	 * by design -- it accepts arbitrary UStructs without requiring dedicated data wrappers.
	 *
	 * SkipAll: processor is skipped entirely. No extra data is loaded or displayed. The only mode
	 *   that guarantees neither crashes nor silently incorrect values, regardless of how much struct
	 *   definitions have changed since recording.
	 *
	 * Reflection-Based Only (default): only reflection-based (property-tagged) entries are loaded;
	 *   natively-serialized entries are filtered out in the processor and never stored in memory.
	 *   No crash risk, but values may be inaccurate if struct properties were added, removed, or
	 *   renamed since recording.
	 *
	 * LoadAll: all entries including natively-serialized ones are loaded. Displays exact recorded
	 *   values when the viewer binary matches the recording binary. If struct layouts have changed,
	 *   natively-serialized entries may crash or show corrupt values. Only use when recording and
	 *   viewer are known to match.
	 *
	 * Note: bSafeLoadingMode applies to any other non-backwards-compatible processors; this enum
	 * takes precedence for particle extra data specifically.
	 */
	UPROPERTY(Config, EditAnywhere, Category = "Backwards Compatibility")
	EChaosVDParticleExtraDataLoadingMode ParticleExtraDataLoadingMode = EChaosVDParticleExtraDataLoadingMode::SerializeBinOnly;

	/** If true, a warning dialog is shown after loading when non-backwards-compatible data was processed
	 *  with Safe Loading Mode disabled. Uncheck to suppress the warning. */
	UPROPERTY(Config, EditAnywhere, Category = "Backwards Compatibility")
	bool bShowUnsafeDataWarning = true;

	/** If true, an informational dialog is shown after loading when data types were skipped because
	 *  Safe Loading Mode is enabled. Uncheck to suppress the warning. */
	UPROPERTY(Config, EditAnywhere, Category = "Backwards Compatibility")
	bool bShowSafeModeSkippedWarning = true;

	/** If true, shows a warning dialog after loading when native-serialized extra data is
	 *  present and ParticleExtraDataLoadingMode is LoadAll. */
	UPROPERTY(Config, EditAnywhere, Category = "Backwards Compatibility")
	bool bShowNativeSerializationWarning = true;

private:
	/** [DEBUG Setting] Default Data Transport for Game Servers - Used to change the transport mode for the trace data. This is not saved by design */
	UPROPERTY(EditAnywhere, Category = Connection)
	UE::TraceBasedDebuggers::ETraceTransportMode GameServerDataTransportMode = UE::TraceBasedDebuggers::ETraceTransportMode::Direct;

	/** [DEBUG Setting] Default Data Transport for Game Clients - Used to change the transport mode for the trace data. This is not saved by design */
	UPROPERTY(EditAnywhere, Category = Connection)
	UE::TraceBasedDebuggers::ETraceTransportMode GameClientDataTransportMode = UE::TraceBasedDebuggers::ETraceTransportMode::Direct;
};
