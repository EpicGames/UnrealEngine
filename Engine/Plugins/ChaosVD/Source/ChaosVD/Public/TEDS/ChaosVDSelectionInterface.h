// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Interfaces/TedsTypedElementBridgeInterface.h"
#include "Elements/Interfaces/TypedElementSelectionInterface.h"
#include "UObject/Object.h"
#include "ChaosVDSelectionInterface.generated.h"

/**
 * CVD Custom Selection Interface. We don't have any specific implementation overrides yet
 */
UCLASS(MinimalAPI)
class UChaosVDSelectionInterface : public UObject, public ITypedElementSelectionInterface
{
	GENERATED_BODY()
};

UCLASS(MinimalAPI)
class UChaosVDTedsTypedElementBridgeInterface : public UObject, public ITedsTypedElementBridgeInterface
{
	GENERATED_BODY()
public:
	virtual FTedsRowHandle GetRowHandle(const FTypedElementHandle& InElementHandle) const override;
};