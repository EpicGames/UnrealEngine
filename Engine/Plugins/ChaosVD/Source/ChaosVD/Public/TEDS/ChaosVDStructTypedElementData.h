// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Elements/Framework/TypedElementData.h"
#include "Elements/Framework/TypedElementHandle.h"
#include "Elements/Framework/TypedElementOwnerStore.h"
#include "Elements/Framework/TypedElementRegistry.h"

#define UE_API CHAOSVD_API

class FStructOnScope;

namespace Chaos::VD::TypedElementDataUtil
{
	static const FName NAME_CVD_StructDataElement = FName("CVD_StructDataElement");

	struct FStructTypedElementData
	{
		UE_DECLARE_TYPED_ELEMENT_DATA_RTTI(FStructTypedElementData, UE_API);

		UStruct* TypeInfo = nullptr;
		void* RawData = nullptr;

		template<typename StructDataType>
		StructDataType* GetData() const;

		UE_API TSharedPtr<FStructOnScope> GetDataAsStructScope() const;
	};

	CHAOSVD_API extern TTypedElementOwnerStore<FStructTypedElementData, void*> GCVDTypedStructDataElementOwnerStore;

	template <typename StructDataType>
	StructDataType* FStructTypedElementData::GetData() const
	{
		if (TypeInfo && StructDataType::StaticStruct()->IsChildOf(TypeInfo))
		{
			return static_cast<StructDataType*>(RawData);
		}

		return nullptr;
	}

	template<typename StructDataType>
	TTypedElementOwner<FStructTypedElementData> CreateTypedElementDataForStructData(StructDataType* InElementData)
	{
		UTypedElementRegistry* Registry = UTypedElementRegistry::GetInstance();

		TTypedElementOwner<FStructTypedElementData> TypedElement;
		if (ensureMsgf(Registry, TEXT("Typed element was requested for '%s' before the registry was available! This usually means that NewObject was used instead of CreateDefaultSubobject during CDO construction."), *InElementData->GetDisplayName()))
		{
			TypedElement = Registry->CreateElement<FStructTypedElementData>(NAME_CVD_StructDataElement);
		}

		if (TypedElement)
		{
			TypedElement.GetDataChecked().TypeInfo = StructDataType::StaticStruct();
			TypedElement.GetDataChecked().RawData = InElementData;
		}
		return TypedElement;	
	}

	template<typename StructDataType>
	FTypedElementHandle AcquireTypedElementHandleForStruct(StructDataType* ElementInstance, const bool bAllowCreate)
	{
		if (!ElementInstance)
		{
			return FTypedElementHandle();
		}

		TTypedElementOwnerScopedAccess<FStructTypedElementData> EditorElement = bAllowCreate
																			   ? GCVDTypedStructDataElementOwnerStore.FindOrRegisterElementOwner(ElementInstance, [ElementInstance]()
																			   {
																				   return CreateTypedElementDataForStructData(ElementInstance);
																			   })
																			   : GCVDTypedStructDataElementOwnerStore.FindElementOwner(ElementInstance);
		
		if (EditorElement)
		{
			return EditorElement->AcquireHandle();
		}

		return FTypedElementHandle();
	}

	CHAOSVD_API void DestroyTypedElementHandleForStruct(void* InElementData);

	template <typename StructDataType>
	StructDataType* GetStructDataFromTypedElementHandle(const FTypedElementHandle& InHandle, const bool bSilent = true)
	{
		const FStructTypedElementData* StructElement = InHandle.GetData<FStructTypedElementData>(bSilent);
		return StructElement ? StructElement->GetData<StructDataType>() : nullptr;
	}

	CHAOSVD_API TSharedPtr<FStructOnScope> GetStructOnScopeDataFromTypedElementHandle(const FTypedElementHandle& InHandle, const bool bSilent = false);

	void CleanUpTypedElementStore();
}

template <>
inline FString GetTypedElementDebugId< Chaos::VD::TypedElementDataUtil::FStructTypedElementData>(const  Chaos::VD::TypedElementDataUtil::FStructTypedElementData& InElementData)
{
	return FString::Printf(TEXT("[%s] - [%p]"), InElementData.TypeInfo ? *InElementData.TypeInfo->GetName() : TEXT("Invalid Struct info"), InElementData.RawData);
}


#undef UE_API
