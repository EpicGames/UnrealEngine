// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/Serialization/SerializedDataBuffer.h"
#include "ChaosVDModule.h"
#include "Containers/SpscQueue.h"
#include "Containers/StringFwd.h"
#include "Framework/Notifications/NotificationManager.h"
#include "Templates/SharedPointer.h"
#include "Trace/StoreClient.h"

struct FChaosVDRelayTraceDataMessage;
enum class EChaosVDLoadRecordedDataMode : uint8;
class FChaosVDEngine;
class FChaosVDTraceModule;

namespace TraceServices { class IAnalysisSession; }

namespace Chaos::VD
{
PRAGMA_DISABLE_DEPRECATION_WARNINGS
	struct UE_DEPRECATED(5.8, "This struct will no longer be exposed publicly") FAsyncProgressNotification : FTSTickerObjectBase
	{
		FAsyncProgressNotification(const FText&) {}
		void EnterProgress(int32 InCurrentProgress) {}
	};

	class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::IRemoteSessionDataStream instead") IRemoteSessionDataStream
	{
	public:
		virtual ~IRemoteSessionDataStream() = default;
		virtual FGuid GetOwningRemoteSessionID() const = 0;
		virtual uint64 UpdateAndGetBytesReadSinceLastMeasurement() = 0;
	};

	class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FSessionStreamStatsUpdater instead") FSessionStreamStatsUpdater : FTSTickerObjectBase
	{
	public:
		explicit FSessionStreamStatsUpdater(IRemoteSessionDataStream* const DataStreamInstance) {}
	};

	class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FDirectSocketStream instead") FDirectSocketStream : public UE::Trace::FDirectSocketStream, public IRemoteSessionDataStream
	{
	public:
		FDirectSocketStream() = delete;
		explicit FDirectSocketStream(FGuid) {}
		explicit FDirectSocketStream(FGuid, TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter)
			: UE::Trace::FDirectSocketStream(MoveTemp(InFileWriter))
		{
		}

		virtual int32 Read(void* Data, uint32 Size) override { return 0; }
		virtual FGuid GetOwningRemoteSessionID() const override { return FGuid{}; }
		virtual uint64 UpdateAndGetBytesReadSinceLastMeasurement() override { return 0; }
	};

	class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FRelayDataStream instead") FRelayDataStream : public UE::Trace::IInDataStream,  public IRemoteSessionDataStream
	{
	public:
		explicit FRelayDataStream(FGuid InRemoteSessionID) {}
		explicit FRelayDataStream(FGuid InRemoteSessionID, TUniquePtr<FArchiveFileWriterGeneric>&& InFileWriter) {}

		void EnqueueRelayedData(const TConstArrayView<uint8> InTraceDataBuffer) {}
		void EnqueueRelayedData(TArray<uint8>&& InTraceDataBuffer) {}

		virtual FGuid GetOwningRemoteSessionID() const override { return FGuid{}; }
		virtual uint64 UpdateAndGetBytesReadSinceLastMeasurement() override { return 0; }
	};
} // Chaos::VD

struct UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FTraceSessionDescriptor instead") FChaosVDTraceSessionDescriptor
{

	/** The trace session name. Used to get the trace analysis instance for this CVD session*/
	FString SessionName;
	/** Port number used to establish a connection to this session. The value is 0 if no specific port was used*/
	uint16 SessionPort = 0;
	/** Session ID of this CVD session in the Remote Session management system*/
	FGuid RemoteSessionID;

	/** True if this is a session recorded live (not loaded from disk)*/
	bool bIsLiveSession = false;

	/** Returns true if valid. For a session to be valued needs at minimum have a Trace Session name */
	bool IsValid() const
	{
		return !SessionName.IsEmpty();
	}

	friend bool operator==(const FChaosVDTraceSessionDescriptor& Lhs, const FChaosVDTraceSessionDescriptor& RHS)
	{
		return Lhs.SessionName == RHS.SessionName
			&& Lhs.RemoteSessionID == RHS.RemoteSessionID;
	}

	friend bool operator!=(const FChaosVDTraceSessionDescriptor& Lhs, const FChaosVDTraceSessionDescriptor& RHS)
	{
		return !(Lhs == RHS);
	}
};

/**
 * Objects that allows us to use TLS to temporarily store and access a ptr to an existing instance.
 * This is temporary to work around the lack of an API method we need in the trace API, and will be removed in the future.
 * Either when we add that to the API, or find another way to pass an existing CVD recording to the trace provider before analysis starts
 */
class UE_EXPERIMENTAL(5.8, "This class is temporary and will be removed when a method in the trace API allows us to pass an existing recording to the trace provider before analysis starts.")
	FChaosVDTraceManagerThreadContext : public TThreadSingleton<FChaosVDTraceManagerThreadContext>
{
public:

	TWeakPtr<FChaosVDRecording> PendingExternalRecordingWeakPtr;
};

class UE_DEPRECATED(5.8, "Use UE::TraceBasedDebuggers::FTraceSessionsManager instead") FChaosVDTraceManager : public TThreadSingleton<FChaosVDTraceManagerThreadContext>
{
public:
	FChaosVDTraceManager();
	~FChaosVDTraceManager();

	/** Load a trace file and starts analyzing it
	 * @param InTraceFilename File Name including (Path Included) of the Trace file to load
	 */
	CHAOSVD_API FString LoadTraceFile(const FString& InTraceFilename, const TSharedPtr<FChaosVDRecording>& ExistingRecordingPtr = nullptr)
	{
		return FString();
	}

	FString LoadTraceFile(TUniquePtr<IFileHandle>&& InFileHandle, const FString& InTraceSessionName, const TSharedPtr<FChaosVDRecording>& ExistingRecordingPtr)
	{
		return FString();
	}

	/**
	 * Connects to a live Trace Session and starts analyzing it.
	 * @param InSessionHost Trace Store Address for this session
	 * @param SessionID Trace ID in the Trace Store provided as host
	 * @param ExistingRecordingPtr Ptr to an existing recording instance to append the new CVD data to, if any
	 * @return Session Name
	 */
	FString ConnectToLiveSession(FStringView InSessionHost, uint32 SessionID, const TSharedPtr<FChaosVDRecording>& ExistingRecordingPtr = nullptr)
	{
		return FString();
	}

	UE_DEPRECATED(5.8, "Please use the version that takes a Remote Session ID")
	FString ConnectToLiveSession_Direct( uint16& OutSessionPort, const TSharedPtr<FChaosVDRecording>& ExistingRecordingPtr = nullptr)
	{
		return FString();
	};

	/**
	 * Opens up a direct trace stream and waits for data
	 * @param RemoteSessionID Session ID to which we want to connect to
	 * @param OutSessionPort Number of the port we open for this session
	 * @param ExistingRecordingPtr Ptr to an existing recording instance to append the new CVD data to, if any
	 * @return Session Name
	 */
	FString ConnectToLiveSession_Direct(FGuid RemoteSessionID, uint16& OutSessionPort, const TSharedPtr<FChaosVDRecording>& ExistingRecordingPtr = nullptr)
	{
		return FString();
	}

	/**
	 * Opens up a relay trace stream and waits for data
	 * @param RemoteSessionID Session ID to which we want to connect to
	 * @param ExistingRecordingPtr Ptr to an existing recording instance to append the new CVD data to, if any
	 * @return Session Name
	 */
	FString ConnectToLiveSession_Relay(FGuid RemoteSessionID, const TSharedPtr<FChaosVDRecording>& ExistingRecordingPtr = nullptr)
	{
		return FString();
	}

	/** Returns the path to the local trace store */
	FString GetLocalTraceStoreDirPath()
	{
		return FString();
	}

	/** Returns a ptr to the session registered with the provided session name. Null if no session is found */
	CHAOSVD_API TSharedPtr<const TraceServices::IAnalysisSession> GetSession(const FString& InSessionName)
	{
		return nullptr;
	}

	/** Stops and de-registers a trace session registered with the provided session name */
	CHAOSVD_API void CloseSession(const FString& InSessionName)
	{
	}

	/** Stops a trace session registered with the provided session name */
	void StopSession(const FString& InSessionName)
	{
	}

	template<typename TVisitor>
	static void EnumerateActiveSessions(FStringView InSessionHost, TVisitor Callback)
	{
		// deprecated
	}

	static const UE::Trace::FStoreClient::FSessionInfo* GetTraceSessionInfo(FStringView InSessionHost, FGuid TraceGuid)
	{
		return nullptr;
	}

	template<typename TVisitor>
	void EnumerateActiveSessions(TVisitor Callback)
	{
		// deprecated
	}

	/** Access the trace store at the provided host address, and returns the file name for the trace file
	 * @param InSessionHost Address of the Trace Store
	 * @param SessionID Session id of the trace data from which get the file name from
	 * @return Full Filename with path to the trace file if exist. Returns an empty string if a trace file cannot be found
	 */
	FString GetTraceFileNameFromStoreForSession(FStringView InSessionHost, uint32 SessionID)
	{
		return FString();
	}

	/** Access the trace store at the provided host address, and returns a trace data stream ready to use
	 * @param InSessionHost Address of the Trace Store
	 * @param SessionID Session id of the trace data
	 */
	UE::Trace::FStoreClient::FTraceData GetTraceDataStreamFromStore(FStringView InSessionHost, uint32 SessionID)
	{
		return UE::Trace::FStoreClient::FTraceData{};
	}

	static FString GetDefaultSaveDirectoryPath();
};
PRAGMA_ENABLE_DEPRECATION_WARNINGS

