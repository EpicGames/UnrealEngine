// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ChaosVDRecording.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "Containers/Array.h"
#include "Containers/Queue.h"
#include "Containers/UnrealString.h"
#include "Templates/SharedPointer.h"
#include "TraceServices/Model/AnalysisSession.h"

namespace Chaos
{
	class FChaosArchiveContext;
}

class FChaosVDDataProcessorBase;

namespace Chaos::VisualDebugger
{
	class FChaosVDSerializableNameTable;
}

class FChaosVDEngine;
class IChaosVDDataProcessor;

/** Set of flags that control how a solver stage is accessed */
enum class EChaosVDSolverStageAccessorFlags : uint8
{
	None = 0,
	/* If the solver frame has valid stage data but the last stage is closed, create a new stage which will be labeled as non-staged data */
	CreateNewIfClosed = 1 << 0,
	/* If the solver frame does not have any solver stage data, create a new stage which will be labeled as non-staged data */
	CreateNewIfEmpty = 1 << 1 
};
ENUM_CLASS_FLAGS(EChaosVDSolverStageAccessorFlags);

/** Provider class for Chaos VD trace recordings.
 * It stores and handles rebuilt recorded frame data from Trace events
 * dispatched by the Chaos VD Trace analyzer
 */
class FChaosVDTraceProvider : public TraceServices::IProvider, public TraceServices::IEditableProvider, public TSharedFromThis<FChaosVDTraceProvider>
{
public:
	
	CHAOSVD_API static FName ProviderName;

	CHAOSVD_API FChaosVDTraceProvider(TraceServices::IAnalysisSession& InSession);

	/** Creates a CVD recording instances where all the data loaded from the Trace Analisis session will be stored
	 * @param InSessionName Trace session name
	 */
	void CreateRecordingInstanceForSession(const FString& InSessionName);

	void SetExternalRecordingInstanceForSession(const TSharedRef<FChaosVDRecording>& InExternalCVDRecording);

	/** Opens a solver frame entry into the active CVD recording structure
	 * @param InSolverGUID ID of the solver
	 * @param FrameData Solver frame data
	 */
	void StartSolverFrame(const int32 InSolverGUID, FChaosVDSolverFrameData&& FrameData);

	/** Opens a game thread frame entry into the active CVD recording structure
	 * @param InFrameData Solver frame data
	 */
	void StartGameFrame(const TSharedPtr<FChaosVDGameFrameData>& InFrameData);

	/** Returns the current solver frame instance that is open and accepting data
	 * @param InSolverGUID ID of the solver
	 */
	CHAOSVD_API FChaosVDSolverFrameData* GetCurrentSolverFrame(const int32 InSolverGUID);

	/** Returns the game thread frame instance that is open and accepting data */
	CHAOSVD_API TWeakPtr<FChaosVDGameFrameData> GetCurrentGameFrame();

	/** Returns the active CVD recording instance */
	CHAOSVD_API TSharedPtr<FChaosVDRecording> GetRecordingForSession() const;

	/** Registers a CVD data processors. These are used to process raw serialized data from CVD's or custom user extensions
	 * @param InDataProcessor Shared ptr to the CVD Data processor we want to register
	 */
	CHAOSVD_API void RegisterDataProcessor(TSharedPtr<FChaosVDDataProcessorBase> InDataProcessor);

	/** Returns the current open solver stage data for the provided solver ID
	 * @param SolverID ID of the solver
	 * @param Flags Indicating what to do if no solver stage is available
	 */
	CHAOSVD_API FChaosVDFrameStageData* GetCurrentSolverStageDataForCurrentFrame(int32 SolverID, EChaosVDSolverStageAccessorFlags Flags);

private:

	struct FBinaryDataContainer
	{
		explicit FBinaryDataContainer(const int32 InDataID)
			: DataID(InDataID)
		{
		}

		int32 DataID;
		bool bIsReady = false;
		bool bIsCompressed = false;
		uint32 UncompressedSize = 0;
		FString TypeName;
		TArray<uint8> RawData;
	};
	
	TMap<int32,int32>& GetCurrentTickOffsetsBySolverID()
	{
		return CurrentNetworkTickOffsets;
	}
	
	void HandleAnalysisComplete();

	/** Shows a Slate dialog listing data types skipped because Safe Loading Mode was active. */
	void ShowSafeModeSkippedDialog();

	/** Shows a Slate dialog warning that non-backwards-compatible types were loaded with Safe Loading Mode disabled. */
	void ShowUnsafeDataDialog();

	/** Shows a Slate dialog warning that natively-serialized particle extra data entries are present,
	 *  and lets the user choose whether to display them or skip them. */
	void ShowNativeSerializationDialog();

	/** Shows a Slate dialog informing the user that natively-serialized particle extra data entries
	 *  were stripped because Particle Extra Data Loading Mode is set to SerializeBin Only. */
	void ShowSerializeBinOnlyStrippedDialog();

	FBinaryDataContainer& FindOrAddUnprocessedData(const int32 DataID);
	void RemoveUnprocessedData(const int32 DataID);

	bool ProcessBinaryData(const int32 DataID);

	void DeleteRecordingInstanceForSession();

	void RegisterDefaultDataProcessorsIfNeeded();

	void EnqueueGameFrameForProcessing(const TSharedPtr<FChaosVDGameFrameData>& FrameData);
	void DeQueueGameFrameForProcessing(TSharedPtr<FChaosVDGameFrameData>& OutFrameData);
	
	void CommitProcessedGameFramesToRecording();

	/** Gathers any solver id from solver data that is not fully processed yet but that will be valid for the provided game frame data later on */
	void GetAvailablePendingSolverIDsAtGameFrame(const TSharedRef<FChaosVDGameFrameData>& InProcessedGameFrameData, TArray<int32, TInlineAllocator<16>>& OutSolverIDs);

public:

	CHAOSVD_API FString GenerateFormattedStringListFromSet(const TSet<FString>& StringsSet) const;

	CHAOSVD_API int32 RemapSolverID(int32 SolverID);

	virtual void BeginEdit() const override{};
	virtual void EndEdit() const override{};
	virtual void EditAccessCheck() const override{};
	
	int32 CHAOSVD_API GetRemappedSolverID(int32 SolverID);

	bool DoesOwnRecordingInstance() const
	{
		return !bHasRecordingOverride;
	}

	/** Returns the name table instances used to de-duplicate strings serialization */
	TSharedPtr<Chaos::VisualDebugger::FChaosVDSerializableNameTable> GetNameTableInstance() const
	{
		return NameTable;
	}

	/** Returns the FArchive header used to read the serialized binary data */
	const Chaos::VisualDebugger::FChaosVDArchiveHeader& GetHeaderData() const
	{
		return HeaderData;
	}

	/** Sets the FArchive header used to read the serialized binary data */
	void SetHeaderData(Chaos::VisualDebugger::FChaosVDArchiveHeader& InNewHeader)
	{
		HeaderData = MoveTemp(InNewHeader);
	}

	/** Returns how many pieces of data we processed so far with a data processor (even if it failed) */
	int64 GetDataProcessedSoFarNum() const
	{
		return DataProcessedSoFarCounter;
	}

	const TSet<FString>& GetTypesFailedToSerialize() const
	{
		return TypesFailedToSerialize;
	}

	/** Types whose data was skipped entirely because Safe Loading Mode was enabled and they are not backwards compatible. */
	const TSet<FString>& GetTypesSkippedInSafeMode() const
	{
		return TypesSkippedInSafeMode;
	}

	/** Types that were processed despite not being backwards compatible (Safe Loading Mode was off). */
	const TSet<FString>& GetUnsafeTypesProcessed() const
	{
		return UnsafeTypesProcessed;
	}

	void AddParticleMetadata(uint64 MetadaId, const TSharedPtr<FChaosVDParticleMetadata>& InMetadata);
	TSharedPtr<FChaosVDParticleMetadata> GetParticleMetadata(uint64 MetadataId);

	void CreateGeometryArchiveContext();

	TSharedPtr<Chaos::FChaosArchiveContext> GetGeometryArchiveContext()
	{
		return GeometryArchiveContext;
	}

private:
	
	int32 GetCurrentGameThreadTrackID() const
	{
		return RemappedGameThreadTrackID;
	}
	
	void SetCurrentGameThreadTrackID(int32 NewID)
	{
		RemappedGameThreadTrackID = NewID;
	}
	
	Chaos::VisualDebugger::FChaosVDArchiveHeader HeaderData;
	
	TSharedPtr<Chaos::VisualDebugger::FChaosVDSerializableNameTable> NameTable;

	TraceServices::IAnalysisSession& Session;

	TSharedPtr<FChaosVDRecording> InternalRecording;

	TMap<int32, TSharedPtr<FBinaryDataContainer>> UnprocessedDataByID;

	TMap<FStringView, TSharedPtr<FChaosVDDataProcessorBase>> RegisteredDataProcessors;

	TMap<int32, FChaosVDSolverFrameData> CurrentSolverFramesByID;

	TQueue<TSharedPtr<FChaosVDGameFrameData>> CurrentGameFrameQueue;

	TWeakPtr<FChaosVDGameFrameData> CurrentGameFrame = nullptr;

	TUniquePtr<Chaos::VisualDebugger::FChaosVDArchiveHeader> CurrentHeaderData;

	int32 CurrentGameFrameQueueSize = 0;

	bool bDefaultDataProcessorsRegistered = false;

	double StartLastCommitedFrameTimeSeconds = 0.0;

	bool bHasRecordingOverride = false;

	TMap<int32, int32> CurrentNetworkTickOffsets;
	TSortedMap<int32, int32> RemappedSolversIDs;

	int32 RemappedGameThreadTrackID = INDEX_NONE;

	TSet<FString> MissingDataProcessors;
	TSet<FString> TypesFailedToSerialize;
	TSet<FString> TypesSkippedInSafeMode;
	TSet<FString> UnsafeTypesProcessed;

	int64 DataProcessedSoFarCounter = 0;

	bool bShouldTrimOutStartEmptyFrames = false;
	int32 MaxGameFramesToQueueNum = 10;

	TMap<uint64, TSharedPtr<FChaosVDParticleMetadata>> SerializedParticleMetadata;

	TSharedPtr<Chaos::FChaosArchiveContext> GeometryArchiveContext;

	friend class FChaosVDTraceAnalyzer;

	// Not idea, but needed to for the serialization fix up logic of the collision data channel info
	// An alternative is expose a plugin function deprecated from the get go
	friend class FChaosVDCollisionChannelsInfoDataProcessor;
};

