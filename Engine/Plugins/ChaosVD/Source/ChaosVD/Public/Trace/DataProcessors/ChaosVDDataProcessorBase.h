// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ImplicitObject.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "UObject/NameTypes.h"
#include "DataWrappers/ChaosVDImplicitObjectDataWrapper.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "Trace/ChaosVDTraceProvider.h"

#include <type_traits>

#include "ChaosVDRecording.h"

struct FFortniteSeasonBranchObjectVersion;

namespace Chaos::VisualDebugger
{
	template<typename TArchive>
	void ApplyHeaderDataToArchive(TArchive& InOutArchive, const FChaosVDArchiveHeader& InRecordedHeader)
	{
		InOutArchive.SetCustomVersions(InRecordedHeader.CustomVersionContainer);
		InOutArchive.SetEngineVer(InRecordedHeader.EngineVersion);
		InOutArchive.SetShouldSkipUpdateCustomVersion(true);
	}

	template<typename TDataToSerialize>
	bool ReadDataFromBuffer(const TArray<uint8>& InDataBuffer, TDataToSerialize& Data, const TSharedRef<FChaosVDTraceProvider>& DataProvider)
	{
		const TSharedPtr<FChaosVDSerializableNameTable> NameTableInstance = DataProvider->GetNameTableInstance();

		if (!ensure(NameTableInstance.IsValid()))
		{
			return false;
		}
		
		FChaosVDMemoryReader MemReader(InDataBuffer, NameTableInstance.ToSharedRef());
		const FChaosVDArchiveHeader& RecordedHeader = DataProvider->GetHeaderData();
		ApplyHeaderDataToArchive(MemReader, RecordedHeader);

		bool bSuccess = false;

		// We need to use FChaosArchive as proxy to properly read serialized Implicit objects
		// Note: I don't expect we will need a proxy archive for other types, but if we end up in that situation, we should use to switch to use traits 
		if constexpr (std::is_same_v<TDataToSerialize, FChaosVDImplicitObjectDataWrapper<Chaos::FImplicitObjectPtr, Chaos::FChaosArchive>>)
		{
			if (TSharedPtr<FChaosArchiveContext> ProvidedGeometryContext = DataProvider->GetGeometryArchiveContext())
			{
				FChaosVDSharedContextArchive Ar(MemReader, ProvidedGeometryContext.ToSharedRef());
				bSuccess = Data.Serialize(Ar);
			}
			else
			{
				FChaosArchive Ar(MemReader);
				bSuccess = Data.Serialize(Ar);
			}
		}
		else
		{
			bSuccess = Data.Serialize(MemReader);
		}

		return bSuccess;
	}
}

/** Abstract base class that should be used for any class that is able to process traced Chaos Visual Debugger binary data */
class FChaosVDDataProcessorBase
{
public:

	explicit FChaosVDDataProcessorBase(FStringView InCompatibleType) : TraceProvider(nullptr), CompatibleType(InCompatibleType), ProcessedBytes(0)
	{
	}

	CHAOSVD_API virtual ~FChaosVDDataProcessorBase() = 0;

	/** Type name this data processor can interpret */
	FStringView GetCompatibleTypeName() const;

	/** Called with the raw serialized data to be processed */
	CHAOSVD_API virtual bool ProcessRawData(const TArray<uint8>& InData);

	/** Returns the amount of data in bytes processed by this data processor at the moment of being called */
	uint64 GetProcessedBytes() const;

	/** Sets the Trace Provider that is storing the data being analyzed */
	CHAOSVD_API void SetTraceProvider(const TSharedPtr<FChaosVDTraceProvider>& InProvider);

	/** Returns natively-serialized struct types encountered during ProcessRawData, mapped to the
	 *  CVD optional data channel they were traced through (NAME_None if unknown).
	 *  Used by the post-load NativeSerialization warning dialog. */
	virtual void GetPostLoadNativeTypesWithChannels(TMap<FName, FName>& OutTypeToChannel) const {}


	/**
	 * Returns true if this processor's data format is backwards compatible, meaning it
	 * uses UE's property system or explicit version fields that allow graceful forward/backward reading.
	 *
	 * Processors that serialise raw binary struct layouts (no versioning) must set
	 * bBackwardsCompatible = false in their constructor. When Safe Loading Mode is enabled
	 * in CVD General Settings, the trace provider will skip those processors entirely rather than
	 * risk a crash on a format mismatch.
	 */
	virtual bool IsBackwardsCompatible() const { return bBackwardsCompatible; }

	/** When true, the provider skips this processor silently regardless of bSafeLoadingMode.
	 *  Use when the processor has its own loading-mode control (e.g. EChaosVDParticleExtraDataLoadingMode). */
	virtual bool ShouldAlwaysSkip() const { return false; }

protected:
	TWeakPtr<FChaosVDTraceProvider> TraceProvider;
	FStringView CompatibleType;
	uint64 ProcessedBytes;

	/**
	 * Set to false in the constructor of any derived processor whose serialised format is not
	 * guaranteed to be stable across engine versions or local source changes.
	 * Defaults to true so existing processors are unaffected until explicitly opted out.
	 */
	bool bBackwardsCompatible = true;
};

class FChaosVDGenericDataProcessor : FChaosVDDataProcessorBase
{
public:

	CHAOSVD_API explicit FChaosVDGenericDataProcessor(FStringView InCompatibleType, const TFunction<bool(const TArray<uint8>&)>& InProcessDataCallback);

	CHAOSVD_API virtual bool ProcessRawData(const TArray<uint8>& InData) override;

private:
	TFunction<bool(const TArray<uint8>&)> ProcessDataCallback;
};

