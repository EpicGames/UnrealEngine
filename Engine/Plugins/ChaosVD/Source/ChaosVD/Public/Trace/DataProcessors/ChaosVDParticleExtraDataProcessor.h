// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HAL/Platform.h"
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

enum class EChaosVDParticleExtraDataLoadingMode : uint8;

/**
 * Data processor that deserializes traced FChaosVDParticleExtraData payloads and stores
 * them in FChaosVDParticleExtraDataContainer inside the solver frame's custom data.
 */
class FChaosVDParticleExtraDataProcessor final : public FChaosVDDataProcessorBase
{
public:
	CHAOSVD_API explicit FChaosVDParticleExtraDataProcessor();
	CHAOSVD_API ~FChaosVDParticleExtraDataProcessor();

	CHAOSVD_API virtual bool ProcessRawData(const TArray<uint8>& InData) override;

	virtual bool ShouldAlwaysSkip() const override;
	virtual bool IsBackwardsCompatible() const override;
	virtual void GetPostLoadNativeTypesWithChannels(TMap<FName, FName>& OutTypeToChannel) const override;

private:
	void HandleSettingsChanged(UObject* InSettings);

	TMap<FName, FName> NativeTypesToChannel;
	EChaosVDParticleExtraDataLoadingMode CachedLoadingMode;
};
