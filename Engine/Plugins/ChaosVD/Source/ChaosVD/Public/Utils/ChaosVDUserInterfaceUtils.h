// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDSettingsManager.h"
#include "IStructureDetailsView.h"
#include "SEnumCombo.h"
#include "ToolMenu.h"
#include "ToolMenuSection.h"
#include "Widgets/SChaosVDEnumFlagsMenu.h"

class UToolMenu;

namespace Chaos::VisualDebugger::Utils
{
	CHAOSVD_API TSharedRef<IStructureDetailsView> MakeStructDetailsViewForMenu();

	CHAOSVD_API TSharedRef<IDetailsView> MakeObjectDetailsViewForMenu();

	template <typename EnumType>
	TSharedRef<SWidget> MakeEnumMenuEntryWidget(const FText& MenuEntryLabel, const SEnumComboBox::FOnEnumSelectionChanged&& EnumValueChanged, const TAttribute<int32>&& CurrentValueAttribute)
	{
		return SNew(SHorizontalBox)
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				.Padding(4.0f, 0.f)
				[
					SNew(STextBlock)
					.Text(MenuEntryLabel)
					.Font(FAppStyle::GetFontStyle(TEXT("MenuItem.Font")))
				]
				+SHorizontalBox::Slot()
				.AutoWidth()
				.VAlign(VAlign_Center)
				[
					SNew(SEnumComboBox, StaticEnum<EnumType>())
					.CurrentValue(CurrentValueAttribute)
					.OnEnumSelectionChanged(EnumValueChanged)
				];
	}

	enum class EChaosVDSaveSettingsOptions
	{
		None = 0,
		ShowSaveButton = 1 << 0,
		ShowResetButton = 1 << 1
	};
	ENUM_CLASS_FLAGS(EChaosVDSaveSettingsOptions)

	CHAOSVD_API void CreateMenuEntryForObject(UToolMenu* Menu, UObject* Object, EChaosVDSaveSettingsOptions MenuEntryOptions = EChaosVDSaveSettingsOptions::None);

	template <typename Object>
	void CreateMenuEntryForSettingsObject(UToolMenu* Menu, EChaosVDSaveSettingsOptions MenuEntryOptions = EChaosVDSaveSettingsOptions::None)
	{
		CreateMenuEntryForObject(Menu, FChaosVDSettingsManager::Get().GetSettingsObject<Object>(), MenuEntryOptions);
	}

	template <typename TStruct>
	void SetStructToDetailsView(TStruct* NewStruct, TSharedRef<IStructureDetailsView>& InDetailsView)
	{
		TSharedPtr<FStructOnScope> StructDataView = nullptr;

		if (NewStruct)
		{
			StructDataView = MakeShared<FStructOnScope>(TStruct::StaticStruct(), reinterpret_cast<uint8*>(NewStruct));
		}

		InDetailsView->SetStructureData(StructDataView);
	}

	template <typename ObjectSettingsType, typename VisualizationFlagsType>
	bool ShouldSettingsObjectVisFlagBeEnabledInUI(VisualizationFlagsType Flag)
	{
		if (ObjectSettingsType* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<ObjectSettingsType>())
		{
			return Settings->CanVisualizationFlagBeChangedByUI(static_cast<uint32>(Flag));
		}

		return true;
	}

	template <typename ObjectSettingsType, typename VisualizationFlagsType>
	void CreateVisualizationOptionsMenuSections(UToolMenu* Menu, FName SectionName, const FText& InSectionLabel, const FText& InFlagsMenuLabel,  const FText& InFlagsMenuTooltip, FSlateIcon FlagsMenuIcon,  const FText& InSettingsMenuLabel, const FText& InSettingsMenuTooltip)
	{
		FToolMenuSection& Section = Menu->AddSection(SectionName, InSectionLabel);
		
		Section.AddSubMenu(FName(InSectionLabel.ToString()), InFlagsMenuLabel, InFlagsMenuTooltip, FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
						   {
							   TSharedRef<SWidget> VisualizationFlagsWidget = SNew(SChaosVDEnumFlagsMenu<VisualizationFlagsType>)
								   .CurrentValue_Static(&ObjectSettingsType::GetDataVisualizationFlags)
								   .OnEnumSelectionChanged_Static(&ObjectSettingsType::SetDataVisualizationFlags)
									.IsFlagEnabled_Static(&ShouldSettingsObjectVisFlagBeEnabledInUI<ObjectSettingsType,VisualizationFlagsType>);
			
							   FToolMenuEntry FlagsMenuEntry = FToolMenuEntry::InitWidget("VisualizationFlags", VisualizationFlagsWidget,FText::GetEmpty());
							   Menu->AddMenuEntry(NAME_None, FlagsMenuEntry);
						   }),
						   false, FlagsMenuIcon);

		using namespace Chaos::VisualDebugger::Utils;
		Section.AddSubMenu(FName(InSettingsMenuLabel.ToString()), InSettingsMenuLabel, InSettingsMenuTooltip, FNewToolMenuDelegate::CreateStatic(&CreateMenuEntryForSettingsObject<ObjectSettingsType>, EChaosVDSaveSettingsOptions::ShowResetButton),
						   false, FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbar.Settings")));
	}

	/**
	 * Evaluates a flag and determines if it should be considered enabled in the UI,
	 * based on the provided current active flags and the general enable draw flags 
	 * @tparam VisFlagsType Type of the enum visualization flags
	 * @return true if the flag should be enabled in the UI
	 */
	template<typename VisFlagsType>
	bool ShouldVisFlagBeEnabledInUI(uint32 FlagToEvaluate, uint32 CurrentFlags, VisFlagsType EnableDrawFlags)
	{
		const VisFlagsType FlagToEvaluateAsEnum = static_cast<VisFlagsType>(FlagToEvaluate);
		const VisFlagsType CurrentFlagsAsEnum = static_cast<VisFlagsType>(CurrentFlags);

		// If the flags is any of the flags that will enable drawing, it needs to be always enabled
		if (EnumHasAnyFlags(EnableDrawFlags, FlagToEvaluateAsEnum))
		{
			return true;
		}

		return EnumHasAnyFlags(CurrentFlagsAsEnum, EnableDrawFlags);
	}
}
