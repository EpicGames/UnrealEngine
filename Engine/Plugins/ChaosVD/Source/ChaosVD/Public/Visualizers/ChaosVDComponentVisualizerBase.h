// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ComponentVisualizer.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"
#include "Widgets/SChaosVDEnumFlagsMenu.h"
#include "Widgets/SWidget.h"

#define UE_API CHAOSVD_API

class SChaosVDMainTab;
class FChaosVDSolverDataSelection;
struct FChaosVDSolverDataSelectionHandle;
class UChaosVDSettingsObjectBase;
class FChaosVDScene;

/** Context needed to be able to visualize data in the viewport */
struct FChaosVDVisualizationContext
{
	FTransform SpaceTransform;
	TWeakPtr<FChaosVDScene> CVDScene;
	int32 SolverID = INDEX_NONE;
	uint32 VisualizationFlags = 0;
	const UChaosVDSettingsObjectBase* DebugDrawSettings = nullptr;
	TSharedPtr<FChaosVDSolverDataSelection> SolverDataSelectionObject = nullptr;
};

/** Custom Hit Proxy for debug drawn particle data */
struct HChaosVDComponentVisProxy : public HComponentVisProxy
{
	DECLARE_HIT_PROXY( UE_API )

	HChaosVDComponentVisProxy(const UActorComponent* InComponent, const TSharedPtr<FChaosVDSolverDataSelectionHandle>& InDataSelectionHandle)
		: HComponentVisProxy(InComponent), DataSelectionHandle(InDataSelectionHandle)
	{
	}

	virtual EMouseCursor::Type GetMouseCursor() override
	{
		return EMouseCursor::Crosshairs;
	}

	TSharedPtr<FChaosVDSolverDataSelectionHandle> DataSelectionHandle;
};

/** Base class used for all component visualizers in CVD - It provides a common code to handle selection and clicks
 * @note Not all functionality of UE's component visualizer framework is supported in CVD
 */
class FChaosVDComponentVisualizerBase : public FComponentVisualizer
{
public:

	/** Handles a click to any CVD componentvisualization hit proxy
	 * @param InViewportClient Viewport Client where the click happened
	 * @param VisProxy Hit proxy clicked
	 * @param Click Click data
	 */
	CHAOSVD_API virtual bool VisProxyHandleClick(FEditorViewportClient* InViewportClient, HComponentVisProxy* VisProxy, const FViewportClick& Click) override;

protected:

	/** Called when the menu for this visualizer (if any) can be registered. And does so if implemented.
	 * This needs to be manually called in the constructor of ech visualizer
	 */
	virtual void RegisterVisualizerMenus() = 0;

	/** Returns true if this visualizer can handle a click in the viewport
	 */
	CHAOSVD_API virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy);

	/** Selects the visualized data referenced by the provided hitproxy
	 * @param VisProxy HitProxy referencing the data to select
	 * @param InCVDScene RefPtr to CVD's Main Scene
	 * @param InMainTabToolkitHost RefPtr to CVD's Main tab (which is also the Editor Toolkit Host)
	 * */
	CHAOSVD_API virtual bool SelectVisualizedData(const HChaosVDComponentVisProxy& VisProxy, const TSharedRef<FChaosVDScene>& InCVDScene,const TSharedRef<SChaosVDMainTab>& InMainTabToolkitHost);

	/** Creates a menu entry for this visualizer data, as long it is using the supporting settings format and types for visualizers
	 * @param MenuToExtend Name of the menu will be extension
	 * @param SectionName Name of the section for this menu
	 * @param InFlagsMenuLabel Label to use for the submenu created to the visualization flags of this visualizer 
	 * @param InFlagsMenuTooltip Tooltip to use for the submenu created to the visualization flags of this visualizer
	 * @param FlagsMenuIcon Icon to use for the submenu created to the visualization flags of this visualizer
	 * @param InSettingsMenuLabel Label to use for the submenu created to the visualization settings of this visualizer
	 * @param InSettingsMenuTooltip Tooltip to use for the submenu created to the visualization settings of this visualizer
	 */
	template<typename ObjectSettingsType, typename VisualizationFlagsType>
	void CreateGenericVisualizerMenu(FName MenuToExtend, FName SectionName, const FText& InSectionLabel, const FText& InFlagsMenuLabel,  const FText& InFlagsMenuTooltip, FSlateIcon FlagsMenuIcon,  const FText& InSettingsMenuLabel, const FText& InSettingsMenuTooltip);

	FName InspectorTabID = NAME_None;
};

template <typename ObjectSettingsType, typename VisualizationFlagsType>
void FChaosVDComponentVisualizerBase::CreateGenericVisualizerMenu(FName MenuToExtend, FName SectionName, const FText& InSectionLabel, const FText& InFlagsMenuLabel,  const FText& InFlagsMenuTooltip, FSlateIcon FlagsMenuIcon,  const FText& InSettingsMenuLabel, const FText& InSettingsMenuTooltip)
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	
	if (!ensure(ToolMenus))
	{
		return;
	}

	if (UToolMenu* Menu = ToolMenus->ExtendMenu(MenuToExtend))
	{
		using namespace Chaos::VisualDebugger::Utils;
		CreateVisualizationOptionsMenuSections<ObjectSettingsType, VisualizationFlagsType>(Menu, SectionName, InSectionLabel, InFlagsMenuLabel, InFlagsMenuTooltip, FlagsMenuIcon, InSettingsMenuLabel, InSettingsMenuTooltip);
	}
}

#undef UE_API
