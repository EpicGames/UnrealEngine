// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Chaos/ImplicitFwd.h"
#include "Containers/Queue.h"
#include "SceneManagement.h"

struct FChaosVDGameFrameData;
struct FChaosVDDebugShapeDataContainer;
class FChaosVDGeometryBuilder;
class FChaosVDPlaybackViewportClient;

struct FChaosVDQueryDataWrapper;
struct FChaosVDRecording;

enum class EChaosVDDebugDrawTextLocationMode
{
	World,
	Screen
};

/** Utility methods that allows Debug draw into the Chaos VD Editor */
class FChaosVDDebugDrawUtils
{
public:
	//TODO: Create a common args struct that can be passed as parameter to all debug draw methods

	CHAOSVD_API static void DrawArrowVector(FPrimitiveDrawInterface* PDI, const FVector& StartLocation, const FVector& EndLocation, const FText& InDebugText, const FColor& Color, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 0.0f);
	CHAOSVD_API static void DrawPoint(FPrimitiveDrawInterface* PDI, const FVector& Location, const FText& InDebugText, const FColor& Color, float Size = 10.0f, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	CHAOSVD_API static void DrawString(FStringView StringToDraw, const FVector& Location, const FColor& Color, EChaosVDDebugDrawTextLocationMode LocationMode = EChaosVDDebugDrawTextLocationMode::World);
	CHAOSVD_API static void DrawText(const FText& InText, const FVector& Location, const FColor& Color, EChaosVDDebugDrawTextLocationMode LocationMode = EChaosVDDebugDrawTextLocationMode::World);
	CHAOSVD_API static void DrawOnScreenWarning(const FText& InText, const FColor& Color);
	CHAOSVD_API static void DrawCircle(FPrimitiveDrawInterface* PDI, const FVector& Origin, float Radius, int32 Segments, const FColor& Color, float Thickness, const FVector& XAxis, const FVector& YAxis, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority = SDPG_World);
	CHAOSVD_API static void DrawBox(FPrimitiveDrawInterface* PDI, const FVector& InExtents, const FColor& InColor, const FTransform& InTransform, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 2.0f);
	CHAOSVD_API static void DrawLine(FPrimitiveDrawInterface* PDI, const FVector& InStartPosition, const FVector& InEndPosition, const FColor& InColor, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 2.0f);
	CHAOSVD_API static void DrawImplicitObject(FPrimitiveDrawInterface* PDI, const TSharedPtr<FChaosVDGeometryBuilder>& GeometryGenerator, const Chaos::FConstImplicitObjectPtr& ImplicitObject, const FTransform& InWorldTransform, const FColor& InColor, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 2.0f);
	
	CHAOSVD_API static void DrawSphere(FPrimitiveDrawInterface* PDI, const FVector& Center, float Radius, int32 Segments, const FColor& InColor, const FText& InDebugText, ESceneDepthPriorityGroup DepthPriority = SDPG_World, float Thickness = 2.0f);

	CHAOSVD_API static void DrawCanvas(FViewport& InViewport, FSceneView& View, FCanvas& Canvas);
	
	CHAOSVD_API static bool CanDebugDraw();
	
	static int32 MaxLinesToDrawPerFrame;

private:
	struct FChaosVDQueuedTextToDraw
	{
		FText Text;
		EChaosVDDebugDrawTextLocationMode LocationMode;
		FVector Location;
		FLinearColor Color;
	};

	static void DebugDrawFrameEnd();
	static void IncreaseDebugDrawLineCounter();

	static TQueue<FChaosVDQueuedTextToDraw> TextToDrawQueue;

	static int32 CurrentLinesDrawn;
	static bool bIsShowingDebugDrawLimitWarning;
	static int32 CurrentWarningsBeingDrawn;

	friend FChaosVDPlaybackViewportClient;
};

namespace Chaos::VisualDebugger::Utils
{
	FString GenerateDebugTextForVector(const FVector& InVector, const FString& VectorName, const FString& InVectorUnits);

	FBox CalculateSceneQueryShapeBounds(const TSharedRef<FChaosVDQueryDataWrapper>& InSceneQueryData, const TSharedRef<FChaosVDRecording> InRecordedData);
}
