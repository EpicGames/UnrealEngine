// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Visualizers/ChaosVDComponentVisualizerBase.h"
#include "ComponentVisualizer.h"
#include "HAL/Platform.h"
#include "Templates/SharedPointer.h"

struct FChaosVDSceneParticle;
class FChaosVDGeometryBuilder;
class UChaosVDParticleVisualizationDebugDrawSettings;

struct FChaosParticleDataDebugDrawSettings;
struct FChaosVDParticleDataWrapper;

enum class EChaosVDParticleDataVisualizationFlags : uint32;

namespace Chaos
{
	struct FTrimeshBVH;
	class FTriangleMeshImplicitObject;
	class FAABBVectorized;
} // namespace Chaos


struct FChaosVDParticleDataVisualizationContext : public FChaosVDVisualizationContext
{
	TWeakPtr<FChaosVDGeometryBuilder> GeometryGenerator = nullptr;
	bool bIsSelectedData = false;
	bool bShowDebugText = false;

	const UChaosVDParticleVisualizationDebugDrawSettings* DebugDrawSettings = nullptr;

	bool IsVisualizationFlagEnabled(EChaosVDParticleDataVisualizationFlags Flag) const;
};

/**
 * Component visualizer in charge of generating debug draw visualizations for for particles
 */
class FChaosVDParticleDataComponentVisualizer : public FChaosVDComponentVisualizerBase
{
public:
	FChaosVDParticleDataComponentVisualizer();

	virtual void RegisterVisualizerMenus() override;

	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;

	virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy) override;

	virtual bool SelectVisualizedData(const HChaosVDComponentVisProxy& VisProxy, const TSharedRef<FChaosVDScene>& InCVDScene, const TSharedRef<SChaosVDMainTab>& InMainTabToolkitHost) override;

protected:
	
	void DrawParticleVector(FPrimitiveDrawInterface* PDI, const FVector& StartLocation, const FVector& InVector, EChaosVDParticleDataVisualizationFlags VectorID, const FChaosVDParticleDataVisualizationContext& InVisualizationContext, float LineThickness);
	void DrawVisualizationForParticleData(const UActorComponent* Component, FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDParticleDataVisualizationContext& InVisualizationContext, const FChaosVDSceneParticle& InParticleInstance);
};

class FChaosVDTrimeshBVHVisualizer
{
public:
	void Draw(FPrimitiveDrawInterface* PDI, const FChaosVDParticleDataVisualizationContext& InVisualizationContext, const FTransform& LocalToWorldTransform, const Chaos::FTriangleMeshImplicitObject& TriMesh) const;
private:
	void DrawBVH(FPrimitiveDrawInterface* PDI, const FTransform& LocalToWorldTransform, const Chaos::FTrimeshBVH& BVH, const int32 TargetLevel) const;
	void DrawBVHLevel(FPrimitiveDrawInterface* PDI, const FTransform& LocalToWorldTransform, const Chaos::FTrimeshBVH& BVH, const int32 NodeIndex, const int32 CurrentLevel, const int32 TargetLevel) const;
	void DrawAabb(FPrimitiveDrawInterface* PDI, const FTransform& LocalToWorldTransform, const Chaos::FAABB3& Aabb, const int32 ColorSeed) const;
	Chaos::FAABB3 ToAabb(const Chaos::FAABBVectorized& AabbVectorized) const;
};
