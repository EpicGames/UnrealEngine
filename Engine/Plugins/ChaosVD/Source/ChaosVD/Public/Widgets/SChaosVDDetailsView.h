// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IStructureDetailsView.h"
#include "Templates/SharedPointer.h"
#include "UObject/StructOnScope.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/SCompoundWidget.h"

class SChaosVDMainTab;
class IStructureDetailsView;
class FSubobjectEditorTreeNode;
class AActor;
class SBox;
class SSplitter;
class IDetailsView;
class SSubobjectEditor;
class UObject;

/**
 * Simple details for CVD objects and structures
 */
class SChaosVDDetailsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChaosVDDetailsView)
		{
		}
	SLATE_END_ARGS()

	/** Constructs this widget with InArgs */
	void Construct(const FArguments& InArgs, const TSharedRef<SChaosVDMainTab>& InMainTab);

	/** Updates the current object this view details is viewing */
	CHAOSVD_API void SetSelectedObject(UObject* NewObject);

	/** Updates the current object this view details is viewing */
	template<typename TStruct>
	void SetSelectedStruct(TStruct* NewStruct);

	CHAOSVD_API void SetSelectedStruct(const TSharedPtr<FStructOnScope>& NewStruct);

protected:

	TSharedPtr<IDetailsView> CreateObjectDetailsView();
	TSharedPtr<IStructureDetailsView> CreateStructureDataDetailsView() const;

	EVisibility GetStructDetailsVisibility() const;
	EVisibility GetObjectDetailsVisibility() const;

	TWeakObjectPtr<UObject> CurrentObjectInView;

	TWeakPtr<FStructOnScope> CurrentStructInView;
	
	TSharedPtr<IDetailsView> DetailsView;

	TSharedPtr<IStructureDetailsView> StructDetailsView;

	TWeakPtr<SChaosVDMainTab> MainTabWeakPtr;
};

template <typename TStruct>
void SChaosVDDetailsView::SetSelectedStruct(TStruct* NewStruct)
{
	TSharedPtr<FStructOnScope> StructDataView = nullptr;

	if (NewStruct)
	{
		StructDataView = MakeShared<FStructOnScope>(TStruct::StaticStruct(), reinterpret_cast<uint8*>(NewStruct));
	}

	SetSelectedStruct(StructDataView);
}
