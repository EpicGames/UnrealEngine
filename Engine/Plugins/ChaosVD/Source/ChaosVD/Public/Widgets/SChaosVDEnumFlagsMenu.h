// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/Commands/UIAction.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"


/**
 * Simple widget that allows that created a checkbox style menu for enum flags
 */
template<typename TEnum>
class SChaosVDEnumFlagsMenu : public SCompoundWidget
{
	
public:
	DECLARE_DELEGATE_OneParam(FOnEnumSelectionChanged, TEnum);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FIsEnumValueEnabled, TEnum);
	struct FEnumInfo
	{
		explicit FEnumInfo(const int32 InIndex, const TEnum InValue, const FText& InDisplayName, const FText& InTooltipText)
			: Index(InIndex), Value(InValue), DisplayName(InDisplayName), TooltipText(InTooltipText)
		{}
		
		int32 Index = 0;
		TEnum Value;
		FText DisplayName;
		FText TooltipText;
	};
	
	SLATE_BEGIN_ARGS(SChaosVDEnumFlagsMenu)
		: _CurrentValue()
	{}

	SLATE_ATTRIBUTE(TEnum, CurrentValue)
	SLATE_EVENT(FOnEnumSelectionChanged, OnEnumSelectionChanged)
	SLATE_EVENT(FIsEnumValueEnabled, IsFlagEnabled)

		
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs);

	void UpdateEnumFlagValue(FEnumInfo FlagInfo);
	bool IsEnumFlagSet(FEnumInfo FlagInfo) const;

	bool IsEnumFlagEnabled(TEnum FlagValue) const;

	TAttribute<TEnum> CurrentValue;
	
	FOnEnumSelectionChanged OnEnumSelectionChangedDelegate;
	FIsEnumValueEnabled EnumValueEnabledDelegate;
	
	const UEnum* Enum = nullptr;
};

namespace Chaos::VisualDebugger::Utils
{
	template<typename Enum>
	void EnumAddToggleFlag(Enum& Flags, Enum Flag)
	{
		using UnderlyingType = __underlying_type(Enum);
		Flags = (Enum)((UnderlyingType)Flags ^ (UnderlyingType)Flag);
	}
}

template <typename TEnum>
void SChaosVDEnumFlagsMenu<TEnum>::Construct(const FArguments& InArgs)
{
	Enum = StaticEnum<TEnum>();

	CurrentValue = InArgs._CurrentValue;
	OnEnumSelectionChangedDelegate = InArgs._OnEnumSelectionChanged;
	EnumValueEnabledDelegate = InArgs._IsFlagEnabled;

	using namespace Chaos::VisualDebugger;
	
	static const FName UseEnumValuesAsMaskValuesInEditorName(TEXT("UseEnumValuesAsMaskValuesInEditor"));

	const bool bUseEnumValuesAsMaskValues = Enum ? Enum->GetBoolMetaData(UseEnumValuesAsMaskValuesInEditorName) : false;

	if (!ensure(bUseEnumValuesAsMaskValues))
	{
		this->ChildSlot
		[
			SNew(SVerticalBox)
			+SVerticalBox::Slot()
			[
				SNew(STextBlock)
				.Text(NSLOCTEXT("ChaosVisualDebugger", "ChaosVDEnumFlagsMenuErrorMessage", "Incompatible enum. Make sure to add the meta tag UseEnumValuesAsMaskValuesInEditor and it is a valid UEnum"))
			]
		];
		return;
	}

	constexpr bool bCloseAfterSelection = false;
	constexpr bool bCloseSelfOnly = true;

	FMenuBuilder MenuBuilder(bCloseAfterSelection, nullptr, nullptr, bCloseSelfOnly);

	const int32 EnumValuesCount = Enum->NumEnums() - 1;

	for (int32 EnumValueIndex = 0; EnumValueIndex < EnumValuesCount ; EnumValueIndex++)
	{
		const int64 Value =Enum->GetValueByIndex(EnumValueIndex);
		const int32 Index = EnumValueIndex;
		const bool bIsHidden = Enum->HasMetaData(TEXT("Hidden"), Index);
		if (Value >= 0 && !bIsHidden)
		{
			if (!FMath::IsPowerOfTwo(Value))
			{
				continue;
			}
	
			FText DisplayName = Enum->GetDisplayNameTextByIndex(Index);
			FText TooltipText = Enum->GetToolTipTextByIndex(Index);
			if (TooltipText.IsEmpty())
			{
				TooltipText = FText::Format(NSLOCTEXT("ChaosVisualDebugger", "BitmaskDefaultFlagToolTipText", "Toggle {0} on/off"), DisplayName);
			}

			FEnumInfo EnumEntryInfo(Index, static_cast<TEnum>(Value), DisplayName, TooltipText);

			MenuBuilder.AddMenuEntry(
				DisplayName,
				EnumEntryInfo.TooltipText,
				FSlateIcon(),
				FUIAction(
				FExecuteAction::CreateSP(this, &SChaosVDEnumFlagsMenu::UpdateEnumFlagValue, EnumEntryInfo),
				FCanExecuteAction::CreateSP(this, &SChaosVDEnumFlagsMenu::IsEnumFlagEnabled, EnumEntryInfo.Value),
				FIsActionChecked::CreateSP(this, &SChaosVDEnumFlagsMenu::IsEnumFlagSet, EnumEntryInfo)),
				NAME_None, EUserInterfaceActionType::ToggleButton);
		}
	}

	this->ChildSlot
	[
		MenuBuilder.MakeWidget()
	];
}

template <typename TEnum>
void SChaosVDEnumFlagsMenu<TEnum>::UpdateEnumFlagValue(FEnumInfo FlagInfo)
{
	// Toggle value
	TEnum CurrentValueCopy = CurrentValue.Get();
	Chaos::VisualDebugger::Utils::EnumAddToggleFlag(CurrentValueCopy, FlagInfo.Value);
	OnEnumSelectionChangedDelegate.ExecuteIfBound(CurrentValueCopy);
}

template <typename TEnum>
bool SChaosVDEnumFlagsMenu<TEnum>::IsEnumFlagSet(FEnumInfo FlagInfo) const
{
	return EnumHasAnyFlags(CurrentValue.Get(), FlagInfo.Value);
}

template <typename TEnum>
bool SChaosVDEnumFlagsMenu<TEnum>::IsEnumFlagEnabled(TEnum FlagValue) const
{
	return EnumValueEnabledDelegate.IsBound() ? EnumValueEnabledDelegate.Execute(FlagValue) : true;
}
