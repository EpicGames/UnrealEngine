// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/SlateDelegates.h"
#include "Framework/Docking/TabManager.h"
#include "Templates/SharedPointer.h"
#include "Toolkits/IToolkitHost.h"
#include "Widgets/SCompoundWidget.h"
#include "ToolMenus.h"

#include "SChaosVDMainTab.generated.h"

class FChaosVDExtension;
class IDetailsView;
class IStructureDetailsView;
class FComponentVisualizer;
class FChaosVDEditorModeTools;
class FChaosVDTabSpawnerBase;
class FChaosVDEditorVisualizationSettingsTab;
class FChaosVDSolversTracksTab;
class FChaosVDEngine;
class FChaosVDIndependentDetailsPanelManager;
class FChaosVDOutputLogTab;
class FChaosVDPlaybackViewportTab;
class FChaosVDObjectDetailsTab;
class FChaosVDStandAloneObjectDetailsTab;
class FChaosVDWorldOutlinerTab;
class FStructOnScope;
class FUICommandList;
class SButton;
class FChaosVDRecordingControls;
class SDockTab;
class FChaosVDScene;

struct FDetailsViewArgs;
struct FStructureDetailsViewArgs;

enum class EChaosVDLoadRecordedDataMode : uint8;

UCLASS(MinimalAPI)
class UChaosVDMainToolbarMenuContext : public UObject
{
	GENERATED_BODY()

public:

	TWeakPtr<class SChaosVDMainTab> MainTab;
};

/** The main widget containing the Chaos Visual Debugger interface */
class SChaosVDMainTab : public SCompoundWidget, public IToolkitHost
{
public:
	
	SLATE_BEGIN_ARGS(SChaosVDMainTab) {}
		SLATE_ARGUMENT(TSharedPtr<SDockTab>, OwnerTab)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FChaosVDEngine> InChaosVDEngine);

	virtual ~SChaosVDMainTab() override;
	
	void BindUICommands(const TSharedRef<FUICommandList>& InGlobalUICommandsRef);

	TSharedRef<FChaosVDEngine> GetChaosVDEngineInstance() const
	{
		return ChaosVDEngine.ToSharedRef();
	}

	CHAOSVD_API TSharedPtr<FChaosVDScene> GetScene();

	template<typename TabType>
	TWeakPtr<TabType> GetTabSpawnerInstance(FName TabID);

	// BEGIN ITOOLKITHOST Interface
	virtual TSharedRef<SWidget> GetParentWidget() override
	{
		return AsShared();
	}

	virtual void BringToFront() override;
	virtual TSharedPtr<FTabManager> GetTabManager() const override
	{
		return TabManager;
	};
	virtual void OnToolkitHostingStarted(const TSharedRef<IToolkit>& Toolkit) override;
	virtual void OnToolkitHostingFinished(const TSharedRef<IToolkit>& Toolkit) override;
	virtual UWorld* GetWorld() const override;
	virtual FEditorModeTools& GetEditorModeManager() const override;

	virtual UTypedElementCommonActions* GetCommonActions() const override
	{
		return nullptr;
	}
	virtual FName GetStatusBarName() const override { return StatusBarID; };
	virtual FOnActiveViewportChanged& OnActiveViewportChanged() override
	{
		return ViewportChangedDelegate;
	};
	// END ITOOLKITHOST Interface

	CHAOSVD_API TSharedPtr<FComponentVisualizer> FindComponentVisualizer(UClass* ClassPtr);
	CHAOSVD_API TSharedPtr<FComponentVisualizer> FindComponentVisualizer(FName ClassName);

	TConstArrayView<TSharedPtr<FComponentVisualizer>> GetAllComponentVisualizers()
	{
		return ComponentVisualizers;
	}

	/** Connects to a live session using via the Unreal Trace Server and its Trace Store. This is a legacy option */
	bool ConnectToLiveSession(int32 SessionID, const FString& InSessionAddress, EChaosVDLoadRecordedDataMode LoadingMode) const;

	UE_DEPRECATED(5.8, "Use the version that takes a Remote session ID")
	bool ConnectToLiveSession_Direct(EChaosVDLoadRecordedDataMode LoadingMode) const;

	bool ConnectToLiveSession_Direct(FGuid RemoteSessionID, EChaosVDLoadRecordedDataMode LoadingMode) const;

	bool ConnectToLiveSession_Relay(FGuid RemoteSessionID, EChaosVDLoadRecordedDataMode LoadingMode) const;

	/**
	 * Evaluates a filename to determine if it is a supported filetype
	 * @param InFilename Filename to evaluate
	 * @return true if the file is supported
	 */
	bool IsSupportedFile(const FString& InFilename);

	/**
	 * Load the provided file into the current CVD instance
	 * @param InFilename Filename with path to load
	 * @param LoadingMode Desired loading mode (Single source or multi source)
	 */
	void LoadCVDFile(const FString& InFilename, EChaosVDLoadRecordedDataMode LoadingMode);
	
	/**
	 * Load the provided files into the current CVD instance
	 * @param InFilenames Array view of Filename with paths to load
	 * @param LoadingMode Desired loading mode (Single source or multi source)
	 */
	void LoadCVDFiles(TConstArrayView<FString> InFilenames, EChaosVDLoadRecordedDataMode LoadingMode);

	TSharedRef<IDetailsView> CreateDetailsView(const FDetailsViewArgs& InDetailsViewArgs);

	TSharedRef<IStructureDetailsView> CreateStructureDetailsView(const FDetailsViewArgs& InDetailsViewArgs, const FStructureDetailsViewArgs& InStructureDetailsViewArgs, const TSharedPtr<FStructOnScope>& InStructData = nullptr, const FText& CustomName = FText::GetEmpty());
	void ProccessKeyEventForPlaybackTrackSelector(const FKeyEvent& InKeyEvent);

	TSharedPtr<FUICommandList> GetGlobalUICommandList() const
	{
		return GlobalCommandList;
	}

	virtual FReply OnKeyDown(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;
	virtual FReply OnKeyUp(const FGeometry& MyGeometry, const FKeyEvent& InKeyEvent) override;

	virtual bool SupportsKeyboardFocus() const override
	{
		return true;
	}

	bool ShouldShowTracksKeyShortcuts();

	template<typename TabType>
	void RegisterTabSpawner(FName TabID);

	CHAOSVD_API void RegisterComponentVisualizer(FName ClassName, const TSharedPtr<FComponentVisualizer>& Visualizer);

	const TSharedPtr<FChaosVDIndependentDetailsPanelManager>& GetIndependentDetailsPanelManager();

	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;

	TSharedPtr<FChaosVDRecordingControls> GetRecordingControls()
	{
		return RecordingControls;
	}

	CHAOSVD_API virtual void AddViewportOverlayWidget(TSharedRef<SWidget> InOverlaidWidget, int32 ZOrder, TSharedPtr<IAssetViewport> InViewport = nullptr) override;

private:

	void ExtendMainToolbar();
	void ExtendStatusToolbar();

	TSharedPtr<SNotificationItem> CreateProcessingTraceDataNotification();

	void UpdateTraceProcessingNotificationUpdate();
	void ShowTraceDataProcessingNotification();
	void HideTraceDataProcessingNotification();

	void HandlePersistLayout(const TSharedRef<FTabManager::FLayout>& InLayoutToSave);

	void HandlePostInitializationExtensionRegistered(const TSharedRef<FChaosVDExtension>& NewExtension);

	void SetCustomPropertyLayouts(IDetailsView* DetailsView);

	void SetUpDisableCPUThrottlingDelegate();
	void CleanUpDisableCPUThrottlingDelegate() const;

	void CreateMainToolbarMenu();
	void AddSettingsMenu();

	CHAOSVD_API void HandleTabSpawned(TSharedRef<SDockTab> Tab, FName TabID);
	CHAOSVD_API void HandleTabDestroyed(TSharedRef<SDockTab> Tab, FName TabID);

	TSharedRef<FTabManager::FLayout> GenerateDefaultLayout();

	void ResetLayout();

	void CombineOpenSessions();

	void GenerateMainWindowMenu();
	void GenerateRecentFilesMenu(FMenuBuilder& MenuBuilder, UToolMenu* Menu);

	FReply BrowseAndOpenChaosVDRecording();

	TSharedRef<SButton> CreateSimpleButton(TFunction<FText()>&& GetTextDelegate, TFunction<FText()>&& ToolTipTextDelegate, const FSlateBrush* ButtonIcon, const UChaosVDMainToolbarMenuContext* MenuContext, const FOnClicked& InButtonClickedCallback);

	TSharedRef<SWidget> GenerateMainToolbarWidget();

	void BrowseChaosVDRecordingFromFolder(FStringView FolderPath, EChaosVDLoadRecordedDataMode LoadingMode);

	void BrowseLiveSessionsFromTraceStore() const;

	bool ShouldDisableCPUThrottling() const;

	TSharedPtr<SNotificationItem> ActiveProcessingTraceDataNotification;

	bool bCanTabManagerPersistLayout = true;
	
	TArray<TWeakPtr<IDetailsView>> CustomizedDetailsPanels;

	TSharedPtr<FChaosVDEngine> ChaosVDEngine;

	FName StatusBarID;

	TSharedPtr<FTabManager> TabManager;
	TWeakPtr<SDockTab> OwnerTab;
	TSharedPtr<FChaosVDEditorModeTools> EditorModeTools;
	
	TMap<FName, TSharedPtr<FChaosVDTabSpawnerBase>> TabSpawnersByIDMap;

	TMap<FName, TSharedPtr<FComponentVisualizer>> ComponentVisualizersMap;
	TArray<TSharedPtr<FComponentVisualizer>> ComponentVisualizers;

	TMap<FName, TWeakPtr<SDockTab>> ActiveTabsByID;

	FOnActiveViewportChanged ViewportChangedDelegate;

	FDelegateHandle DisableCPUThrottleHandle;

	FReply HandleSessionConnectionClicked();
	void HandleDisconnectSessionClicked();
	FText GetDisconnectButtonText() const;

	bool bShowTrackSelectorKeyShortcut = false;

	TSharedPtr<FChaosVDIndependentDetailsPanelManager> IndependentDetailsPanelManager;

	/** Commandlist used for any UI actions that need to be processed globally regardless on which specific widget we are in */
	TSharedPtr<FUICommandList> GlobalCommandList;

	TSharedPtr<FChaosVDRecordingControls> RecordingControls;

	static const FName MainToolBarName;

	static const FName MainMenuName;
};

template <typename TabType>
TWeakPtr<TabType> SChaosVDMainTab::GetTabSpawnerInstance(FName TabID)
{
	if (TSharedPtr<FChaosVDTabSpawnerBase>* TabSpawnerPtrPtr = TabSpawnersByIDMap.Find(TabID))
	{
		return StaticCastSharedPtr<TabType>(*TabSpawnerPtrPtr);
	}

	return nullptr;
}

template <typename TabType>
void SChaosVDMainTab::RegisterTabSpawner(FName TabID)
{
	static_assert(std::is_base_of_v<FChaosVDTabSpawnerBase, TabType> , "SChaosVDMainTab::RegisterTabSpawner Only supports FChaosVDTabSpawnerBase based spawners");

	if (!TabSpawnersByIDMap.Contains(TabID))
	{
		TSharedPtr<TabType> TabSpawner = MakeShared<TabType>(TabID, TabManager, StaticCastWeakPtr<SChaosVDMainTab>(AsWeak()));
		TabSpawner->OnTabSpawned().AddRaw(this, &SChaosVDMainTab::HandleTabSpawned, TabID);
		TabSpawner->OnTabDestroyed().AddRaw(this, &SChaosVDMainTab::HandleTabDestroyed, TabID);
		TabSpawnersByIDMap.Add(TabID, TabSpawner);
	}
}
