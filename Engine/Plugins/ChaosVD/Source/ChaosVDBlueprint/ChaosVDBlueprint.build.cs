// Copyright Epic Games, Inc. All Rights Reserved.

using System.Linq;
using System.Collections.Generic;

namespace UnrealBuildTool.Rules
{
	public class ChaosVDBlueprint : ModuleRules
	{
		public ChaosVDBlueprint(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[] {
					"Core",
					"CoreUObject",
					"Chaos",
					"Engine"
				}
			);
		}
	}
}