// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDRuntimeBlueprintLibrary.h"
#include "Engine/World.h"

#include "Physics/Experimental/PhysScene_Chaos.h"
#include "Chaos/Framework/PhysicsSolverBase.h"
#include "PBDRigidsSolver.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDRuntimeBlueprintLibrary)

IMPLEMENT_MODULE(FDefaultModuleImpl, ChaosVDBlueprint);

void UChaosVDRuntimeBlueprintLibrary::RecordDebugDrawBox(const UObject* WorldContext, const FBox& InBox, FName Tag, FLinearColor Color)
{
	CVD_TRACE_DEBUG_DRAW_BOX(InBox, Tag, Color.ToFColorSRGB(), WorldContext ? CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(WorldContext->GetWorld()) : INDEX_NONE);
}

void UChaosVDRuntimeBlueprintLibrary::RecordDebugDrawLine(const UObject* WorldContext, const FVector& InStartLocation, const FVector& InEndLocation, FName Tag, FLinearColor Color)
{
	CVD_TRACE_DEBUG_DRAW_LINE(InStartLocation, InEndLocation, Tag, Color.ToFColorSRGB(), WorldContext ? CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(WorldContext->GetWorld()) : INDEX_NONE);
}

void UChaosVDRuntimeBlueprintLibrary::RecordDebugDrawVector(const UObject* WorldContext, const FVector& InStartLocation, const FVector& InVector, FName Tag, FLinearColor Color)
{
	CVD_TRACE_DEBUG_DRAW_VECTOR(InStartLocation, InVector, Tag, Color.ToFColorSRGB(), WorldContext ? CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(WorldContext->GetWorld()) : INDEX_NONE);
}

void UChaosVDRuntimeBlueprintLibrary::RecordDebugDrawSphere(const UObject* WorldContext, const FVector& InCenter, float Radius, FName Tag, FLinearColor Color)
{
	CVD_TRACE_DEBUG_DRAW_SPHERE(InCenter, Radius, Tag, Color.ToFColorSRGB(), WorldContext ? CVD_TRACE_GET_SOLVER_ID_FROM_WORLD(WorldContext->GetWorld()) : INDEX_NONE);
}

void UChaosVDRuntimeBlueprintLibrary::SetTraceRelevancyVolume(const UObject* WorldContext, const FBox& RelevancyVolume)
{
	CVD_SET_RELEVANCY_VOLUME(RelevancyVolume);
}
