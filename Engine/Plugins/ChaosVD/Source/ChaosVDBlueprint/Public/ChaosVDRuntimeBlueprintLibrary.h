// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Kismet/BlueprintFunctionLibrary.h"

#include "ChaosVDRuntimeBlueprintLibrary.generated.h"

/**
 * Library function to record debug draw shapes that will be played back when a CVD recording is loaded
 */
UCLASS(MinimalAPI)
class UChaosVDRuntimeBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

	UFUNCTION(BlueprintCallable, Category = "Chaos Visual Debugger", DisplayName="CVD Record Debug Draw Box", meta = (WorldContext = "WorldContext"))
	static void RecordDebugDrawBox(const UObject* WorldContext, const FBox& InBox, FName Tag = NAME_None, FLinearColor Color = FLinearColor::Blue);

	UFUNCTION(BlueprintCallable, Category = "Chaos Visual Debugger",  DisplayName="CVD Record Debug Draw Line", meta = (WorldContext = "WorldContext"))
	static void RecordDebugDrawLine(const UObject* WorldContext, const FVector& InStartLocation, const FVector& InEndLocation, FName Tag = NAME_None, FLinearColor Color = FLinearColor::Blue);

	UFUNCTION(BlueprintCallable, Category = "Chaos Visual Debugger",  DisplayName="CVD Record Debug Draw Vector", meta = (WorldContext = "WorldContext"))
	static void RecordDebugDrawVector(const UObject* WorldContext, const FVector& InStartLocation, const FVector& InVector, FName Tag = NAME_None, FLinearColor Color = FLinearColor::Blue);

	UFUNCTION(BlueprintCallable, Category = "Chaos Visual Debugger", DisplayName="CVD Record Debug Draw Sphere", meta = (WorldContext = "WorldContext"))
	static void RecordDebugDrawSphere(const UObject* WorldContext, const FVector& InCenter, float Radius, FName Tag = NAME_None, FLinearColor Color = FLinearColor::Blue);

	UFUNCTION(BlueprintCallable, Category = "Chaos Visual Debugger", DisplayName="CVD Set Trace Relevancy Volume", meta = (WorldContext = "WorldContext"))
	static void SetTraceRelevancyVolume(const UObject* WorldContext, const FBox& RelevancyVolume);
};
