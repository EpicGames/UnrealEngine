// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class ChaosVDBuiltInExtensions : ModuleRules
{
    public ChaosVDBuiltInExtensions(ReadOnlyTargetRules Target) : base(Target)
    {
	    PublicDependencyModuleNames.AddRange(
		    new string[]
		    {
			    "ChaosVD",
		    });

	    PrivateDependencyModuleNames.AddRange(
            new string[]
            {
	            "Core",
                "CoreUObject",
                "Engine",
                "ChaosVDData",
                "ChaosVDRuntime",
                "Slate",
                "SlateCore",
                "ToolMenus",
                "UnrealEd",
                "EditorWidgets",
                "TraceServices",
				"InputCore",
				"TypedElementFramework",
				"AppFramework",
	}
        );
        
        SetupModulePhysicsSupport(Target);
    }
}