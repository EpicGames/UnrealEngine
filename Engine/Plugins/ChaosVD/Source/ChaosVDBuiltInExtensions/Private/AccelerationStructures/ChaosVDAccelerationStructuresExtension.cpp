// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDAccelerationStructuresExtension.h"

#include "Components/ChaosVDGTAccelerationStructuresDataComponent.h"
#include "Trace/ChaosVDAABBTreeDataProcessor.h"
#include "Visualizers/ChaosVDGTAccelerationStructureDataComponentVisualizer.h"
#include "Widgets/SChaosVDMainTab.h"

FChaosVDAccelerationStructuresExtension::FChaosVDAccelerationStructuresExtension() : FChaosVDExtension()
{
	DataComponentsClasses.Add(UChaosVDGTAccelerationStructuresDataComponent::StaticClass());

	ExtensionName = FName(TEXT("FChaosVDAccelerationStructuresExtension"));
}

FChaosVDAccelerationStructuresExtension::~FChaosVDAccelerationStructuresExtension()
{
	DataComponentsClasses.Reset();
}

void FChaosVDAccelerationStructuresExtension::RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider)
{
	FChaosVDExtension::RegisterDataProcessorsInstancesForProvider(InTraceProvider);
	
	TSharedPtr<FChaosVDAABBTreeDataProcessor> AABBTreeDataProcessor = MakeShared<FChaosVDAABBTreeDataProcessor>();
    AABBTreeDataProcessor->SetTraceProvider(InTraceProvider);
    InTraceProvider->RegisterDataProcessor(AABBTreeDataProcessor);

}

TConstArrayView<TSubclassOf<UActorComponent>> FChaosVDAccelerationStructuresExtension::GetSolverDataComponentsClasses()
{
	return DataComponentsClasses;
}

void FChaosVDAccelerationStructuresExtension::RegisterComponentVisualizers(const TSharedRef<SChaosVDMainTab>& InCVDToolKit)
{
	FChaosVDExtension::RegisterComponentVisualizers(InCVDToolKit);
	
	InCVDToolKit->RegisterComponentVisualizer(UChaosVDGTAccelerationStructuresDataComponent::StaticClass()->GetFName(), MakeShared<FChaosVDGTAccelerationStructureDataComponentVisualizer>());
}