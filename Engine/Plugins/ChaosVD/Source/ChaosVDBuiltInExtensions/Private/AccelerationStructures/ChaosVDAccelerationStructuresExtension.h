// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#pragma once
#include "ExtensionsSystem/ChaosVDExtension.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"

class FChaosVDTraceProvider;
class UActorComponent;
class SChaosVDMainTab;

class FChaosVDAccelerationStructuresExtension final : public FChaosVDExtension
{
public:
	
	FChaosVDAccelerationStructuresExtension();
	virtual ~FChaosVDAccelerationStructuresExtension() override;

	virtual void RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider) override;
	virtual TConstArrayView<TSubclassOf<UActorComponent>> GetSolverDataComponentsClasses() override;
	virtual void RegisterComponentVisualizers(const TSharedRef<SChaosVDMainTab>& InCVDToolKit) override;

private:
	TArray<TSubclassOf<UActorComponent>> DataComponentsClasses;
};
