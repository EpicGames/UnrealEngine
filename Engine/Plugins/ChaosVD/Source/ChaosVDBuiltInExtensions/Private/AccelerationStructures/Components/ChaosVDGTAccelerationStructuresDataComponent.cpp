// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGTAccelerationStructuresDataComponent.h"

#include "ChaosVDRecording.h"
#include "Algo/Copy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDGTAccelerationStructuresDataComponent)

UChaosVDGTAccelerationStructuresDataComponent::UChaosVDGTAccelerationStructuresDataComponent()
{
	PrimaryComponentTick.bCanEverTick = false;
	
	SetCanEverAffectNavigation(false);
	bNavigationRelevant = false;
}

void UChaosVDGTAccelerationStructuresDataComponent::UpdateAABBTreeData(TConstArrayView<TSharedPtr<FChaosVDAABBTreeDataWrapper>> AABBTreeDataView)
{
	RecordedABBTreeData.Reset(AABBTreeDataView.Num());
	Algo::Copy(AABBTreeDataView, RecordedABBTreeData);
}

void UChaosVDGTAccelerationStructuresDataComponent::UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData)
{
	if (TSharedPtr<FChaosVDAccelerationStructureContainer> AABBTreeDataContainer = InGameFrameData.GetCustomDataHandler().GetData<FChaosVDAccelerationStructureContainer>())
	{
		if (const TArray<TSharedPtr<FChaosVDAABBTreeDataWrapper>>* RecordedAABBTreesData = AABBTreeDataContainer->RecordedAABBTreesBySolverID.Find(SolverID))
		{
			UpdateAABBTreeData(*RecordedAABBTreesData);
		}
	}
}

void UChaosVDGTAccelerationStructuresDataComponent::ClearData()
{
	RecordedABBTreeData.Reset();
}
