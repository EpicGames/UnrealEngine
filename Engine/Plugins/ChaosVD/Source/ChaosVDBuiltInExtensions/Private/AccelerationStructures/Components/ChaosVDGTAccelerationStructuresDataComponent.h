// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChaosVDSolverDataComponent.h"
#include "ChaosVDGTAccelerationStructuresDataComponent.generated.h"

struct FChaosVDAABBTreeDataWrapper;
struct FChaosVDGameFrameData;

UCLASS()
class UChaosVDGTAccelerationStructuresDataComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	UChaosVDGTAccelerationStructuresDataComponent();

	void UpdateAABBTreeData(TConstArrayView<TSharedPtr<FChaosVDAABBTreeDataWrapper>> AABBTreeDataView);

	TConstArrayView<TSharedPtr<FChaosVDAABBTreeDataWrapper>> GetAABBTreeData() const { return RecordedABBTreeData; }

	virtual void UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData) override;

	virtual void ClearData() override;
protected:
	
	TArray<TSharedPtr<FChaosVDAABBTreeDataWrapper>> RecordedABBTreeData;
};
