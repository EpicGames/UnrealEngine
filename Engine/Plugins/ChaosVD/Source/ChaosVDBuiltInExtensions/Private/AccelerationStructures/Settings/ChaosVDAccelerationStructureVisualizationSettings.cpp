// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDAccelerationStructureVisualizationSettings.h"

#include "ChaosVDSettingsManager.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDAccelerationStructureVisualizationSettings)


void UChaosVDAccelerationStructureVisualizationSettings::SetDataVisualizationFlags(EChaosVDAccelerationStructureDataVisualizationFlags NewFlags)
{
	if (UChaosVDAccelerationStructureVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDAccelerationStructureVisualizationSettings>())
	{
		Settings->AccelerationStructureDataVisualizationFlags = static_cast<uint32>(NewFlags);
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDAccelerationStructureDataVisualizationFlags UChaosVDAccelerationStructureVisualizationSettings::GetDataVisualizationFlags()
{
	if (UChaosVDAccelerationStructureVisualizationSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDAccelerationStructureVisualizationSettings>())
	{
		return static_cast<EChaosVDAccelerationStructureDataVisualizationFlags>(Settings->AccelerationStructureDataVisualizationFlags);
	}

	return EChaosVDAccelerationStructureDataVisualizationFlags::None;
}

bool UChaosVDAccelerationStructureVisualizationSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, AccelerationStructureDataVisualizationFlags, EChaosVDAccelerationStructureDataVisualizationFlags::EnableDraw);
}
