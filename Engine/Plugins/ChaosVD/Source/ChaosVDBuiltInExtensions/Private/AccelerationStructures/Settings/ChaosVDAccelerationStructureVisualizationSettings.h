// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "ChaosVDAccelerationStructureVisualizationSettings.generated.h"

/** Visualization flags used to control what is debug draw of the recorded acceleration structure data */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDAccelerationStructureDataVisualizationFlags : uint32
{
	None	= 0 UMETA(Hidden),
	/** If Set, draws the bound of all nodes in the tree  */
	DrawNodesBounds = 1 << 0,
	/** If Set, draws lines to represent the branches of the tree  */
	DrawBranches = 1 << 1,
	/** If Set, draws the bounds of the leaves of the tree  */
	DrawLeavesBounds = 1 << 2,
	/** If Set, draws the bounds of each element in the leaves of the tree  */
	DrawLeavesElementBounds = 1 << 3,
	/** If Set, inside each leaf bound, it draws lines from the elements to the center of the leaf bounds */
	DrawLeavesElementConnections = 1 << 4,
	/** If Set, all dynamic trees will be drawn */
	DrawDynamicTrees = 1 << 5,
	/** If Set, all static trees will be drawn */
	DrawStaticTrees = 1 << 6,
	/** If Set, draws the real bounds of each element (not the bounds recorded in the leaf) in the leaves of the tree -
	 * This should match the bounds recorded in the leaf itself, otherwise it means the AABBtree might have out of sync data */
	DrawLeavesRealElementBounds = 1 << 7 UMETA(Hidden), // TODO: This will be unhidden in the nex CL with the required object version bump

	/** If set, enabled debug drawing for any recorded acceleration structure available at the current visualized frame */
	EnableDraw = DrawDynamicTrees | DrawStaticTrees
};
ENUM_CLASS_FLAGS(EChaosVDAccelerationStructureDataVisualizationFlags);

/**
 * Settings object that stores the values that control how acceleration structures are debug drawn
 */
UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDAccelerationStructureVisualizationSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()
public:

	/** The depth priority used for while drawing contact data. Can be World or Foreground (with this one the shapes will be drawn on top of the geometry and be always visible) */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground;

	/** The base thickness used to draw node bounds */
	UPROPERTY(EditAnywhere, config, Category=DebugDraw)
	float BaseThickness = 3.0f;
	
	static void SetDataVisualizationFlags(EChaosVDAccelerationStructureDataVisualizationFlags NewFlags);
	static EChaosVDAccelerationStructureDataVisualizationFlags GetDataVisualizationFlags();
	
	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;
private:
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVD.EChaosVDAccelerationStructureDataVisualizationFlags"))
	uint32 AccelerationStructureDataVisualizationFlags = static_cast<uint32>(EChaosVDAccelerationStructureDataVisualizationFlags::DrawNodesBounds | EChaosVDAccelerationStructureDataVisualizationFlags::DrawBranches);
};
