// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDAABBTreeDataProcessor.h"

#include "DataWrappers/ChaosVDAccelerationStructureDataWrappers.h"

namespace Chaos::VisualDebugger::Utils
{
	int32 CalculateAABBTreeDepth(const TSharedRef<FChaosVDAABBTreeDataWrapper>& InTree, int32 StartNodeIndex, int32 CurrentDepth = 0)
	{
		if(InTree->Nodes.IsEmpty() || (!InTree->Nodes.IsValidIndex(StartNodeIndex)))
		{
			return 0;
		}

		if (InTree->Nodes[StartNodeIndex].bLeaf)
		{
			return 0;
		}

		int32 LeftDepth = CalculateAABBTreeDepth(InTree, InTree->Nodes[StartNodeIndex].ChildrenNodes[0]);
		int32 RightDepth = CalculateAABBTreeDepth(InTree, InTree->Nodes[StartNodeIndex].ChildrenNodes[1]);

		if (LeftDepth > RightDepth)
		{
			return LeftDepth + 1;
		}
		else
		{
			return RightDepth + 1;
		}
	}
}

FChaosVDAABBTreeDataProcessor::FChaosVDAABBTreeDataProcessor() : FChaosVDDataProcessorBase(FChaosVDAABBTreeDataWrapper::WrapperTypeName)
{
}

bool FChaosVDAABBTreeDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	const TSharedPtr<FChaosVDAABBTreeDataWrapper> AABBTreeData = MakeShared<FChaosVDAABBTreeDataWrapper>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *AABBTreeData, ProviderSharedPtr.ToSharedRef());
	
	if (bSuccess)
	{
		if (TSharedPtr<FChaosVDGameFrameData> CurrentFrameData = ProviderSharedPtr->GetCurrentGameFrame().Pin())
		{
			AABBTreeData->SolverId = ProviderSharedPtr->GetRemappedSolverID(AABBTreeData->SolverId);

			AABBTreeData->TreeDepth = Chaos::VisualDebugger::Utils::CalculateAABBTreeDepth(AABBTreeData.ToSharedRef(), AABBTreeData->GetCorrectedRootNodeIndex());

			if (TSharedPtr<FChaosVDAccelerationStructureContainer> AABBTreeDataContainer = CurrentFrameData->GetCustomDataHandler().GetOrAddDefaultData<FChaosVDAccelerationStructureContainer>())
			{
				TArray<TSharedPtr<FChaosVDAABBTreeDataWrapper>>& ABBTrees = AABBTreeDataContainer->RecordedAABBTreesBySolverID.FindOrAdd(AABBTreeData->SolverId);
				ABBTrees.Add(AABBTreeData);
				CurrentFrameData->MarkDirty();
			}
		}
	}

	return bSuccess;
}
