// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Trace/DataProcessors/ChaosVDDataProcessorBase.h"

/**
 * Data processor implementation that is able to deserialize traced AABB Tree Data
 */
class FChaosVDAABBTreeDataProcessor final : public FChaosVDDataProcessorBase
{
public:
	explicit FChaosVDAABBTreeDataProcessor();
	
	virtual bool ProcessRawData(const TArray<uint8>& InData) override;
};
