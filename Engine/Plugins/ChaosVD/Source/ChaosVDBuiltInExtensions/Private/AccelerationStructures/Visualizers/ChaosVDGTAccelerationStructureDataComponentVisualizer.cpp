// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGTAccelerationStructureDataComponentVisualizer.h"

#include "AccelerationStructures/Components/ChaosVDGTAccelerationStructuresDataComponent.h"
#include "AccelerationStructures/Settings/ChaosVDAccelerationStructureVisualizationSettings.h"
#include "Actors/ChaosVDDataContainerBaseActor.h"
#include "ChaosVDScene.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "SceneView.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDGTAccelerationStructureDataComponentVisualizer)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

class AChaosVDSolverInfoActor;

namespace Chaos::VisualDebugger::Utils
{
	void DrawFBoxAtLocation(FPrimitiveDrawInterface* PDI, const FBox& InBox, FColor Color, ESceneDepthPriorityGroup DepthPriority, float Thickness)
	{
		FVector Center;
		FVector Extents;	
		InBox.GetCenterAndExtents(Center, Extents);

		FTransform LocationTransform;
		LocationTransform.SetLocation(Center);
			
		FChaosVDDebugDrawUtils::DrawBox(PDI, Extents, Color, LocationTransform, FText::GetEmpty(), DepthPriority, Thickness);
	}

	void DrawBoxAtLocation(FPrimitiveDrawInterface* PDI, const FVector& Center, const FVector& Extents, FColor Color, ESceneDepthPriorityGroup DepthPriority, float Thickness)
	{
		FTransform LocationTransform;
		LocationTransform.SetLocation(Center);
			
		FChaosVDDebugDrawUtils::DrawBox(PDI, Extents, Color, LocationTransform, FText::GetEmpty(), DepthPriority, Thickness);
	}
}

bool FChaosVDGTAccelerationStructureSelectionHandle::IsSelected()
{
	// In contrast to other recorded data types, AABB Tree data is recorded as a single struct, so we need to also use the context data to match a selection handle
	if (bool bIsPrimaryDataSelected = FChaosVDSolverDataSelectionHandle::IsSelected())
	{
		if (TSharedPtr<FChaosVDSolverDataSelection> OwnerPtr = Owner.Pin())
		{
			TSharedPtr<FChaosVDSolverDataSelectionHandle> CurrentSelectedDataHandle = OwnerPtr->GetCurrentSelectionHandle();

			FChaosVDAABBTreeSelectionContext* CurrentSelectionContext = CurrentSelectedDataHandle->GetContextData<FChaosVDAABBTreeSelectionContext>();
			FChaosVDAABBTreeSelectionContext* HandleSelectionContext = GetContextData<FChaosVDAABBTreeSelectionContext>();

			return CurrentSelectionContext && HandleSelectionContext && (*CurrentSelectionContext) == (*HandleSelectionContext);
		}
	}

	return false;
}

void FChaosVDGTAccelerationStructureSelectionHandle::CreateStructViewForDetailsPanelIfNeeded()
{
	if (StructDataView)
	{
		return;
	}

	StructDataView = MakeShared<FChaosVDSelectionMultipleView>();

	if (FChaosVDAABBTreeDataWrapper* TreeData = GetData<FChaosVDAABBTreeDataWrapper>())
	{
		StructDataView->AddData(TreeData);
	}

	if (FChaosVDAABBTreeSelectionContext* SelectionContext = GetContextData<FChaosVDAABBTreeSelectionContext>())
	{
		StructDataView->AddData(const_cast<FChaosVDAABBTreeNodeDataWrapper*>(SelectionContext->NodeData));
		StructDataView->AddData(const_cast<FChaosVDAABBTreeLeafDataWrapper*>(SelectionContext->LeafData));
	}

	StructDataViewStructOnScope = MakeShared<FStructOnScope>(FChaosVDSelectionMultipleView::StaticStruct(), reinterpret_cast<uint8*>(StructDataView.Get()));
}

TSharedPtr<FStructOnScope> FChaosVDGTAccelerationStructureSelectionHandle::GetCustomDataReadOnlyStructViewForDetails()
{
	// To avoid unnecessary work, only create and cache a view struct when requested (which happens when we try to use this selection handle to udpate a details panel)
	CreateStructViewForDetailsPanelIfNeeded();

	return StructDataViewStructOnScope;
}

FChaosVDGTAccelerationStructureDataComponentVisualizer::FChaosVDGTAccelerationStructureDataComponentVisualizer()
{
	RegisterVisualizerMenus();
	InspectorTabID = FChaosVDTabID::DetailsPanel;
}

void FChaosVDGTAccelerationStructureDataComponentVisualizer::RegisterVisualizerMenus()
{
	FName MenuSection("AccelerationStructureDataVisualization.Show");
	FText MenuSectionLabel = LOCTEXT("AccelerationStructureDataShowMenuLabel", "Acceleration Structure Data Visualization");
	FText FlagsMenuLabel = LOCTEXT("AccelerationStructureDataFlagsMenuLabel", "Acceleration Structure Data Flags");
	FText FlagsMenuTooltip = LOCTEXT("AccelerationStructureDataFlagsMenuToolTip", "Set of flags to enable/disable visibility of specific types of acceleration structure data");
	FSlateIcon FlagsMenuIcon = FSlateIcon(FChaosVDStyle::Get().GetStyleSetName(), TEXT("SceneQueriesInspectorIcon"));

	FText SettingsMenuLabel = LOCTEXT("AccelerationStructureSettingsMenuLabel", "Acceleration Structure Visualization Settings");
	FText SettingsMenuTooltip = LOCTEXT("AccelerationStructureSettingsMenuToolTip", "Options to change how the recorded acceleration structure data is debug drawn");
	
	CreateGenericVisualizerMenu<UChaosVDAccelerationStructureVisualizationSettings, EChaosVDAccelerationStructureDataVisualizationFlags>(FName("ChaosVDViewportToolbarBase.Show"), MenuSection, MenuSectionLabel, FlagsMenuLabel, FlagsMenuTooltip, FlagsMenuIcon, SettingsMenuLabel, SettingsMenuTooltip);
}

void FChaosVDGTAccelerationStructureDataComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDGTAccelerationStructuresDataComponent* DataComponent = Cast<UChaosVDGTAccelerationStructuresDataComponent>(Component);
	if (!DataComponent)
	{
		return;
	}
	
	AChaosVDDataContainerBaseActor* DataInfoActor = Cast<AChaosVDDataContainerBaseActor>(Component->GetOwner());
	if (!DataInfoActor)
	{
		return;
	}

	if (!DataInfoActor->IsVisible())
	{
		return;
	}

	const TSharedPtr<FChaosVDScene> CVDScene = DataInfoActor->GetScene().Pin();
	if (!CVDScene)
	{
		return;
	}
	
	FChaosGTAccelerationStructureVisualizationDataContext VisualizationContext;
	VisualizationContext.CVDScene = CVDScene;
	VisualizationContext.SpaceTransform = DataInfoActor->GetSimulationTransform();
	VisualizationContext.SolverDataSelectionObject = CVDScene->GetSolverDataSelectionObject().Pin();
	VisualizationContext.DataComponent = DataComponent;

	if (const UChaosVDAccelerationStructureVisualizationSettings* EditorSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDAccelerationStructureVisualizationSettings>())
	{
		VisualizationContext.VisualizationFlags = static_cast<uint32>(UChaosVDAccelerationStructureVisualizationSettings::GetDataVisualizationFlags());
		VisualizationContext.DebugDrawSettings = EditorSettings;
		VisualizationContext.DepthPriority = EditorSettings->DepthPriority;
	}

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::EnableDraw))
	{
		return;
	}

	TConstArrayView<TSharedPtr<FChaosVDAABBTreeDataWrapper>> RecordedAABBTrees = DataComponent->GetAABBTreeData();

	for (const TSharedPtr<FChaosVDAABBTreeDataWrapper>& AABBTreeDataWrapper : RecordedAABBTrees)
	{
		if (AABBTreeDataWrapper)
		{
			const bool bCanDrawTree = (AABBTreeDataWrapper->bDynamicTree && VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawDynamicTrees))
								|| (!AABBTreeDataWrapper->bDynamicTree && VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawStaticTrees));

			if (bCanDrawTree)
			{
				DrawAABBTree(View, PDI, VisualizationContext, AABBTreeDataWrapper.ToSharedRef());
			}	
		}
	}
}

bool FChaosVDGTAccelerationStructureDataComponentVisualizer::CanHandleClick(const HChaosVDComponentVisProxy& VisProxy)
{
	return VisProxy.DataSelectionHandle && (VisProxy.DataSelectionHandle->IsA<FChaosVDAABBTreeDataWrapper>()
											|| VisProxy.DataSelectionHandle->IsA<FChaosVDAABBTreeNodeDataWrapper>()
											|| VisProxy.DataSelectionHandle->IsA<FChaosVDAABBTreeLeafDataWrapper>());
}

void FChaosVDGTAccelerationStructureDataComponentVisualizer::DrawAABBTree(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosGTAccelerationStructureVisualizationDataContext& VisualizationContext, const TSharedRef<FChaosVDAABBTreeDataWrapper>& AABBTreeData)
{
	const UChaosVDAccelerationStructureVisualizationSettings* Settings = Cast<const UChaosVDAccelerationStructureVisualizationSettings>(VisualizationContext.DebugDrawSettings);
	if (!ensure(Settings))
	{
		return;
	}

	int32 RootNodeIndex = AABBTreeData->GetCorrectedRootNodeIndex();
	if (AABBTreeData->Nodes.Num() > 0 && AABBTreeData->Nodes.IsValidIndex(RootNodeIndex))
	{
		DrawAABBTreeNode(View, PDI, VisualizationContext, AABBTreeData, AABBTreeData->Nodes[RootNodeIndex], Settings->BaseThickness);
	}
}

void FChaosVDGTAccelerationStructureDataComponentVisualizer::DrawAABBTreeNode(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosGTAccelerationStructureVisualizationDataContext& VisualizationContext, const TSharedRef<FChaosVDAABBTreeDataWrapper>& AABBTreeData, const FChaosVDAABBTreeNodeDataWrapper& AABBTreeNodeData, float Thickness, int32 CurrentTreeLevel)
{
	const bool bCanDrawNodeData = VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawNodesBounds | EChaosVDAccelerationStructureDataVisualizationFlags::DrawBranches);
	const bool bCanDrawLeavesData = VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawLeavesBounds | EChaosVDAccelerationStructureDataVisualizationFlags::DrawLeavesElementBounds | EChaosVDAccelerationStructureDataVisualizationFlags::DrawLeavesElementConnections);

	if (!bCanDrawNodeData && !bCanDrawLeavesData)
	{
		return;
	}

	if (AABBTreeNodeData.bLeaf)
	{
		if (bCanDrawLeavesData)
		{
			for (int32 ChildIndex : AABBTreeNodeData.ChildrenNodes)
			{
				if (AABBTreeData->TreeArrayLeafs.IsValidIndex(ChildIndex))
				{
					DrawAABBTreeArrayLeaf(View, PDI, VisualizationContext, AABBTreeData->TreeArrayLeafs[ChildIndex], AABBTreeData, Thickness);
				}	
			}
		}
		return;
	}

	auto IsNodeVisible = [View](const FBox& NodeBounds)
	{
		return View->ViewFrustum.IntersectBox(NodeBounds.GetCenter(), NodeBounds.GetExtent());
	};

	// Calculate and cache the total bounds of this node and its visibility state
	FBox TotalNodeBounds(ForceInitToZero);
	bool IsChildNodeVisible[2] = { false, false };

	constexpr int32 MaxChildNodeNum = 2;
	for (int32 ChildIndex = 0; ChildIndex < MaxChildNodeNum; ++ChildIndex)
	{
		const FBox& ChildBounds = AABBTreeNodeData.ChildrenBounds[ChildIndex];
		IsChildNodeVisible[ChildIndex] = IsNodeVisible(ChildBounds);

		TotalNodeBounds += ChildBounds;
	}

	const bool bIsCurrentNodeVisible = IsChildNodeVisible[0] || IsChildNodeVisible[1];
	if (!bIsCurrentNodeVisible)
	{
		// If this node is not visible at all, nothing to do here
		return;
	}

	// If node data drawing is disabled, we can skip all the logic to create the selection handle and draw the lines and go straight to continue traversing the tree
	if (bCanDrawNodeData)
	{
		bool bIsRootNode = AABBTreeNodeData.ParentNode == INDEX_NONE;
		float FinalThickness = Thickness;

		FColor BoundsColor = FColor::MakeRedToGreenColorFromScalar(static_cast<float>(CurrentTreeLevel) / static_cast<float>(AABBTreeData->TreeDepth));

		TSharedPtr<FChaosVDSolverDataSelectionHandle> NodeSelectionHandle = VisualizationContext.SolverDataSelectionObject->MakeSelectionHandle<FChaosVDAABBTreeDataWrapper, FChaosVDGTAccelerationStructureSelectionHandle>(AABBTreeData.ToSharedPtr());

		FChaosVDAABBTreeSelectionContext ContextData;
		// The lifetime of the structure where this node data lives is bound to the selection handle, so we can safely store a ptr to it
		ContextData.NodeData = &AABBTreeNodeData;
		NodeSelectionHandle->SetHandleContext(MoveTemp(ContextData));
			
		FinalThickness = NodeSelectionHandle->IsSelected() ? FinalThickness * 2.5f : FinalThickness;

		PDI->SetHitProxy(new HChaosVDComponentVisProxy(VisualizationContext.DataComponent, NodeSelectionHandle));
	
		for (int32 ChildIndex = 0; ChildIndex < MaxChildNodeNum; ++ChildIndex)
		{
			if (!IsChildNodeVisible[ChildIndex])
			{
				continue;
			}
			
			const FBox& ChildBounds = AABBTreeNodeData.ChildrenBounds[ChildIndex];

			if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawNodesBounds))
			{
				Chaos::VisualDebugger::Utils::DrawFBoxAtLocation(PDI, ChildBounds, BoundsColor, VisualizationContext.DepthPriority, FinalThickness);
			}
		
			if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawBranches))
			{			
				const FVector NodeCenter = TotalNodeBounds.GetCenter();
				FColor BranchColor = FColor::MakeRedToGreenColorFromScalar(static_cast<float>(CurrentTreeLevel) / static_cast<float>(AABBTreeData->TreeDepth));

				FChaosVDDebugDrawUtils::DrawLine(PDI, NodeCenter, ChildBounds.GetCenter(), BranchColor, FText::GetEmpty(), VisualizationContext.DepthPriority, FinalThickness * 1.2f);

				constexpr float BranchStatPointBoxSize = 1.0f;
				static FVector StartPointBoxExtent(BranchStatPointBoxSize, BranchStatPointBoxSize, BranchStatPointBoxSize);
				Chaos::VisualDebugger::Utils::DrawBoxAtLocation(PDI, NodeCenter, StartPointBoxExtent, bIsRootNode ? FColor::Red : BranchColor, VisualizationContext.DepthPriority, FinalThickness * (bIsRootNode ? 7.0f : 4.0f));
			}
		}

		if (bIsRootNode && VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawNodesBounds))
		{
			// If we are the root node, also draw a box showing the bounds of the whole three
			Chaos::VisualDebugger::Utils::DrawFBoxAtLocation(PDI, TotalNodeBounds, FColor::Red, VisualizationContext.DepthPriority, FinalThickness);
		}
	
		PDI->SetHitProxy(nullptr);
	}

	// We can have leaf data drawing enabled while node data drawing is disabled, therefore we still need to continue traversing the tree to get to the leaves.
	if (bCanDrawLeavesData || bCanDrawNodeData)
	{
		for (int32 ChildIndex = 0; ChildIndex < MaxChildNodeNum; ++ChildIndex)
		{
			// if the child node is not visible, we can discard the entire branch
			if (!IsChildNodeVisible[ChildIndex])
			{
				continue;
			}

			int32 ChildNodeIndex = AABBTreeNodeData.ChildrenNodes[ChildIndex];
			if (ChildNodeIndex > 0 && ChildNodeIndex < AABBTreeData->Nodes.Num())
			{
				constexpr float LineThicknessRatio = 0.75f;
				DrawAABBTreeNode(View, PDI, VisualizationContext, AABBTreeData, AABBTreeData->Nodes[ChildNodeIndex], Thickness * LineThicknessRatio, CurrentTreeLevel + 1);
			}
		}
	}
}

void FChaosVDGTAccelerationStructureDataComponentVisualizer::DrawAABBTreeArrayLeaf(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosGTAccelerationStructureVisualizationDataContext& VisualizationContext, const FChaosVDAABBTreeLeafDataWrapper& AABBTreeArrayLeafData, const TSharedRef<FChaosVDAABBTreeDataWrapper>& AABBTreeData, float Thickness)
{
	// Early out if this leaf will not be visible
	if (!View->ViewFrustum.IntersectBox(AABBTreeArrayLeafData.Bounds.GetCenter(), AABBTreeArrayLeafData.Bounds.GetExtent()))
	{
		return;
	}

	constexpr float MaxDensityNumForColor = 10.0f;
	float InverseAlpha =  1.0f - (static_cast<float>(AABBTreeArrayLeafData.Elements.Num()) / MaxDensityNumForColor);

	FColor ColorByDensity = FColor::MakeRedToGreenColorFromScalar(InverseAlpha);

	TSharedPtr<FChaosVDSolverDataSelectionHandle> LeafSelectionHandle = VisualizationContext.SolverDataSelectionObject->MakeSelectionHandle<FChaosVDAABBTreeDataWrapper, FChaosVDGTAccelerationStructureSelectionHandle>(AABBTreeData.ToSharedPtr());

	FChaosVDAABBTreeSelectionContext ContextData;
	// The lifetime of the structure where this leaf data lives is bound to the selection handle, so we can safely store a ptr to it
	ContextData.LeafData = &AABBTreeArrayLeafData;
	LeafSelectionHandle->SetHandleContext(MoveTemp(ContextData));

	PDI->SetHitProxy(new HChaosVDComponentVisProxy(VisualizationContext.DataComponent, LeafSelectionHandle));

	float FinalThickness = LeafSelectionHandle->IsSelected() ? Thickness * 2.5f : Thickness; 

	if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawLeavesBounds))
	{		
		Chaos::VisualDebugger::Utils::DrawFBoxAtLocation(PDI, AABBTreeArrayLeafData.Bounds, FColor::Green, VisualizationContext.DepthPriority, FinalThickness);
	}

	for (const FChaosVDAABBTreePayloadBoundsElement& TreeArrayLeafElement : AABBTreeArrayLeafData.Elements)
	{
		if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawLeavesElementConnections))
		{
			FChaosVDDebugDrawUtils::DrawLine(PDI, AABBTreeArrayLeafData.Bounds.GetCenter(), TreeArrayLeafElement.Bounds.GetCenter(), ColorByDensity, FText::GetEmpty(), VisualizationContext.DepthPriority, FinalThickness);
		}

		if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawLeavesElementBounds))
		{	
			Chaos::VisualDebugger::Utils::DrawFBoxAtLocation(PDI, TreeArrayLeafElement.Bounds, ColorByDensity, VisualizationContext.DepthPriority, FinalThickness * 0.7f);
		}

		if (VisualizationContext.IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags::DrawLeavesRealElementBounds))
		{
			Chaos::VisualDebugger::Utils::DrawFBoxAtLocation(PDI, TreeArrayLeafElement.ActualBounds, FColor::Red, VisualizationContext.DepthPriority, FinalThickness * 0.7f);
		}
	}

	PDI->SetHitProxy(nullptr);
}

#undef LOCTEXT_NAMESPACE
