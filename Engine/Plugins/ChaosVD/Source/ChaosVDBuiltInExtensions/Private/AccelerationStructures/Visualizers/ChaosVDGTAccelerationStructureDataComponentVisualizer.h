// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Visualizers/ChaosVDComponentVisualizerBase.h"
#include "ChaosVDSolverDataSelection.h"
#include "AccelerationStructures/Settings/ChaosVDAccelerationStructureVisualizationSettings.h"

#include "ChaosVDGTAccelerationStructureDataComponentVisualizer.generated.h"

class UChaosVDGTAccelerationStructuresDataComponent;
struct FChaosVDAABBTreeLeafDataWrapper;
struct FChaosVDAABBTreeNodeDataWrapper;
struct FChaosVDAABBTreeDataWrapper;

enum class EChaosVDVisibleAABBTreeNodes
{
	None = 0,
	Left = 1 << 0,
	Right = 1 << 1
};
ENUM_CLASS_FLAGS(EChaosVDVisibleAABBTreeNodes)

USTRUCT()
struct FChaosVDAABBTreeSelectionContext
{
	GENERATED_BODY()

	const FChaosVDAABBTreeNodeDataWrapper* NodeData = nullptr;
	const FChaosVDAABBTreeLeafDataWrapper* LeafData = nullptr;

	bool operator==(const FChaosVDAABBTreeSelectionContext& Other) const
	{
		return NodeData == Other.NodeData && LeafData == Other.LeafData;
	}
};

struct FChaosVDGTAccelerationStructureSelectionHandle : public FChaosVDSolverDataSelectionHandle
{
	virtual bool IsSelected() override;
	void CreateStructViewForDetailsPanelIfNeeded();

	virtual TSharedPtr<FStructOnScope> GetCustomDataReadOnlyStructViewForDetails() override;

private:
	TSharedPtr<FChaosVDSelectionMultipleView> StructDataView;
	TSharedPtr<FStructOnScope> StructDataViewStructOnScope;
};

/** Visualization context structure specific for acceleration structure visualizations */
struct FChaosGTAccelerationStructureVisualizationDataContext : public FChaosVDVisualizationContext
{
	TSharedPtr<FChaosVDSolverDataSelectionHandle> DataSelectionHandle = MakeShared<FChaosVDSolverDataSelectionHandle>();

	ESceneDepthPriorityGroup DepthPriority = SDPG_Foreground;

	const UChaosVDGTAccelerationStructuresDataComponent* DataComponent = nullptr;

	bool IsVisualizationFlagEnabled(EChaosVDAccelerationStructureDataVisualizationFlags Flag) const
	{
		const EChaosVDAccelerationStructureDataVisualizationFlags FlagsAsAccelerationStructureVisFlags = static_cast<EChaosVDAccelerationStructureDataVisualizationFlags>(VisualizationFlags);
		return EnumHasAnyFlags(FlagsAsAccelerationStructureVisFlags, Flag);
	}	
};

class FChaosVDGTAccelerationStructureDataComponentVisualizer final : public FChaosVDComponentVisualizerBase
{
public:
	FChaosVDGTAccelerationStructureDataComponentVisualizer();

	virtual void RegisterVisualizerMenus() override;

	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;

	virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy) override;

protected:

	void DrawAABBTree(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosGTAccelerationStructureVisualizationDataContext& VisualizationContext, const TSharedRef<FChaosVDAABBTreeDataWrapper>& AABBTreeData);
	void DrawAABBTreeNode(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosGTAccelerationStructureVisualizationDataContext& VisualizationContext, const TSharedRef<FChaosVDAABBTreeDataWrapper>& AABBTreeData, const FChaosVDAABBTreeNodeDataWrapper& AABBTreeNodeData, float Thickness, int32 CurrentTreeLevel = 1);
	void DrawAABBTreeArrayLeaf(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosGTAccelerationStructureVisualizationDataContext& VisualizationContext, const FChaosVDAABBTreeLeafDataWrapper& AABBTreeArrayLeafData, const TSharedRef<FChaosVDAABBTreeDataWrapper>& AABBTreeData, float Thickness);
};
