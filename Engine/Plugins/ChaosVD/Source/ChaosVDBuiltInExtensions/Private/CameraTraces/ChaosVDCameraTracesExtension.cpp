// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCameraTracesExtension.h"
#include "Actors/ChaosVDDataContainerBaseActor.h"
#include "Trace/ChaosVDCameraDataProcessor.h"
#include "Settings/ChaosVDCameraDataSettings.h"
#include "Visualizers/ChaosVDCameraDataComponentVisualizer.h"
#include "Widgets/SChaosVDMainTab.h"
#include "ChaosVD/Public/ChaosVDPlaybackViewportClient.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "SEditorViewport.h"
#include "ToolMenu.h"
#include "ToolMenus.h"
#include "ToolMenuEntry.h"
#include "ToolMenuSection.h"
#include "Widgets/SWidget.h"
#include "ChaosVDPlaybackController.h"
#include "ChaosVDScene.h"
#include "Components/ChaosVDCameraDataComponent.h"
#include "ChaosVDEngine.h"

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

const FName FChaosVDCameraTracesExtension::ChaosViewportType = FName("SChaosVDPlaybackViewport");
const FName FChaosVDCameraTracesExtension::CameraSeparatorName = FName("ChaosVDCameraMenuSeparator");

FChaosVDCameraTracesExtension::FChaosVDCameraTracesExtension() : FChaosVDExtension()
{
	DataComponentsClasses.Add(UChaosVDCameraDataComponent::StaticClass());

	ExtensionName = FName(TEXT("FChaosVDCameraTracesExtension"));
}

FChaosVDCameraTracesExtension::~FChaosVDCameraTracesExtension()
{
	DataComponentsClasses.Reset();
}

void FChaosVDCameraTracesExtension::RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider)
{
	FChaosVDExtension::RegisterDataProcessorsInstancesForProvider(InTraceProvider);
	
	TSharedPtr<FChaosVDCameraDataProcessor> CameraDataProcessor = MakeShared<FChaosVDCameraDataProcessor>();
	CameraDataProcessor->SetTraceProvider(InTraceProvider);
    InTraceProvider->RegisterDataProcessor(CameraDataProcessor);

}

void FChaosVDCameraTracesExtension::HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController)
{
	if (UChaosVDCameraDataSettings* EditorSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCameraDataSettings>())
	{
		if(EditorSettings->bViewportStartAtCameraTrace)
		{
			if (TSharedPtr<FChaosVDPlaybackController> Playback = InController.Pin())
			{
				if (Playback->IsRecordingLoaded())
				{
					if (FChaosVDPlaybackViewportClient* ViewportClient = GetPlaybackViewportClient())
					{
						FViewportCameraTransform& ViewTransform = ViewportClient->GetViewTransform();
						FVector Position;
						FQuat Rotation;
						float FieldOfView;

						if (!ViewportClient->IsAutoTrackingSelectedObject() && FindCameraData(Position, Rotation, FieldOfView, ECameraFindFlags::FindFirst))
						{
							ViewTransform.SetLocation(Position);
							ViewTransform.SetRotation(FRotator(Rotation));
						}
					}
				}
			}
		}
	}
}

TConstArrayView<TSubclassOf<UActorComponent>> FChaosVDCameraTracesExtension::GetSolverDataComponentsClasses()
{
	return DataComponentsClasses;
}

bool FChaosVDCameraTracesExtension::FindCameraData(FVector& Position, FQuat& Rotation, float& FieldOfView, ECameraFindFlags CameraFindFlags = ECameraFindFlags::Default) const
{
	if (SelectedCamera.IsSet() || CameraFindFlags == ECameraFindFlags::FindFirst)
	{
		if (TSharedPtr<FChaosVDScene> ScenePtr = WeakScenePtr.Pin())
		{
			TConstArrayView<TObjectPtr<AChaosVDDataContainerBaseActor>> DataContainerActors = ScenePtr->GetDataContainerActorsView();
			TInlineComponentArray<const UChaosVDCameraDataComponent*> CameraComponents;

			for (const TObjectPtr<AChaosVDDataContainerBaseActor>& DataContainerActor : DataContainerActors)
			{
				constexpr bool bIncludeFromChildActors = false;

				DataContainerActor->ForEachComponent(bIncludeFromChildActors, [&CameraComponents](UActorComponent* Component)
				{
					if (const UChaosVDCameraDataComponent* CameraComponent = Cast<UChaosVDCameraDataComponent>(Component))
					{
						CameraComponents.Emplace(CameraComponent);
					}
				});
			}

			if (CameraComponents.Num() > 0)
			{
				for (const UChaosVDCameraDataComponent* Container : CameraComponents)
				{
					TConstArrayView<TSharedPtr<FChaosVDCameraDataWrapper>> DataWrappers = Container->GetCameraData();
					for (const TSharedPtr<FChaosVDCameraDataWrapper>& DataWrapper : DataWrappers)
					{
						if ((SelectedCamera.IsSet() && DataWrapper->Camera == SelectedCamera.GetValue()) || CameraFindFlags == ECameraFindFlags::FindFirst)
						{
							Position = DataWrapper->Position;
							Rotation = DataWrapper->Rotation;
							FieldOfView = DataWrapper->FOV;
							return true;
						}
					}
				}
			}
		}
	}
	return false;
}

void FChaosVDCameraTracesExtension::SetTrackedCamera(FChaosVDCameraIdentifier InTargetCamera)
{
	SelectedCamera = InTargetCamera;

	HandleCVDSceneUpdated();
}

void FChaosVDCameraTracesExtension::ClearTrackedCamera()
{
	SelectedCamera.Reset();
}

FChaosVDPlaybackViewportClient* FChaosVDCameraTracesExtension::GetPlaybackViewportClient()
{
	if (TSharedPtr<FEditorModeTools> EditorModeToolsPtr = EditorModeToolsWeakPtr.Pin())
	{
		if (FEditorViewportClient* EditorClient = EditorModeToolsPtr->GetHoveredViewportClient())
		{
			if (TSharedPtr<SEditorViewport> EditorViewport = EditorClient->GetEditorViewportWidget())
			{
				if (EditorViewport->GetType() == ChaosViewportType)
				{
					return static_cast<FChaosVDPlaybackViewportClient*>(EditorModeToolsPtr->GetHoveredViewportClient());
				}
			}
		}
	}
	return nullptr;
}

void FChaosVDCameraTracesExtension::HandleCVDSceneUpdated()
{
	if (FChaosVDPlaybackViewportClient* ViewportClient = GetPlaybackViewportClient())
	{
		if (!ViewportClient->IsAutoTrackingSelectedObject())
		{
			FViewportCameraTransform& ViewTransform = ViewportClient->GetViewTransform();
			FVector Position;
			FQuat Rotation;
			float FieldOfView;

			if (FindCameraData(Position, Rotation, FieldOfView))
			{
				ViewTransform.SetLocation(Position);
				ViewTransform.SetRotation(FRotator(Rotation));
				ViewportClient->ViewFOV = FieldOfView;
			}
			else
			{
				ViewportClient->ViewFOV = ViewportClient->FOVAngle;
			}

			ViewportClient->Invalidate();
		}
		else
		{
			ClearTrackedCamera();
		}
	}
}

void FChaosVDCameraTracesExtension::OnSolverVisibilityUpdated(int32 SolverID, bool bNewVisibility)
{
	HandleCVDSceneUpdated();
}

void FChaosVDCameraTracesExtension::HandleSettingsChanged(UObject* SettingsObject)
{
	if (UChaosVDCameraDataSettings* Settings = Cast<UChaosVDCameraDataSettings>(SettingsObject))
	{
		bViewportStartAtCameraTrace = Settings->bViewportStartAtCameraTrace;
	}
}

void FChaosVDCameraTracesExtension::ToggleOffObjectTracking()
{
	if (FChaosVDPlaybackViewportClient* ViewportClient = GetPlaybackViewportClient())
	{
		if (ViewportClient->IsAutoTrackingSelectedObject())
		{
			ViewportClient->ToggleObjectTrackingIfSelected();
		}
	}
}

void FChaosVDCameraTracesExtension::CreateCameraTracesMenu()
{
	if (UChaosVDCameraDataSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCameraDataSettings>())
	{
		Settings->OnSettingsChanged().AddSP(this, &FChaosVDCameraTracesExtension::HandleSettingsChanged);
		HandleSettingsChanged(Settings);
	}

	ToggleOffObjectTracking();

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (UToolMenu* Menu = ToolMenus->ExtendMenu(FName("ChaosVDViewportToolbarBase")))
	{
		FToolMenuEntry Entry = FToolMenuEntry::InitSubMenu("CameraTraceSettings",
			LOCTEXT("CameraTracingSubMenuLabel", "Camera Trace Settings"),
			LOCTEXT("CameraTracingSubMenuTooltip", "Additional settings for camera traces."),
			FNewToolMenuDelegate::CreateLambda([this](UToolMenu* Menu)
				{
					FToolMenuSection& OptionsSection = Menu->AddDynamicSection("ViewportOptions", FNewToolMenuDelegate::CreateLambda(
						[this](UToolMenu* InDynamicMenu)
						{
							FToolMenuSection& Section = InDynamicMenu->AddSection("CameraTraces");

							Section.AddMenuEntry(
								NAME_None,
								LOCTEXT("CameraTracingViewportStartAtCameraLabel", "Get viewport starting position from trace"),
								FText::GetEmpty(),
								FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("Icons.Toolbar.Settings")),
								FUIAction(
									FExecuteAction::CreateLambda([this]()
										{

											if (UChaosVDCameraDataSettings* EditorSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCameraDataSettings>())
											{
												UChaosVDCameraDataSettings::SetViewportStartAtCameraTrace(!EditorSettings->bViewportStartAtCameraTrace);
												bViewportStartAtCameraTrace = EditorSettings->bViewportStartAtCameraTrace;
											}
										}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([this]()
										{
											return bViewportStartAtCameraTrace;
										})
								),
								EUserInterfaceActionType::Check
							);

							Section.AddSeparator(CameraSeparatorName);

							bool bFoundEntry = false;

							if (TSharedPtr<FChaosVDScene> ScenePtr = this->WeakScenePtr.Pin()) {
								constexpr bool bIncludeFromChildActors = false;
								TInlineComponentArray<const UChaosVDCameraDataComponent*> CameraComponents;
								TConstArrayView<TObjectPtr<AChaosVDDataContainerBaseActor>> DataContainerActors = ScenePtr->GetDataContainerActorsView();
								for (const TObjectPtr<AChaosVDDataContainerBaseActor>& DataContainerActor : DataContainerActors)
								{
									DataContainerActor->ForEachComponent(bIncludeFromChildActors, [&CameraComponents](UActorComponent* Component)
									{
										if (const UChaosVDCameraDataComponent* CameraComponent = Cast<UChaosVDCameraDataComponent>(Component))
										{
											CameraComponents.Emplace(Cast<UChaosVDCameraDataComponent>(CameraComponent));
										}
									});
								}

								TSet<FChaosVDCameraIdentifier> CameraSet;

								if (CameraComponents.Num() > 0)
								{
									for (const UChaosVDCameraDataComponent* Container : CameraComponents)
									{
										TConstArrayView<TSharedPtr<FChaosVDCameraDataWrapper>> DataWrappers = Container->GetCameraData();
										for (const TSharedPtr<FChaosVDCameraDataWrapper>& DataWrapper : DataWrappers)
										{
											FChaosVDCameraIdentifier CameraWrapper = DataWrapper->Camera;

											if(!CameraSet.Contains(CameraWrapper))
											{
												FUIAction Action(
													FExecuteAction::CreateLambda([this, CameraWrapper]()
														{
															ToggleOffObjectTracking();
															SetTrackedCamera(CameraWrapper);
														}),
													FCanExecuteAction(),
													FIsActionChecked::CreateLambda([this, CameraWrapper]()
														{
															return SelectedCamera.IsSet() && SelectedCamera.GetValue() == CameraWrapper;
														})
												);

												bFoundEntry |= (SelectedCamera.IsSet() && SelectedCamera.GetValue() == CameraWrapper);

												Section.AddMenuEntry(
													NAME_None,
													FText::FromName(CameraWrapper.CameraName),
													FText::FromString(FString::Printf(TEXT("%s - %s"), *CameraWrapper.MapAssetPath.ToString(), *CameraWrapper.ActorName.ToString())),
													FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ClassIcon.CameraComponent"),
													Action,
													EUserInterfaceActionType::RadioButton
												);
												CameraSet.Add(CameraWrapper);
											}
										}
									}
								}
							}

							FToolMenuEntry PerspectiveEntry = FToolMenuEntry::InitMenuEntry(
								NAME_None,
								LOCTEXT("CameraTracingSubMenuPerspectiveLabel", "Perspective View"),
								FText::GetEmpty(),
								FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ClassIcon.CameraComponent"),
								FUIAction(
									FExecuteAction::CreateLambda([this]()
										{
											ClearTrackedCamera();
										}),
									FCanExecuteAction(),
									FIsActionChecked::CreateLambda([this, bFoundEntry]()
										{
											return !bFoundEntry;
										})
								),
								EUserInterfaceActionType::RadioButton
							);

							PerspectiveEntry.InsertPosition.Position = EToolMenuInsertType::After;
							PerspectiveEntry.InsertPosition.Name = CameraSeparatorName;
							Section.AddEntry(PerspectiveEntry);

						}
					));
				}));

		Entry.ToolBarData.LabelOverride = LOCTEXT("CameraTracingMenuLabel", "Camera Tracing");
		Entry.Icon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ClassIcon.CameraComponent");

		for (FToolMenuSection& Sec : Menu->Sections)
		{
			if (Sec.Name.IsEqual(FName("Left")))
			{
				Sec.AddEntry(Entry);
			}
		}
	}
}

void FChaosVDCameraTracesExtension::PostMainTabInitialization(const TSharedRef<SChaosVDMainTab>& InParentTabWidget)
{
	CreateCameraTracesMenu();
}

void FChaosVDCameraTracesExtension::RegisterComponentVisualizers(const TSharedRef<SChaosVDMainTab>& InCVDToolKit)
{
	FChaosVDExtension::RegisterComponentVisualizers(InCVDToolKit);
	
	InCVDToolKit->RegisterComponentVisualizer(UChaosVDCameraDataComponent::StaticClass()->GetFName(), MakeShared<FChaosVDCameraDataComponentVisualizer>());
	TSharedPtr<FChaosVDRecordingControls> a = InCVDToolKit->GetRecordingControls();
	TSharedRef<FChaosVDEngine> EngineInstance = InCVDToolKit->GetChaosVDEngineInstance();

	WeakScenePtr = nullptr;
	if (TSharedPtr<FChaosVDScene> ScenePtr = EngineInstance->GetCurrentScene())
	{
		WeakScenePtr = ScenePtr;
	}

	EditorModeToolsWeakPtr = InCVDToolKit->GetEditorModeManager().AsWeak();

	TSharedPtr<FChaosVDPlaybackController> PlaybackControllerPtr = EngineInstance->GetPlaybackController();

	if (PlaybackControllerPtr)
	{
		if (TSharedPtr<FChaosVDScene> ScenePtr = PlaybackControllerPtr->GetControllerScene().Pin())
		{
			ScenePtr->OnSceneUpdated().AddSP(this, &FChaosVDCameraTracesExtension::HandleCVDSceneUpdated);
			ScenePtr->OnSolverVisibilityUpdated().AddSP(this, &FChaosVDCameraTracesExtension::OnSolverVisibilityUpdated);
		}
	}
}

#undef LOCTEXT_NAMESPACE