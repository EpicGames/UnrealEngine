// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#pragma once
#include "ExtensionsSystem/ChaosVDExtension.h"
#include "Templates/SubclassOf.h"
#include "Templates/SharedPointer.h"
#include "DataWrappers/ChaosVDCameraDataWrapper.h"

class FChaosVDTraceProvider;
class UActorComponent;
class SChaosVDMainTab;
class FChaosVDPlaybackViewportClient;
class FEditorModeTools;
class FChaosVDPlaybackController;
class FChaosVDScene;
struct FChaosVDCameraIdentifier;

class FChaosVDCameraTracesExtension final : public FChaosVDExtension, public TSharedFromThis<FChaosVDCameraTracesExtension>
{
public:
	
	FChaosVDCameraTracesExtension();
	virtual ~FChaosVDCameraTracesExtension() override;

	virtual void RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider) override;
	virtual TConstArrayView<TSubclassOf<UActorComponent>> GetSolverDataComponentsClasses() override;
	virtual void RegisterComponentVisualizers(const TSharedRef<SChaosVDMainTab>& InCVDToolKit) override;
	virtual void PostMainTabInitialization(const TSharedRef<SChaosVDMainTab>& InParentTabWidget) override;
	virtual void HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController) override;

protected:
	enum class ECameraFindFlags : uint8
	{
		Default,
		FindFirst
	};

	TWeakPtr<FChaosVDScene> WeakScenePtr;
	TWeakPtr<FEditorModeTools> EditorModeToolsWeakPtr;

	TOptional<FChaosVDCameraIdentifier> SelectedCamera;
	bool bViewportStartAtCameraTrace = true;
	static const FName ChaosViewportType;
	static const FName CameraSeparatorName;

	void SetTrackedCamera(FChaosVDCameraIdentifier InTargetCamera);
	void ClearTrackedCamera();
	bool FindCameraData(FVector& Position, FQuat& Rotation, float& FieldOfView, ECameraFindFlags CameraFindFlags) const;
	void HandleCVDSceneUpdated();
	void OnSolverVisibilityUpdated(int32 SolverID, bool bNewVisibility);
	void CreateCameraTracesMenu();
	void HandleSettingsChanged(UObject* SettingsObject);
	void ToggleOffObjectTracking();
	FChaosVDPlaybackViewportClient* GetPlaybackViewportClient();

private:
	TArray<TSubclassOf<UActorComponent>> DataComponentsClasses;
};
