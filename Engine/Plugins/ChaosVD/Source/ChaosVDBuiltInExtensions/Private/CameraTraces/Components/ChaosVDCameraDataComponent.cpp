// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCameraDataComponent.h"
#include "DataWrappers/ChaosVDCameraDataWrapper.h"
#include "ChaosVDRecording.h"
#include "Algo/Copy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCameraDataComponent)

UChaosVDCameraDataComponent::UChaosVDCameraDataComponent()
{
	PrimaryComponentTick.bCanEverTick = false;
	
	SetCanEverAffectNavigation(false);
	bNavigationRelevant = false;
}

void UChaosVDCameraDataComponent::UpdateCameraData(TConstArrayView<TSharedPtr<FChaosVDCameraDataWrapper>> CameraDataView)
{
	RecordedCameraData.Reset(CameraDataView.Num());
	Algo::Copy(CameraDataView, RecordedCameraData);
}

void UChaosVDCameraDataComponent::UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData)
{
	if (TSharedPtr<FChaosVDCameraDataContainer> CameraDataContainer = InGameFrameData.GetCustomDataHandler().GetData<FChaosVDCameraDataContainer>())
	{
		const TArray<TSharedPtr<FChaosVDCameraDataWrapper>>& RecordedCameraTraceData = CameraDataContainer->CameraData;
		UpdateCameraData(RecordedCameraTraceData);
	}
}

void UChaosVDCameraDataComponent::ClearData()
{
	RecordedCameraData.Reset();
}
