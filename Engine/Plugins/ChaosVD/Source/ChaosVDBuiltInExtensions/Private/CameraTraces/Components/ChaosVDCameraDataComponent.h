// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChaosVDSolverDataComponent.h"
#include "ChaosVDCameraDataComponent.generated.h"

struct FChaosVDCameraDataWrapper;
struct FChaosVDGameFrameData;

UCLASS()
class UChaosVDCameraDataComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()

public:
	// Sets default values for this component's properties
	UChaosVDCameraDataComponent();

	void UpdateCameraData(TConstArrayView<TSharedPtr<FChaosVDCameraDataWrapper>> CameraDataView);

	TConstArrayView<TSharedPtr<FChaosVDCameraDataWrapper>> GetCameraData() const
	{ 
		return RecordedCameraData;
	}

	virtual void UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData) override;

	virtual void ClearData() override;
protected:
	
	TArray<TSharedPtr<FChaosVDCameraDataWrapper>> RecordedCameraData;
};
