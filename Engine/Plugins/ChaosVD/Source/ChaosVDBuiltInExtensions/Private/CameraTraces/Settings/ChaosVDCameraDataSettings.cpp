// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCameraDataSettings.h"

#include "ChaosVDSettingsManager.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCameraDataSettings)


void UChaosVDCameraDataSettings::SetDataVisualizationFlags(EChaosVDCameraDataVisualizationFlags NewFlags)
{
	if (UChaosVDCameraDataSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCameraDataSettings>())
	{
		Settings->DebugDrawFlags = static_cast<uint32>(NewFlags);
		Settings->BroadcastSettingsChanged();
	}
}

void UChaosVDCameraDataSettings::SetViewportStartAtCameraTrace(bool bInViewportStartAtCameraTrace)
{
	if (UChaosVDCameraDataSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCameraDataSettings>())
	{
		Settings->bViewportStartAtCameraTrace = bInViewportStartAtCameraTrace;
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDCameraDataVisualizationFlags UChaosVDCameraDataSettings::GetDataVisualizationFlags()
{
	if (UChaosVDCameraDataSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCameraDataSettings>())
	{
		return static_cast<EChaosVDCameraDataVisualizationFlags>(Settings->DebugDrawFlags);
	}

	return EChaosVDCameraDataVisualizationFlags::None;
}

bool UChaosVDCameraDataSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, DebugDrawFlags, EChaosVDCameraDataVisualizationFlags::EnableDraw);
}
