// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "ChaosVDCameraDataSettings.generated.h"

/** Set of visualization flags options for camera data */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDCameraDataVisualizationFlags : uint32
{
	None					= 0 UMETA(Hidden),
	EnableDraw				= 1 << 0,
};
ENUM_CLASS_FLAGS(EChaosVDCameraDataVisualizationFlags);

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDCameraDataSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()

public:

	/** If true and if there is an existing camera trace, the initial viewport position will be at the rather than at the world origin*/
	UPROPERTY(config, EditAnywhere, Category = DebugDraw)
	bool bViewportStartAtCameraTrace = true;

	/** The depth priority used for while drawing data. Can be World or Foreground (with this one the shapes will be drawn on top of the geometry and be always visible) */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground;

	/** Length of the camera direction vector when drawn. */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float DirectionVectorScale = 2.0f;

	static void SetDataVisualizationFlags(EChaosVDCameraDataVisualizationFlags NewFlags);
	static EChaosVDCameraDataVisualizationFlags GetDataVisualizationFlags();

	static void SetViewportStartAtCameraTrace(bool bInViewportStartAtCameraTrace);

	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;

private:

	/** Set of flags to enable/disable visualization of debug draw data shapes */
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVDBuiltInExtensions.EChaosVDCameraDataVisualizationFlags"))
	uint32 DebugDrawFlags = static_cast<uint32>(EChaosVDCameraDataVisualizationFlags::None);
};
