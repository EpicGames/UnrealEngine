// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCameraDataProcessor.h"

#include "ChaosVDRecording.h"
#include "ChaosVisualDebugger/ChaosVDMemWriterReader.h"
#include "ChaosVisualDebugger/ChaosVDSerializedNameTable.h"
#include "ChaosVisualDebugger/ChaosVisualDebuggerTrace.h"
#include "DataWrappers/ChaosVDCameraDataWrapper.h"
#include "Trace/ChaosVDTraceProvider.h"

FChaosVDCameraDataProcessor::FChaosVDCameraDataProcessor() : FChaosVDDataProcessorBase(FChaosVDCameraDataWrapper::WrapperTypeName)
{
}

bool FChaosVDCameraDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	const TSharedPtr<FChaosVDCameraDataWrapper> CameraData = MakeShared<FChaosVDCameraDataWrapper>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *CameraData, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		if (TSharedPtr<FChaosVDGameFrameData> CurrentFrameData = ProviderSharedPtr->GetCurrentGameFrame().Pin())
		{
			if (TSharedPtr<FChaosVDCameraDataContainer> CamDataContainer = CurrentFrameData->GetCustomDataHandler().GetOrAddDefaultData<FChaosVDCameraDataContainer>())
			{
				CamDataContainer->CameraData.Add(CameraData);
				CurrentFrameData->MarkDirty();
			}
		}
	}

	return bSuccess;
}

