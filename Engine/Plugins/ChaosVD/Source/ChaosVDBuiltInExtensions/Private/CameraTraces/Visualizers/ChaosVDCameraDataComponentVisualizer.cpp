// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDCameraDataComponentVisualizer.h"

#include "CameraTraces/Components/ChaosVDCameraDataComponent.h"
#include "CameraTraces/Settings/ChaosVDCameraDataSettings.h"
#include "Actors/ChaosVDDataContainerBaseActor.h"
#include "ChaosVDScene.h"
#include "ChaosVDStyle.h"
#include "ChaosVDTabsIDs.h"
#include "SceneView.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDCameraDataComponentVisualizer)

#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

class AChaosVDSolverInfoActor;

bool FChaosVDCameraSelectionHandle::IsSelected()
{
	if (bool bIsPrimaryDataSelected = FChaosVDSolverDataSelectionHandle::IsSelected())
	{
		if (TSharedPtr<FChaosVDSolverDataSelection> OwnerPtr = Owner.Pin())
		{
			TSharedPtr<FChaosVDSolverDataSelectionHandle> CurrentSelectedDataHandle = OwnerPtr->GetCurrentSelectionHandle();

			FChaosVDCameraSelectionContext* CurrentSelectionContext = CurrentSelectedDataHandle->GetContextData<FChaosVDCameraSelectionContext>();
			FChaosVDCameraSelectionContext* HandleSelectionContext = GetContextData<FChaosVDCameraSelectionContext>();

			return CurrentSelectionContext && HandleSelectionContext && (*CurrentSelectionContext) == (*HandleSelectionContext);
		}
	}

	return false;
}

void FChaosVDCameraSelectionHandle::CreateStructViewForDetailsPanelIfNeeded()
{
	if (StructDataView)
	{
		return;
	}

	StructDataView = MakeShared<FChaosVDSelectionMultipleView>();

	if (FChaosVDCameraDataWrapper* CameraData = GetData<FChaosVDCameraDataWrapper>())
	{
		StructDataView->AddData(CameraData);
	}

	if (FChaosVDCameraSelectionContext* SelectionContext = GetContextData<FChaosVDCameraSelectionContext>())
	{
		StructDataView->AddData(const_cast<FChaosVDCameraDataWrapper*>(SelectionContext->CameraData));
	}

	StructDataViewStructOnScope = MakeShared<FStructOnScope>(FChaosVDSelectionMultipleView::StaticStruct(), reinterpret_cast<uint8*>(StructDataView.Get()));
}

TSharedPtr<FStructOnScope> FChaosVDCameraSelectionHandle::GetCustomDataReadOnlyStructViewForDetails()
{
	CreateStructViewForDetailsPanelIfNeeded();

	return StructDataViewStructOnScope;
}

FChaosVDCameraDataComponentVisualizer::FChaosVDCameraDataComponentVisualizer()
{
	RegisterVisualizerMenus();
	InspectorTabID = FChaosVDTabID::DetailsPanel;
}

void FChaosVDCameraDataComponentVisualizer::RegisterVisualizerMenus()
{

	FName MenuSection("CameraDataVisualization.Show");
	FText MenuSectionLabel = LOCTEXT("CameraDataShowMenuLabel", "Camera Data Visualization");
	FText FlagsMenuLabel = LOCTEXT("CameraDataFlagsMenuLabel", "Camera Data Flags");
	FText FlagsMenuTooltip = LOCTEXT("CameraDataFlagsMenuToolTip", "Set of flags to enable/disable visibility of camera data visualization");
	FSlateIcon FlagsMenuIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), "ClassIcon.CameraComponent");

	FText SettingsMenuLabel = LOCTEXT("CameraDataSettingsMenuLabel", "Camera Data Visualization Settings");
	FText SettingsMenuTooltip = LOCTEXT("CameraDataSettingsMenuToolTip", "Options to change how the camera data is debug drawn");
	
	CreateGenericVisualizerMenu<UChaosVDCameraDataSettings, EChaosVDCameraDataVisualizationFlags>(FName("ChaosVDViewportToolbarBase.Show"), MenuSection, MenuSectionLabel, FlagsMenuLabel, FlagsMenuTooltip, FlagsMenuIcon, SettingsMenuLabel, SettingsMenuTooltip);
}

void FChaosVDCameraDataComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDCameraDataComponent* DataComponent = Cast<UChaosVDCameraDataComponent>(Component);
	if (!DataComponent)
	{
		return;
	}
	
	AChaosVDDataContainerBaseActor* DataInfoActor = Cast<AChaosVDDataContainerBaseActor>(Component->GetOwner());
	if (!DataInfoActor)
	{
		return;
	}

	if (!DataInfoActor->IsVisible())
	{
		return;
	}

	const TSharedPtr<FChaosVDScene> CVDScene = DataInfoActor->GetScene().Pin();
	if (!CVDScene)
	{
		return;
	}
	
	FChaosVDCameraVisualizationDataContext VisualizationContext;
	VisualizationContext.CVDScene = CVDScene;
	VisualizationContext.SpaceTransform = DataInfoActor->GetSimulationTransform();
	VisualizationContext.SolverDataSelectionObject = CVDScene->GetSolverDataSelectionObject().Pin();
	VisualizationContext.DataComponent = DataComponent;

	if (const UChaosVDCameraDataSettings* EditorSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDCameraDataSettings>())
	{
		VisualizationContext.VisualizationFlags = static_cast<uint32>(UChaosVDCameraDataSettings::GetDataVisualizationFlags());
		VisualizationContext.DebugDrawSettings = EditorSettings;
		VisualizationContext.DepthPriority = EditorSettings->DepthPriority;
	}

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDCameraDataVisualizationFlags::EnableDraw))
	{
		return;
	}

	TConstArrayView<TSharedPtr<FChaosVDCameraDataWrapper>> CameraData = DataComponent->GetCameraData();

	for (const TSharedPtr<FChaosVDCameraDataWrapper>& CameraDataWrapper : CameraData)
	{
		if (CameraDataWrapper)
		{
			DrawCameraTrace(View, PDI, VisualizationContext, CameraDataWrapper.ToSharedRef());
		}
	}
}

bool FChaosVDCameraDataComponentVisualizer::CanHandleClick(const HChaosVDComponentVisProxy& VisProxy)
{
	return VisProxy.DataSelectionHandle && VisProxy.DataSelectionHandle->IsA<FChaosVDCameraDataWrapper>();
}

void FChaosVDCameraDataComponentVisualizer::DrawCameraTrace(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosVDCameraVisualizationDataContext& VisualizationContext, const TSharedRef<FChaosVDCameraDataWrapper>& CameraData)
{
	const UChaosVDCameraDataSettings* Settings = Cast<const UChaosVDCameraDataSettings>(VisualizationContext.DebugDrawSettings);
	if (!ensure(Settings))
	{
		return;
	}

	const FVector Center = CameraData->Position;
	const FVector DirPos = Center + FRotator(CameraData->Rotation).RotateVector(FVector(42 * Settings->DirectionVectorScale, 0, 0));
	

	FChaosVDDebugDrawUtils::DrawLine(PDI, Center, DirPos, FColor::Red, FText::GetEmpty(), VisualizationContext.DepthPriority, 2.0f);

	constexpr FVector BoxExtents(5.0f, UE::Math::TVectorConstInit{});

	FTransform LocationTransform;
	LocationTransform.SetLocation(Center);

	FChaosVDDebugDrawUtils::DrawBox(PDI, BoxExtents, FColor::Red, LocationTransform, FText::GetEmpty(), VisualizationContext.DepthPriority, 2.0f);
}

#undef LOCTEXT_NAMESPACE
