// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Visualizers/ChaosVDComponentVisualizerBase.h"
#include "ChaosVDSolverDataSelection.h"
#include "DataWrappers/ChaosVDCameraDataWrapper.h"
#include "CameraTraces/Settings/ChaosVDCameraDataSettings.h"

#include "ChaosVDCameraDataComponentVisualizer.generated.h"

class UChaosVDCameraDataComponent;

USTRUCT()
struct FChaosVDCameraSelectionContext
{
	GENERATED_BODY()

	const FChaosVDCameraDataWrapper* CameraData = nullptr;

	bool operator==(const FChaosVDCameraSelectionContext& Other) const
	{
		return CameraData == Other.CameraData;
	}
};

struct FChaosVDCameraSelectionHandle : public FChaosVDSolverDataSelectionHandle
{
	virtual bool IsSelected() override;
	void CreateStructViewForDetailsPanelIfNeeded();

	virtual TSharedPtr<FStructOnScope> GetCustomDataReadOnlyStructViewForDetails() override;

private:
	TSharedPtr<FChaosVDSelectionMultipleView> StructDataView;
	TSharedPtr<FStructOnScope> StructDataViewStructOnScope;
};

/** Visualization context structure specific for camera data visualizations */
struct FChaosVDCameraVisualizationDataContext : public FChaosVDVisualizationContext
{
	TSharedPtr<FChaosVDSolverDataSelectionHandle> DataSelectionHandle = MakeShared<FChaosVDSolverDataSelectionHandle>();

	ESceneDepthPriorityGroup DepthPriority = SDPG_Foreground;

	const UChaosVDCameraDataComponent* DataComponent = nullptr;

	bool IsVisualizationFlagEnabled(EChaosVDCameraDataVisualizationFlags Flag) const
	{
		const EChaosVDCameraDataVisualizationFlags FlagsAsCameraDataVisFlags = static_cast<EChaosVDCameraDataVisualizationFlags>(VisualizationFlags);
		return EnumHasAnyFlags(FlagsAsCameraDataVisFlags, Flag);
	}
};

class FChaosVDCameraDataComponentVisualizer final : public FChaosVDComponentVisualizerBase
{
public:
	FChaosVDCameraDataComponentVisualizer();

	virtual void RegisterVisualizerMenus() override;

	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;

	virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy) override;

protected:

	void DrawCameraTrace(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosVDCameraVisualizationDataContext& VisualizationContext, const TSharedRef<FChaosVDCameraDataWrapper>& CameraData);
};
