// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDBuiltInExtensions.h"

#include "AccelerationStructures/ChaosVDAccelerationStructuresExtension.h"
#include "GenericDebugDraw/ChaosVDGenericDebugDrawExtension.h"
#include "CameraTraces/ChaosVDCameraTracesExtension.h"
#include "ParticleExtraData/ChaosVDParticleExtraDataExtension.h"
#include "PerformanceMetrics/PerformanceMetricsExtension.h"

void FChaosVDBuiltInExtensionsModule::StartupModule()
{
	CreateAndRegisterExtensionInstance<FChaosVDGenericDebugDrawExtension>();
	CreateAndRegisterExtensionInstance<FChaosVDAccelerationStructuresExtension>();
	CreateAndRegisterExtensionInstance<FChaosVDCameraTracesExtension>();
	CreateAndRegisterExtensionInstance<FChaosVDParticleExtraDataExtension>();
	CreateAndRegisterExtensionInstance<FPerformanceMetricsExtension>();
}

void FChaosVDBuiltInExtensionsModule::ShutdownModule()
{
	UnregisterCreatedExtensions();
}

void FChaosVDBuiltInExtensionsModule::UnregisterCreatedExtensions()
{
	for (const TWeakPtr<FChaosVDExtension>& Extension : AvailableExtensions)
	{
		if(const TSharedPtr<FChaosVDExtension>& ExtensionPtr = Extension.Pin())
		{
			FChaosVDExtensionsManager::Get().UnRegisterExtension(ExtensionPtr.ToSharedRef());
		}
	}

	AvailableExtensions.Reset();
}

IMPLEMENT_MODULE(FChaosVDBuiltInExtensionsModule, ChaosVDBuiltInExtensions)
