// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGenericDebugDrawExtension.h"

#include "Components/ChaosVDGenericDebugDrawDataComponent.h"
#include "Trace/DataProcessors/ChaosVDDebugDrawBoxDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDDebugDrawImplicitObjectDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDDebugDrawLineDataProcessor.h"
#include "Trace/DataProcessors/ChaosVDDebugDrawSphereDataProcessor.h"
#include "Visualizers/ChaosVDGenericDebugDrawDataComponentVisualizer.h"

#include "Widgets/SChaosVDMainTab.h"

FChaosVDGenericDebugDrawExtension::FChaosVDGenericDebugDrawExtension() : FChaosVDExtension()
{
	DataComponentsClasses.Add(UChaosVDGenericDebugDrawDataComponent::StaticClass());

	ExtensionName = FName(TEXT("FChaosVDGenericDebugDrawExtension"));
}

FChaosVDGenericDebugDrawExtension::~FChaosVDGenericDebugDrawExtension()
{
	DataComponentsClasses.Reset();
}

void FChaosVDGenericDebugDrawExtension::RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider)
{
	FChaosVDExtension::RegisterDataProcessorsInstancesForProvider(InTraceProvider);
	
	TSharedPtr<FChaosVDDebugDrawBoxDataProcessor> DebugDrawBoxesDataProcessor = MakeShared<FChaosVDDebugDrawBoxDataProcessor>();
    DebugDrawBoxesDataProcessor->SetTraceProvider(InTraceProvider);
    InTraceProvider->RegisterDataProcessor(DebugDrawBoxesDataProcessor);

    TSharedPtr<FChaosVDDebugDrawLineDataProcessor> DebugDrawLinesDataProcessor = MakeShared<FChaosVDDebugDrawLineDataProcessor>();
    DebugDrawLinesDataProcessor->SetTraceProvider(InTraceProvider);
    InTraceProvider->RegisterDataProcessor(DebugDrawLinesDataProcessor);

    TSharedPtr<FChaosVDDebugDrawSphereDataProcessor> DebugDrawSpheresDataProcessor = MakeShared<FChaosVDDebugDrawSphereDataProcessor>();
    DebugDrawSpheresDataProcessor->SetTraceProvider(InTraceProvider);
    InTraceProvider->RegisterDataProcessor(DebugDrawSpheresDataProcessor);

    TSharedPtr<FChaosVDDebugDrawImplicitObjectDataProcessor> DebugDrawImplicitObjectDataProcessor = MakeShared<FChaosVDDebugDrawImplicitObjectDataProcessor>();
    DebugDrawImplicitObjectDataProcessor->SetTraceProvider(InTraceProvider);
    InTraceProvider->RegisterDataProcessor(DebugDrawImplicitObjectDataProcessor);
}

TConstArrayView<TSubclassOf<UActorComponent>> FChaosVDGenericDebugDrawExtension::GetSolverDataComponentsClasses()
{
	return DataComponentsClasses;
}

void FChaosVDGenericDebugDrawExtension::RegisterComponentVisualizers(const TSharedRef<SChaosVDMainTab>& InCVDToolKit)
{
	FChaosVDExtension::RegisterComponentVisualizers(InCVDToolKit);
	
	InCVDToolKit->RegisterComponentVisualizer(UChaosVDGenericDebugDrawDataComponent::StaticClass()->GetFName(), MakeShared<FChaosVDGenericDebugDrawDataComponentVisualizer>());
}
