// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGenericDebugDrawDataComponent.h"

#include "ChaosVDRecording.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDGenericDebugDrawDataComponent)

UChaosVDGenericDebugDrawDataComponent::UChaosVDGenericDebugDrawDataComponent()
{
	SetCanEverAffectNavigation(false);
	bNavigationRelevant = false;
	PrimaryComponentTick.bCanEverTick = false;
}

void UChaosVDGenericDebugDrawDataComponent::UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData)
{
	Super::UpdateFromNewGameFrameData(InGameFrameData);

	if (TSharedPtr<FChaosVDMultiSolverDebugShapeDataContainer> MultiSolverData = InGameFrameData.GetCustomDataHandler().GetData<FChaosVDMultiSolverDebugShapeDataContainer>())
	{
		TSharedPtr<FChaosVDDebugShapeDataContainer>* SolverData = MultiSolverData->DataBySolverID.Find(SolverID);
		CurrentGameFrameDebugDrawData = SolverData ? *SolverData : nullptr;
	}
	else
	{
		CurrentGameFrameDebugDrawData = nullptr;
	}
}

void UChaosVDGenericDebugDrawDataComponent::UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData)
{
	CurrentSolverFrameDebugDrawData = InSolverFrameData.GetCustomData().GetData<FChaosVDDebugShapeDataContainer>();
}

void UChaosVDGenericDebugDrawDataComponent::UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData)
{
	CurrentSolverFrameDebugDrawData = InSolverFrameStageData.GetCustomDataHandler().GetData<FChaosVDDebugShapeDataContainer>();
}

void UChaosVDGenericDebugDrawDataComponent::ClearData()
{
	CurrentSolverStageDebugDrawData.Reset();
	CurrentSolverFrameDebugDrawData.Reset();
	CurrentGameFrameDebugDrawData.Reset();
}

TConstArrayView<TSharedPtr<FChaosVDDebugDrawBoxDataWrapper>> UChaosVDGenericDebugDrawDataComponent::GetDebugDrawBoxesDataView(EChaosVDDrawDataContainerSource Source) const
{
	if (TSharedPtr<FChaosVDDebugShapeDataContainer> DataContainer = GetShapeDataContainer(Source))
	{
		return DataContainer->RecordedDebugDrawBoxes;
	}

	return TConstArrayView<TSharedPtr<FChaosVDDebugDrawBoxDataWrapper>>();
}

TConstArrayView<TSharedPtr<FChaosVDDebugDrawLineDataWrapper>> UChaosVDGenericDebugDrawDataComponent::GetDebugDrawLinesDataView( EChaosVDDrawDataContainerSource Source) const
{
	if (TSharedPtr<FChaosVDDebugShapeDataContainer> DataContainer = GetShapeDataContainer(Source))
	{
		return DataContainer->RecordedDebugDrawLines;
	}

	return TConstArrayView<TSharedPtr<FChaosVDDebugDrawLineDataWrapper>>();
}

TConstArrayView<TSharedPtr<FChaosVDDebugDrawSphereDataWrapper>> UChaosVDGenericDebugDrawDataComponent::GetDebugDrawSpheresDataView( EChaosVDDrawDataContainerSource Source) const
{
	if (TSharedPtr<FChaosVDDebugShapeDataContainer> DataContainer = GetShapeDataContainer(Source))
	{
		return DataContainer->RecordedDebugDrawSpheres;
	}

	return TConstArrayView<TSharedPtr<FChaosVDDebugDrawSphereDataWrapper>>();
}

TConstArrayView<TSharedPtr<FChaosVDDebugDrawImplicitObjectDataWrapper>> UChaosVDGenericDebugDrawDataComponent::GetDebugDrawImplicitObjectsDataView( EChaosVDDrawDataContainerSource Source) const
{
	if (TSharedPtr<FChaosVDDebugShapeDataContainer> DataContainer = GetShapeDataContainer(Source))
	{
		return DataContainer->RecordedDebugDrawImplicitObjects;
	}

	return TConstArrayView<TSharedPtr<FChaosVDDebugDrawImplicitObjectDataWrapper>>();
}

TSharedPtr<FChaosVDDebugShapeDataContainer> UChaosVDGenericDebugDrawDataComponent::GetShapeDataContainer(EChaosVDDrawDataContainerSource Source) const
{
	switch (Source)
	{
		case EChaosVDDrawDataContainerSource::SolverFrame:
			return CurrentSolverFrameDebugDrawData;
		case EChaosVDDrawDataContainerSource::SolverStage:
			return CurrentSolverStageDebugDrawData;
		case EChaosVDDrawDataContainerSource::GameFrame:
			return CurrentGameFrameDebugDrawData;
		default:
			return nullptr;
	}
}
