// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Components/ChaosVDSolverDataComponent.h"
#include "Components/ActorComponent.h"
#include "DataWrappers/ChaosVDDebugShapeDataWrapper.h"
#include "ChaosVDGenericDebugDrawDataComponent.generated.h"

class UChaosVDGenericDebugDrawDataComponent;
struct FChaosVDGameFrameData;

enum class EChaosVDDrawDataContainerSource : uint8
{
	SolverFrame,
	SolverStage,
	GameFrame
};

UCLASS()
class UChaosVDGenericDebugDrawDataComponent : public UChaosVDSolverDataComponent
{
	GENERATED_BODY()
public:
	UChaosVDGenericDebugDrawDataComponent();

	virtual void UpdateFromNewGameFrameData(const FChaosVDGameFrameData& InGameFrameData) override;

	virtual void UpdateFromSolverFrameData(const FChaosVDSolverFrameData& InSolverFrameData) override;

	virtual void UpdateFromNewSolverStageData(const FChaosVDSolverFrameData& InSolverFrameData, const FChaosVDFrameStageData& InSolverFrameStageData) override;

	virtual void ClearData() override;

	TConstArrayView<TSharedPtr<FChaosVDDebugDrawBoxDataWrapper>> GetDebugDrawBoxesDataView(EChaosVDDrawDataContainerSource Source) const;
	TConstArrayView<TSharedPtr<FChaosVDDebugDrawLineDataWrapper>> GetDebugDrawLinesDataView(EChaosVDDrawDataContainerSource Source) const;
	TConstArrayView<TSharedPtr<FChaosVDDebugDrawSphereDataWrapper>> GetDebugDrawSpheresDataView(EChaosVDDrawDataContainerSource Source) const;
	TConstArrayView<TSharedPtr<FChaosVDDebugDrawImplicitObjectDataWrapper>> GetDebugDrawImplicitObjectsDataView(EChaosVDDrawDataContainerSource Source) const;

	TSharedPtr<FChaosVDDebugShapeDataContainer> GetShapeDataContainer(EChaosVDDrawDataContainerSource Source) const;

private:

	TSharedPtr<FChaosVDDebugShapeDataContainer> CurrentSolverStageDebugDrawData;
	TSharedPtr<FChaosVDDebugShapeDataContainer> CurrentSolverFrameDebugDrawData;
	TSharedPtr<FChaosVDDebugShapeDataContainer> CurrentGameFrameDebugDrawData;
};
