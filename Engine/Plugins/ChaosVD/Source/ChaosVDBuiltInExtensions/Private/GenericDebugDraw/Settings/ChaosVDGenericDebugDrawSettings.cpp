// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGenericDebugDrawSettings.h"

#include "ChaosVDSettingsManager.h"
#include "Utils/ChaosVDUserInterfaceUtils.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDGenericDebugDrawSettings)


void UChaosVDGenericDebugDrawSettings::SetDataVisualizationFlags(EChaosVDGenericDebugDrawVisualizationFlags NewFlags)
{
	if (UChaosVDGenericDebugDrawSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGenericDebugDrawSettings>())
	{
		Settings->DebugDrawFlags = static_cast<uint32>(NewFlags);
		Settings->BroadcastSettingsChanged();
	}
}

EChaosVDGenericDebugDrawVisualizationFlags UChaosVDGenericDebugDrawSettings::GetDataVisualizationFlags()
{
	if (UChaosVDGenericDebugDrawSettings* Settings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGenericDebugDrawSettings>())
	{
		return static_cast<EChaosVDGenericDebugDrawVisualizationFlags>(Settings->DebugDrawFlags);
	}

	return EChaosVDGenericDebugDrawVisualizationFlags::None;
}

bool UChaosVDGenericDebugDrawSettings::CanVisualizationFlagBeChangedByUI(uint32 Flag)
{
	return Chaos::VisualDebugger::Utils::ShouldVisFlagBeEnabledInUI(Flag, DebugDrawFlags, EChaosVDGenericDebugDrawVisualizationFlags::EnableDraw);
}
