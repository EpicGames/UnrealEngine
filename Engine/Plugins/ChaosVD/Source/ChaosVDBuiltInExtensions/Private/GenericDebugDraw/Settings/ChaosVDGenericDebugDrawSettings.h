// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "ChaosVDGenericDebugDrawSettings.generated.h"

/** Set of visualization flags options for Scene Queries */
UENUM(meta = (Bitflags, UseEnumValuesAsMaskValuesInEditor = "true"))
enum class EChaosVDGenericDebugDrawVisualizationFlags : uint32
{
	None					= 0 UMETA(Hidden),
	EnableDraw				= 1 << 0,
	DrawBoxes				= 1 << 1,
	DrawLines				= 1 << 2,
	DrawSpheres				= 1 << 3,
	DrawImplicitObjects		= 1 << 4,
};
ENUM_CLASS_FLAGS(EChaosVDGenericDebugDrawVisualizationFlags);

UCLASS(config=ChaosVD, PerObjectConfig)
class UChaosVDGenericDebugDrawSettings : public UChaosVDVisualizationSettingsObjectBase
{
	GENERATED_BODY()

public:

	/** If true, any debug draw text available will be drawn */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	bool bShowDebugText = false;

	/** The depth priority used for while drawing data. Can be World or Foreground (with this one the shapes will be drawn on top of the geometry and be always visible) */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	TEnumAsByte<ESceneDepthPriorityGroup> DepthPriority = ESceneDepthPriorityGroup::SDPG_Foreground;

	/** Thickness to apply to any debug draw shape controlled by this setting. */
	UPROPERTY(config, EditAnywhere, Category=DebugDraw)
	float BaseThickness = 2.0f;

	static void SetDataVisualizationFlags(EChaosVDGenericDebugDrawVisualizationFlags NewFlags);
	static EChaosVDGenericDebugDrawVisualizationFlags GetDataVisualizationFlags();

	virtual bool CanVisualizationFlagBeChangedByUI(uint32 Flag) override;

private:

	/** Set of flags to enable/disable visualization of debug draw data shapes */
	UPROPERTY(config, meta = (Bitmask, BitmaskEnum = "/Script/ChaosVD.EChaosVDGenericDebugDrawVisualizationFlags"))
	uint32 DebugDrawFlags = static_cast<uint32>(EChaosVDGenericDebugDrawVisualizationFlags::DrawBoxes | EChaosVDGenericDebugDrawVisualizationFlags::DrawLines | EChaosVDGenericDebugDrawVisualizationFlags::DrawSpheres);
};
