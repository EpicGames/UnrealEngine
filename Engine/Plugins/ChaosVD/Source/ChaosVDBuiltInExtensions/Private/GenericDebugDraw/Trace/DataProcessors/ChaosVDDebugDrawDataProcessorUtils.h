// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "ChaosVDRecording.h"
#include "Templates/SharedPointer.h"
#include "Trace/ChaosVDTraceProvider.h"

namespace Chaos::VisualDebugger::Utils
{
	enum class EShapeDataContainerAccessorFlags : uint8
	{
		None = 0,
		/** If the container lives in a game thread frame, it will mark it as dirty */
		MarkFrameDirty = 1 << 0
	};
	ENUM_CLASS_FLAGS(EShapeDataContainerAccessorFlags)

	template<typename DataType>
	TSharedPtr<FChaosVDDebugShapeDataContainer> GetShapeDataContainer(const TSharedRef<DataType>& InData, const TSharedRef<FChaosVDTraceProvider>& TraceProvider, EShapeDataContainerAccessorFlags Flags)
	{
		EChaosVDParticleContext TracedThreadContext = InData->ThreadContext;
		if (InData->SolverID == INDEX_NONE)
		{
			TracedThreadContext = EChaosVDParticleContext::GameThread;
		}

		switch (TracedThreadContext)
		{
			case EChaosVDParticleContext::GameThread:
				{
					if (TSharedPtr<FChaosVDGameFrameData> CurrentFrameData = TraceProvider->GetCurrentGameFrame().Pin())
					{
						if (TSharedPtr<FChaosVDMultiSolverDebugShapeDataContainer> MultiSolverData = CurrentFrameData->GetCustomDataHandler().GetOrAddDefaultData<FChaosVDMultiSolverDebugShapeDataContainer>())
						{
							TSharedPtr<FChaosVDDebugShapeDataContainer>& SolverData = MultiSolverData->DataBySolverID.FindOrAdd(InData->SolverID);

							if (!SolverData)
							{
								SolverData = MakeShared<FChaosVDDebugShapeDataContainer>();
							}

							if (EnumHasAnyFlags(Flags, EShapeDataContainerAccessorFlags::MarkFrameDirty))
							{
								CurrentFrameData->MarkDirty();
							}

							return SolverData;
						}

						return nullptr;
					}
					break;
				}
			case EChaosVDParticleContext::PhysicsThread:
				{
					EChaosVDSolverStageAccessorFlags StageAccessorFlags = EChaosVDSolverStageAccessorFlags::None;

					if (FChaosVDFrameStageData* CurrentSolverStage = TraceProvider->GetCurrentSolverStageDataForCurrentFrame(InData->SolverID, StageAccessorFlags))
					{
						return CurrentSolverStage->GetCustomDataHandler().GetOrAddDefaultData<FChaosVDDebugShapeDataContainer>();
					}

					if (FChaosVDSolverFrameData* CurrentSolverFrameData = TraceProvider->GetCurrentSolverFrame(InData->SolverID))
					{
						return CurrentSolverFrameData->GetCustomData().GetOrAddDefaultData<FChaosVDDebugShapeDataContainer>();
					}
				}
			case EChaosVDParticleContext::Invalid:
			default:
				return nullptr;
		}

		return nullptr;
	}

	template<typename DataType>
	void RemapDebugDrawShapeDataSolverID(const TSharedRef<DataType>& InData, const TSharedRef<FChaosVDTraceProvider>& TraceProvider)
	{
		if (InData->SolverID != INDEX_NONE)
		{
			InData->SolverID = TraceProvider->GetRemappedSolverID(InData->SolverID);
		}
	}
}
