// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDDebugDrawImplicitObjectDataProcessor.h"

#include "ChaosVDDebugDrawDataProcessorUtils.h"
#include "DataWrappers/ChaosVDDebugShapeDataWrapper.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"

FChaosVDDebugDrawImplicitObjectDataProcessor::FChaosVDDebugDrawImplicitObjectDataProcessor() : FChaosVDDataProcessorBase(FChaosVDDebugDrawImplicitObjectDataWrapper::WrapperTypeName)
{
}

bool FChaosVDDebugDrawImplicitObjectDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	const TSharedPtr<FChaosVDDebugDrawImplicitObjectDataWrapper> DebugDrawData = MakeShared<FChaosVDDebugDrawImplicitObjectDataWrapper>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *DebugDrawData, ProviderSharedPtr.ToSharedRef());

	if (bSuccess)
	{
		using namespace Chaos::VisualDebugger::Utils;

		TSharedRef<FChaosVDTraceProvider> ProviderAsSharedRef = ProviderSharedPtr.ToSharedRef();
		TSharedRef<FChaosVDDebugDrawImplicitObjectDataWrapper> DebugDrawDataAsSharedRef = DebugDrawData.ToSharedRef();
		RemapDebugDrawShapeDataSolverID(DebugDrawDataAsSharedRef, ProviderAsSharedRef);
	
		if (TSharedPtr<FChaosVDDebugShapeDataContainer> DebugDrawShapeData = GetShapeDataContainer(DebugDrawDataAsSharedRef, ProviderAsSharedRef, EShapeDataContainerAccessorFlags::MarkFrameDirty))
		{
			DebugDrawShapeData->RecordedDebugDrawImplicitObjects.Add(DebugDrawData);
		}
	}

	return bSuccess;
}
