// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDDebugDrawLineDataProcessor.h"

#include "ChaosVDDebugDrawDataProcessorUtils.h"
#include "DataWrappers/ChaosVDDebugShapeDataWrapper.h"
#include "Visualizers/ChaosVDDebugDrawUtils.h"


FChaosVDDebugDrawLineDataProcessor::FChaosVDDebugDrawLineDataProcessor() : FChaosVDDataProcessorBase(FChaosVDDebugDrawLineDataWrapper::WrapperTypeName)
{
}

bool FChaosVDDebugDrawLineDataProcessor::ProcessRawData(const TArray<uint8>& InData)
{
	FChaosVDDataProcessorBase::ProcessRawData(InData);

	const TSharedPtr<FChaosVDTraceProvider> ProviderSharedPtr = TraceProvider.Pin();
	if (!ensure(ProviderSharedPtr.IsValid()))
	{
		return false;
	}

	const TSharedPtr<FChaosVDDebugDrawLineDataWrapper> DebugDrawData = MakeShared<FChaosVDDebugDrawLineDataWrapper>();
	const bool bSuccess = Chaos::VisualDebugger::ReadDataFromBuffer(InData, *DebugDrawData, ProviderSharedPtr.ToSharedRef());
	
	if (bSuccess)
	{
		using namespace Chaos::VisualDebugger::Utils;

		TSharedRef<FChaosVDTraceProvider> ProviderAsSharedRef = ProviderSharedPtr.ToSharedRef();
		TSharedRef<FChaosVDDebugDrawLineDataWrapper> DebugDrawDataAsSharedRef = DebugDrawData.ToSharedRef();
		RemapDebugDrawShapeDataSolverID(DebugDrawDataAsSharedRef, ProviderAsSharedRef);
	
		if (TSharedPtr<FChaosVDDebugShapeDataContainer> DebugDrawShapeData = GetShapeDataContainer(DebugDrawDataAsSharedRef, ProviderAsSharedRef, EShapeDataContainerAccessorFlags::MarkFrameDirty))
		{
			DebugDrawShapeData->RecordedDebugDrawLines.Add(DebugDrawData);
		}
	}

	return bSuccess;
}