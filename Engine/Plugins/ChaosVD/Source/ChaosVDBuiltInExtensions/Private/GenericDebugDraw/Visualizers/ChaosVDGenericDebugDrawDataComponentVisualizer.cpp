// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDGenericDebugDrawDataComponentVisualizer.h"

#include "Visualizers/ChaosVDDebugDrawUtils.h"
#include "ChaosVDScene.h"
#include "GenericDebugDraw/Components/ChaosVDGenericDebugDrawDataComponent.h"
#include "SceneView.h"
#include "Actors/ChaosVDDataContainerBaseActor.h"


#define LOCTEXT_NAMESPACE "ChaosVisualDebugger"

FChaosVDGenericDebugDrawDataComponentVisualizer::FChaosVDGenericDebugDrawDataComponentVisualizer()
{
	FChaosVDGenericDebugDrawDataComponentVisualizer::RegisterVisualizerMenus();
}

void FChaosVDGenericDebugDrawDataComponentVisualizer::RegisterVisualizerMenus()
{
	FName MenuSection("GenericDebugDrawDataVisualization.Show");
	FText MenuSectionLabel = LOCTEXT("GenericDebugDrawDataShowMenuLabel", "Generic Debug Draw Data Visualization");
	FText FlagsMenuLabel = LOCTEXT("GenericDebugDrawDataFlagsMenuLabel", "Generic Debug Draw Data Flags");
	FText FlagsMenuTooltip = LOCTEXT("GenericDebugDrawDataFlagsMenuToolTip", "Set of flags to enable/disable visibility of specific types of Debug Draw Data that are not solver related");
	FSlateIcon FlagsMenuIcon = FSlateIcon(FAppStyle::Get().GetStyleSetName(), TEXT("EditorViewport.CollisionVisibility"));

	FText SettingsMenuLabel = LOCTEXT("GenericDebugDrawDataSettingsMenuLabel", "Generic Debug Draw Data Visualization Settings");
	FText SettingsMenuTooltip = LOCTEXT("GenericDebugDrawDataSettingsMenuToolTip", "Options to change how the recorded Generic Debug Draw Data is debug drawn");
	
	CreateGenericVisualizerMenu<UChaosVDGenericDebugDrawSettings, EChaosVDGenericDebugDrawVisualizationFlags>(FName("ChaosVDViewportToolbarBase.Show"), MenuSection, MenuSectionLabel, FlagsMenuLabel, FlagsMenuTooltip, FlagsMenuIcon, SettingsMenuLabel, SettingsMenuTooltip);
}

void FChaosVDGenericDebugDrawDataComponentVisualizer::DrawData(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext)
{
	DrawBoxes(PDI, View, InVisualizationContext);

	DrawLines(PDI, View, InVisualizationContext);

	DrawSpheres(PDI, View, InVisualizationContext);

	DrawImplicitObjects(PDI, View, InVisualizationContext);
}

void FChaosVDGenericDebugDrawDataComponentVisualizer::DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI)
{
	const UChaosVDGenericDebugDrawDataComponent* DebugDrawDataComponent = Cast<UChaosVDGenericDebugDrawDataComponent>(Component);

	if (!DebugDrawDataComponent)
	{
		return;
	}

		
	AChaosVDDataContainerBaseActor* InfoActor = Cast<AChaosVDDataContainerBaseActor>(Component->GetOwner());
	if (!InfoActor)
	{
		return;
	}

	if (!InfoActor->IsVisible())
	{
		return;
	}

	const TSharedPtr<FChaosVDScene> CVDScene = InfoActor->GetScene().Pin();
	
	FChaosVDGenericDebugDrawDataVisualizationSettings VisualizationContext;
	VisualizationContext.CVDScene = CVDScene;
	VisualizationContext.SolverDataSelectionObject = CVDScene->GetSolverDataSelectionObject().Pin();
	VisualizationContext.DataComponent = DebugDrawDataComponent;

	if (const UChaosVDGenericDebugDrawSettings* EditorSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDGenericDebugDrawSettings>())
	{
		VisualizationContext.VisualizationFlags = static_cast<uint32>(UChaosVDGenericDebugDrawSettings::GetDataVisualizationFlags());
		VisualizationContext.DebugDrawSettings = EditorSettings;
		VisualizationContext.DepthPriority = EditorSettings->DepthPriority;
		VisualizationContext.Thickness = EditorSettings->BaseThickness;
		VisualizationContext.bShowDebugText = EditorSettings->bShowDebugText;
	}

	if (!VisualizationContext.IsVisualizationFlagEnabled(EChaosVDGenericDebugDrawVisualizationFlags::EnableDraw))
	{
		return;
	}

	VisualizationContext.DataSource = EChaosVDDrawDataContainerSource::GameFrame;
	DrawData(View, PDI, VisualizationContext);

	VisualizationContext.DataSource = EChaosVDDrawDataContainerSource::SolverFrame;
	DrawData(View, PDI, VisualizationContext);

	VisualizationContext.DataSource = EChaosVDDrawDataContainerSource::SolverStage;
	DrawData(View, PDI, VisualizationContext);
}

void FChaosVDGenericDebugDrawDataComponentVisualizer::DrawBoxes(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext)
{
	if (InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDGenericDebugDrawVisualizationFlags::DrawBoxes))
	{
		TConstArrayView<TSharedPtr<FChaosVDDebugDrawBoxDataWrapper>> DebugDrawBoxesView = InVisualizationContext.DataComponent->GetDebugDrawBoxesDataView(InVisualizationContext.DataSource);
		for (const TSharedPtr<FChaosVDDebugDrawBoxDataWrapper>& DebugDrawBox : DebugDrawBoxesView)
		{
			if (DebugDrawBox)
			{
				FVector BoxCenter = DebugDrawBox->Box.GetCenter();
				FVector BoxExtent = DebugDrawBox->Box.GetExtent();

				if (!View->ViewFrustum.IntersectBox(BoxCenter, BoxExtent))
				{
					continue;
				}
				
				const FText& DebugText = InVisualizationContext.bShowDebugText ? FText::FromName(DebugDrawBox->Tag) : FText::GetEmpty();
	
				FTransform BoxTransform(BoxCenter);
				FChaosVDDebugDrawUtils::DrawBox(PDI, BoxExtent, DebugDrawBox->Color, BoxTransform, DebugText, InVisualizationContext.DepthPriority, InVisualizationContext.Thickness);			
			}
		}
	}
}

void FChaosVDGenericDebugDrawDataComponentVisualizer::DrawLines(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext)
{
	if (InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDGenericDebugDrawVisualizationFlags::DrawLines))
	{
		TConstArrayView<TSharedPtr<FChaosVDDebugDrawLineDataWrapper>> DebugDrawLinesView = InVisualizationContext.DataComponent->GetDebugDrawLinesDataView(InVisualizationContext.DataSource);
		for (const TSharedPtr<FChaosVDDebugDrawLineDataWrapper>& DebugDrawLine : DebugDrawLinesView)
		{
			if (DebugDrawLine)
			{
				if (!View->ViewFrustum.IntersectLineSegment(DebugDrawLine->StartLocation, DebugDrawLine->EndLocation))
				{
					continue;
				}

				const FText& DebugText = InVisualizationContext.bShowDebugText ? FText::FromName(DebugDrawLine->Tag) : FText::GetEmpty();

				if (DebugDrawLine->bIsArrow)
				{
					FChaosVDDebugDrawUtils::DrawArrowVector(PDI, DebugDrawLine->StartLocation, DebugDrawLine->EndLocation, DebugText, DebugDrawLine->Color, InVisualizationContext.DepthPriority, InVisualizationContext.Thickness);
				}
				else
				{
					FChaosVDDebugDrawUtils::DrawLine(PDI, DebugDrawLine->StartLocation, DebugDrawLine->EndLocation, DebugDrawLine->Color, DebugText, InVisualizationContext.DepthPriority, InVisualizationContext.Thickness);
				}			
			}
		}
	}
}

void FChaosVDGenericDebugDrawDataComponentVisualizer::DrawSpheres(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext)
{
	if (InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDGenericDebugDrawVisualizationFlags::DrawSpheres))
	{
		TConstArrayView<TSharedPtr<FChaosVDDebugDrawSphereDataWrapper>> DebugDrawSpheres = InVisualizationContext.DataComponent->GetDebugDrawSpheresDataView(InVisualizationContext.DataSource);
		for (const TSharedPtr<FChaosVDDebugDrawSphereDataWrapper>& DebugDrawSphere : DebugDrawSpheres)
		{
			if (DebugDrawSphere)
			{
				if (!View->ViewFrustum.IntersectSphere(DebugDrawSphere->Origin,  DebugDrawSphere->Radius))
				{
					continue;
				}

				const FText& DebugText = InVisualizationContext.bShowDebugText ? FText::FromName(DebugDrawSphere->Tag) : FText::GetEmpty();
				constexpr int32 Segments = 12;

				FChaosVDDebugDrawUtils::DrawSphere(PDI, DebugDrawSphere->Origin, DebugDrawSphere->Radius, Segments, DebugDrawSphere->Color, DebugText, InVisualizationContext.DepthPriority, InVisualizationContext.Thickness);			
			}
		}
	}
}

void FChaosVDGenericDebugDrawDataComponentVisualizer::DrawImplicitObjects(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext)
{
	using namespace Chaos;

	TSharedPtr<FChaosVDScene> CVDScene = InVisualizationContext.CVDScene.Pin();

	if (!CVDScene)
	{
		return;
	}
	
	if (InVisualizationContext.IsVisualizationFlagEnabled(EChaosVDGenericDebugDrawVisualizationFlags::DrawImplicitObjects))
	{
		TConstArrayView<TSharedPtr<FChaosVDDebugDrawImplicitObjectDataWrapper>> DebugDrawImplicitObjects = InVisualizationContext.DataComponent->GetDebugDrawImplicitObjectsDataView(InVisualizationContext.DataSource);
		for (const TSharedPtr<FChaosVDDebugDrawImplicitObjectDataWrapper>& DebugDrawImplicitObjectData : DebugDrawImplicitObjects)
		{
			if (DebugDrawImplicitObjectData)
			{
				if (FConstImplicitObjectPtr LoadedGeometry = CVDScene->GetUpdatedGeometry(DebugDrawImplicitObjectData->ImplicitObjectHash))
				{
					FAABB3 Bounds = LoadedGeometry->CalculateTransformedBounds(DebugDrawImplicitObjectData->ParentTransform);
					if (!View->ViewFrustum.IntersectBox(Bounds.Center(),  Bounds.Extents()))
					{
						continue;
					}

					if (InVisualizationContext.bShowDebugText)
					{
						FChaosVDDebugDrawUtils::DrawText(FText::FromName(DebugDrawImplicitObjectData->Tag), DebugDrawImplicitObjectData->ParentTransform.GetLocation(), DebugDrawImplicitObjectData->Color, EChaosVDDebugDrawTextLocationMode::World);
					}				
				
					FChaosVDDebugDrawUtils::DrawImplicitObject(PDI, CVDScene->GetGeometryGenerator().Pin(), LoadedGeometry, DebugDrawImplicitObjectData->ParentTransform, DebugDrawImplicitObjectData->Color, FText::GetEmpty(), InVisualizationContext.DepthPriority, InVisualizationContext.Thickness);
				}
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
