// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "GenericDebugDraw/Components/ChaosVDGenericDebugDrawDataComponent.h"
#include "Visualizers/ChaosVDComponentVisualizerBase.h"
#include "GenericDebugDraw/Settings/ChaosVDGenericDebugDrawSettings.h"

enum class EChaosVDDrawDataContainerSource : uint8;
struct FChaosVDDebugDrawImplicitObjectDataWrapper;
class UChaosVDGenericDebugDrawDataComponent;
class UChaosVDGenericDebugDrawSettings;

/** Visualization context structure specific for acceleration structure visualizations */
struct FChaosVDGenericDebugDrawDataVisualizationSettings : public FChaosVDVisualizationContext
{
	ESceneDepthPriorityGroup DepthPriority = SDPG_Foreground;

	float Thickness = 0.0f;

	bool bShowDebugText = false;

	EChaosVDDrawDataContainerSource DataSource = EChaosVDDrawDataContainerSource::GameFrame;

	const UChaosVDGenericDebugDrawDataComponent* DataComponent = nullptr;

	bool IsVisualizationFlagEnabled(EChaosVDGenericDebugDrawVisualizationFlags Flag) const
	{
		const EChaosVDGenericDebugDrawVisualizationFlags FlagsAsGenericDebugDrawVisualizationVisFlags = static_cast<EChaosVDGenericDebugDrawVisualizationFlags>(VisualizationFlags);
		return EnumHasAnyFlags(FlagsAsGenericDebugDrawVisualizationVisFlags, Flag);
	}	
};

class FChaosVDGenericDebugDrawDataComponentVisualizer final : public FChaosVDComponentVisualizerBase
{
public:
	
	FChaosVDGenericDebugDrawDataComponentVisualizer();

	virtual void RegisterVisualizerMenus() override;

	virtual bool CanHandleClick(const HChaosVDComponentVisProxy& VisProxy) override { return false;}

	virtual void DrawVisualization(const UActorComponent* Component, const FSceneView* View, FPrimitiveDrawInterface* PDI) override;
	
	void DrawData(const FSceneView* View, FPrimitiveDrawInterface* PDI, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext);

	void DrawBoxes(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext);
    void DrawLines(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext);
    void DrawSpheres(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext);
    void DrawImplicitObjects(FPrimitiveDrawInterface* PDI, const FSceneView* View, const FChaosVDGenericDebugDrawDataVisualizationSettings& InVisualizationContext);
};
