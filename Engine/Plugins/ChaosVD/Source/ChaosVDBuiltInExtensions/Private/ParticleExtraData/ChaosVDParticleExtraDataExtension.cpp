// Copyright Epic Games, Inc. All Rights Reserved.

#include "ParticleExtraData/ChaosVDParticleExtraDataExtension.h"

#include "Trace/DataProcessors/ChaosVDParticleExtraDataProcessor.h"
#include "Trace/ChaosVDTraceProvider.h"

FChaosVDParticleExtraDataExtension::FChaosVDParticleExtraDataExtension()
{
	ExtensionName = FName(TEXT("FChaosVDParticleExtraDataExtension"));
}

void FChaosVDParticleExtraDataExtension::RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider)
{
	FChaosVDExtension::RegisterDataProcessorsInstancesForProvider(InTraceProvider);

	TSharedPtr<FChaosVDParticleExtraDataProcessor> Processor = MakeShared<FChaosVDParticleExtraDataProcessor>();
	Processor->SetTraceProvider(InTraceProvider);
	InTraceProvider->RegisterDataProcessor(Processor);
}
