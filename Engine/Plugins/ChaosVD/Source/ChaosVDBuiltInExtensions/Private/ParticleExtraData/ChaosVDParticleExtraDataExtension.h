// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ExtensionsSystem/ChaosVDExtension.h"

/**
 * Built-in CVD extension that registers the FChaosVDParticleExtraDataProcessor.
 *
 * UChaosVDParticleExtraDataComponent is registered directly as a default subobject
 * on AChaosVDSolverInfoActor, so this extension only handles processor registration.
 */
class FChaosVDParticleExtraDataExtension final : public FChaosVDExtension
{
public:
	FChaosVDParticleExtraDataExtension();
	virtual ~FChaosVDParticleExtraDataExtension() override = default;

	virtual void RegisterDataProcessorsInstancesForProvider(const TSharedRef<FChaosVDTraceProvider>& InTraceProvider) override;
};
