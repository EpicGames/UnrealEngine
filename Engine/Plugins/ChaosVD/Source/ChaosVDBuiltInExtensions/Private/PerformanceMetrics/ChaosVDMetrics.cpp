// Copyright Epic Games, Inc. All Rights Reserved.

#include "PerformanceMetrics/ChaosVDMetrics.h"
#include "Chaos/Convex.h"
#include "Chaos/HeightField.h"
#include "Chaos/ChaosArchive.h"
#include "ChaosVDScene.h"
#include "Serialization/MemoryWriter.h"
#include "Serialization/ArchiveCountMem.h"

double FChaosVDMetricsSettings::GetMinThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity) const
{
	switch (Complexity)
	{
		case ChaosVDCollisionComplexityFilteringOptions::Simple:
			return SimpleMinThreshold;
		case ChaosVDCollisionComplexityFilteringOptions::Complex:
			return ComplexMinThreshold;
		case ChaosVDCollisionComplexityFilteringOptions::All:
			return AllMinThreshold;
	}
	return 0;
}

void FChaosVDMetricsSettings::SetMinThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, double InMin)
{
	switch (Complexity)
	{
		case ChaosVDCollisionComplexityFilteringOptions::Simple:
			SimpleMinThreshold = InMin;
		break;
		case ChaosVDCollisionComplexityFilteringOptions::Complex:
			ComplexMinThreshold = InMin;
		break;
		case ChaosVDCollisionComplexityFilteringOptions::All:
			AllMinThreshold = InMin;
		break;
	}
}

double FChaosVDMetricsSettings::GetMaxThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity) const
{
	switch (Complexity)
	{
		case ChaosVDCollisionComplexityFilteringOptions::Simple:
			return SimpleMaxThreshold;
		case ChaosVDCollisionComplexityFilteringOptions::Complex:
			return ComplexMaxThreshold;
		case ChaosVDCollisionComplexityFilteringOptions::All:
			return AllMaxThreshold;
	}
	return 0;
}

void FChaosVDMetricsSettings::SetMaxThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, double InMax)
{
	switch (Complexity)
	{
		case ChaosVDCollisionComplexityFilteringOptions::Simple:
			SimpleMaxThreshold = InMax;
		break;
		case ChaosVDCollisionComplexityFilteringOptions::Complex:
			ComplexMaxThreshold = InMax;
		break;
		case ChaosVDCollisionComplexityFilteringOptions::All:
			AllMaxThreshold = InMax;
		break;
	}
}

double FParticleMetricEntry::GetVolumeSafe()
{
	const FVector& Extents = ParticleBounds.GetExtent();
		
	FVector Bounded = FVector::Max(Extents, FVector(100));
		
	return (Bounded.X * Bounded.Y * Bounded.Z) * 8 / 1000000;
}

double FParticleMetricEntry::GetMetric(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metric)
{
	switch (Complexity)
	{
		case ChaosVDCollisionComplexityFilteringOptions::Simple:
			switch (Metric)
			{
				case ChaosVDParticleMetricsType::PrimitiveDensity:
					return SimplePrimitives / GetVolumeSafe();
				case ChaosVDParticleMetricsType::MemoryUsage:
					return SimpleMemoryUsage;
			}
		break;
		case ChaosVDCollisionComplexityFilteringOptions::Complex:
			switch (Metric)
			{
				case ChaosVDParticleMetricsType::PrimitiveDensity:
					return ComplexPrimitives / GetVolumeSafe();
				case ChaosVDParticleMetricsType::MemoryUsage:
					return ComplexMemoryUsage;
			}
		break;
		case ChaosVDCollisionComplexityFilteringOptions::All:
			switch (Metric)
			{
				case ChaosVDParticleMetricsType::PrimitiveDensity:
					return AllPrimitives / GetVolumeSafe();
				case ChaosVDParticleMetricsType::MemoryUsage:
					return SimpleMemoryUsage + ComplexMemoryUsage;
			}
		break;
	}
	return 0;
}

void FParticleMetricEntry::Aggregate(FParticleMetricEntry& Metric)
{
	SimplePrimitives += Metric.SimplePrimitives;
	SimpleMemoryUsage += Metric.SimpleMemoryUsage;
	ComplexPrimitives += Metric.ComplexPrimitives;
	ComplexMemoryUsage += Metric.ComplexMemoryUsage;
	AllPrimitives += Metric.AllPrimitives;
}

uint32 ChaosVDMetrics::GetPrimitiveCount(const Chaos::FImplicitObject* ImplicitPtr, const Chaos::EImplicitObjectType ImplicitType)
{
	using namespace Chaos;

	switch (ImplicitType)
	{
		case ImplicitObjectType::Box:
		case ImplicitObjectType::Capsule:
		case ImplicitObjectType::Sphere:
		{
			return 1;
		}
		case ImplicitObjectType::Convex:
		{
			if (const FConvex* AsConvex = ImplicitPtr->template AsA<FConvex>())
			{
				return AsConvex->NumPlanes();
			}
		}
		break;
		case ImplicitObjectType::TriangleMesh:
		{
			if (const FTriangleMeshImplicitObject* AsTriMesh = ImplicitPtr->template AsA<FTriangleMeshImplicitObject>())
			{
				return AsTriMesh->Elements().GetNumTriangles();
			}
		}
		break;
		case ImplicitObjectType::HeightField:
		{
			if (const FHeightField* AsHeightField = ImplicitPtr->template AsA<FHeightField>())
			{
				return AsHeightField->GetNumCols() * AsHeightField->GetNumRows();
			}
		}
		break;
	}

	return 0;
}

uint32 ChaosVDMetrics::GetMemoryUsage(const Chaos::FImplicitObject* ImplicitPtr, const Chaos::EImplicitObjectType ImplicitType)
{
	using namespace Chaos;

	switch (ImplicitType)
	{
		case ImplicitObjectType::Box:
		{
			return sizeof(TBox<double, 3>);
		}
		case ImplicitObjectType::Capsule:
		{
			return sizeof(Chaos::FCapsule);
		}
		case ImplicitObjectType::Sphere:
		{
			return sizeof(Chaos::TSphere<double, 3>);
		}
		case ImplicitObjectType::Convex:
		{
			//Needed for serializing to get the object size
			if (FConvex* AsConvex = const_cast<FConvex*>(ImplicitPtr->template AsA<FConvex>()))
			{
				FArchiveCountMem CountMem(nullptr);
				CountMem.SetIsSaving(true);
				CountMem.SetIsPersistent(false);
				Chaos::FChaosArchive ChaosAr(CountMem);
				AsConvex->Serialize(ChaosAr);
				return CountMem.GetNum();
			}
		}
		break;
		case ImplicitObjectType::TriangleMesh:
		{
			//Needed for serializing to get the object size
			if (FTriangleMeshImplicitObject* AsTriMesh = const_cast<FTriangleMeshImplicitObject*>(ImplicitPtr->template AsA<FTriangleMeshImplicitObject>()))
			{
				FArchiveCountMem CountMem(nullptr);
				CountMem.SetIsSaving(true);
				CountMem.SetIsPersistent(false);
				Chaos::FChaosArchive ChaosAr(CountMem);
				ChaosAr.GetArchiveState().SetShouldSkipUpdateCustomVersion(true);
				AsTriMesh->Serialize(ChaosAr);
				return CountMem.GetNum();
			}
		}
		break;
		case ImplicitObjectType::HeightField:
		{
			//Needed for serializing to get the object size
			if (FHeightField* AsHeightmap = const_cast<FHeightField*>(ImplicitPtr->template AsA<FHeightField>()))
			{
				FArchiveCountMem CountMem(nullptr);
				CountMem.SetIsSaving(true);
				CountMem.SetIsPersistent(false);
				Chaos::FChaosArchive ChaosAr(CountMem);
				AsHeightmap->Serialize(ChaosAr);
				return CountMem.GetNum();
			}
		}
		break;
	}

	return 0;
}

void ChaosVDMetrics::CalculateMetrics(const TSharedRef<FChaosVDSceneParticle>& InParticleInstance, TWeakPtr<FChaosVDScene> WeakCVDScene, FParticleMetricEntry& OutMetrics)
{
	TSharedPtr<FChaosVDScene> ScenePtr = WeakCVDScene.Pin();
	if (!ScenePtr)
	{
		return;
	}

	TSharedPtr<const FChaosVDParticleDataWrapper> ParticleData = InParticleInstance->GetParticleData();

	Chaos::FConstImplicitObjectPtr RootGeometry = ParticleData ? ScenePtr->GetUpdatedGeometry(ParticleData->GeometryHash) : nullptr;
	OutMetrics.ParticleID = ParticleData->ParticleIndex;
	OutMetrics.SolverID = ParticleData->SolverID;
	OutMetrics.ParticleName = FName(InParticleInstance->GetDisplayName());
	CalculateImplicitMetrics(RootGeometry, WeakCVDScene, InParticleInstance, OutMetrics, 0);

	OutMetrics.ParticleBounds = InParticleInstance->GetBoundingBox();
}



void ChaosVDMetrics::CalculateImplicitMetrics(Chaos::FConstImplicitObjectPtr InImplicitObject, TWeakPtr<FChaosVDScene> WeakCVDScene, const TSharedRef<FChaosVDSceneParticle>& InParticleInstance, FParticleMetricEntry& OutMetrics, int ShapeIndex)
{
	using namespace Chaos;

	if (!InImplicitObject)
	{
		return;
	}
	
	TSharedPtr<FChaosVDScene> ScenePtr = WeakCVDScene.Pin();
	if (!ScenePtr)
	{
		return;
	}
	const bool bNeedsUnpack = ScenePtr->DoesGeometryNeedUnpacking(InImplicitObject);
	FRigidTransform3 ExtractedTransform;
	const FImplicitObject* ImplicitObjectToProcess = bNeedsUnpack ? ScenePtr->UnpackGeometry(InImplicitObject, ExtractedTransform) : InImplicitObject.GetReference();

	bool bSimpleCollision = false;
	bool bComplexCollision = false;

	if (InParticleInstance->GetParticleData() && InParticleInstance->GetParticleData()->CollisionDataPerShape.IsValidIndex(ShapeIndex))
	{
		const FChaosVDShapeCollisionData& Data = InParticleInstance->GetParticleData()->CollisionDataPerShape[ShapeIndex];

		uint32 Flags = Data.FilterData.GetFlags();

		bSimpleCollision = Flags & static_cast<uint32>(EFilterFlags::SimpleCollision);
		bComplexCollision = Flags & static_cast<uint32>(EFilterFlags::ComplexCollision);
	}

	const EImplicitObjectType InnerType = GetInnerType(ImplicitObjectToProcess->GetType());

	uint32 PrimitiveCount = GetPrimitiveCount(ImplicitObjectToProcess, InnerType);
	uint32 MemoryUsage = GetMemoryUsage(ImplicitObjectToProcess, InnerType);

	if (bSimpleCollision)
	{
		OutMetrics.SimplePrimitives += PrimitiveCount;

		//Don't double count triangle meshes if they also are used for complex
		OutMetrics.SimpleMemoryUsage += (bComplexCollision && InnerType == ImplicitObjectType::TriangleMesh) ? 0 : MemoryUsage;
	}

	if (bComplexCollision)
	{
		OutMetrics.ComplexPrimitives += PrimitiveCount;

		//Don't double count simple primitives if they are also used for simple
		OutMetrics.ComplexMemoryUsage += (bSimpleCollision && InnerType != ImplicitObjectType::TriangleMesh) ? 0 : MemoryUsage;
	}

	OutMetrics.AllPrimitives += PrimitiveCount;
	
	switch (InnerType)
	{
		case ImplicitObjectType::Union:
		case ImplicitObjectType::UnionClustered:
		{
			const bool bIsRootUnion = ShapeIndex == 0;
			const bool bIsCluster = InnerType == ImplicitObjectType::UnionClustered;

			if (const FImplicitObjectUnion* Union = InImplicitObject->template AsA<FImplicitObjectUnion>())
			{
				const TArray<Chaos::FImplicitObjectPtr>& UnionObjects = Union->GetObjects();
				int32 ObjectIndex = 0;
				for (const FImplicitObjectPtr& UnionImplicit : UnionObjects)
				{
					if (bIsRootUnion)
					{
						if (bIsCluster)
						{
							// Geometry Collections might break the usual rule of how may shape data instances we have per geometry
							// Sometimes they can create clusters where all particles share a single instance
							if (InParticleInstance->GetParticleData() && InParticleInstance->GetParticleData()->CollisionDataPerShape.Num() == 1)
							{
								ShapeIndex = 0;
							}
						}
						else
						{
							ShapeIndex = ObjectIndex;
						}
					}

					CalculateImplicitMetrics(UnionImplicit.GetReference(), WeakCVDScene, InParticleInstance, OutMetrics, ShapeIndex);
					ObjectIndex++;
				}
			}
		}
		break;
		case ImplicitObjectType::Transformed:
		{
			const TImplicitObjectTransformed<FReal, 3>* Transformed = InImplicitObject->template GetObject<TImplicitObjectTransformed<FReal, 3>>();
			CalculateImplicitMetrics(Transformed->GetTransformedObject(), WeakCVDScene, InParticleInstance, OutMetrics, ShapeIndex);
		}
		break;
	}
}