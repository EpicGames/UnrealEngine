// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Chaos/ImplicitFwd.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "Chaos/ImplicitObjectScaled.h"
#include "Chaos/ImplicitObjectType.h"
#include "ChaosVDMetrics.generated.h"

class FChaosVDScene;

UENUM()
enum class ChaosVDCollisionComplexityFilteringOptions : uint8
{
	Simple,
	Complex,
	All,
};

UENUM()
enum class ChaosVDParticleMetricsType : uint8
{
	PrimitiveDensity,
	MemoryUsage,
};

USTRUCT()
struct FChaosVDMetricsSettings
{
	GENERATED_BODY()

	UPROPERTY()
	float SimpleMinThreshold = 0.0f;

	UPROPERTY()
	float SimpleMaxThreshold = 0.0f;

	UPROPERTY()
	float ComplexMinThreshold = 0.0f;

	UPROPERTY()
	float ComplexMaxThreshold = 0.0f;

	UPROPERTY()
	float AllMinThreshold = 0.0f;

	UPROPERTY()
	float AllMaxThreshold = 0.0f;

	double GetMinThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity) const;
	void SetMinThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, double InMin);

	double GetMaxThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity) const;
	void SetMaxThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, double InMax);
};

struct FParticleMetricEntry
{
	FName ParticleName;
	int32 SolverID = INDEX_NONE;
	int32 ParticleID = INDEX_NONE;
	uint32 SimplePrimitives = 0;
	uint32 SimpleMemoryUsage = 0;
	uint32 ComplexPrimitives = 0;
	uint32 ComplexMemoryUsage = 0;
	uint32 AllPrimitives = 0;
	FBox ParticleBounds = FBox(EForceInit::ForceInitToZero);

	double GetVolumeSafe();

	double GetMetric(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metric);

	void Aggregate(FParticleMetricEntry& Metric);
};

namespace ChaosVDMetrics
{
	uint32 GetPrimitiveCount(const Chaos::FImplicitObject* ImplicitPtr, const Chaos::EImplicitObjectType ImplicitType);
	uint32 GetMemoryUsage(const Chaos::FImplicitObject* ImplicitPtr, const Chaos::EImplicitObjectType ImplicitType);
	void CalculateMetrics(const TSharedRef<FChaosVDSceneParticle>& InParticleInstance, TWeakPtr<FChaosVDScene> WeakCVDScene, FParticleMetricEntry& OutMetrics);
	void CalculateImplicitMetrics(Chaos::FConstImplicitObjectPtr InImplicitObject, TWeakPtr<FChaosVDScene> WeakCVDScene, const TSharedRef<FChaosVDSceneParticle>& InParticleInstance, FParticleMetricEntry& OutMetrics, int ShapeIndex);
};