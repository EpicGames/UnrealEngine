// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDMetricsHeatmapCommands.h"
#include "Styling/AppStyle.h"

#define LOCTEXT_NAMESPACE "ChaosVDMetricsHeatmap"

FChaosVDMetricsHeatmapCommands::FChaosVDMetricsHeatmapCommands()
	: Super(TEXT("ChaosVDMetricsHeatmapCommands"), NSLOCTEXT("Contexts", "ChaosVDMetricsHeatmapCommands", "Grid"), NAME_None, FAppStyle::Get().GetStyleSetName())
{
}

void FChaosVDMetricsHeatmapCommands::RegisterCommands()
{
	UI_COMMAND(FocusBounds, "Focus Bounds", "Focus on the center of the map bounds", EUserInterfaceActionType::Button, FInputChord());
	UI_COMMAND(TrackEditorView, "Track Editor View", "Toggle Track Editor View", EUserInterfaceActionType::ToggleButton, FInputChord());
}

#undef LOCTEXT_NAMESPACE