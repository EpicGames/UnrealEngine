// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Framework/Commands/Commands.h"

class FChaosVDMetricsHeatmapCommands : public TCommands<FChaosVDMetricsHeatmapCommands>
{
	using ThisClass = FChaosVDMetricsHeatmapCommands;
	using Super = TCommands<FChaosVDMetricsHeatmapCommands>;

public:
	FChaosVDMetricsHeatmapCommands();

	virtual ~FChaosVDMetricsHeatmapCommands() override = default;

	virtual void RegisterCommands() override;

	TSharedPtr<FUICommandInfo> FocusBounds;
	TSharedPtr<FUICommandInfo> TrackEditorView;
};
