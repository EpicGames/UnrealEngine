// Copyright Epic Games, Inc. All Rights Reserved.
#include "PerformanceMetricsExtension.h"
#include "Tabs/PerformanceMetricsTabSpawner.h"
#include "Widgets/SChaosVDMainTab.h"
#include "PerformanceMetrics/Commands/ChaosVDMetricsHeatmapCommands.h"
#include "ChaosVD/Public/ChaosVDPlaybackController.h"
#include "EditorModeManager.h"

FName FPerformanceMetricsExtension::TabID = FName("PerformanceMetricsTab");

FPerformanceMetricsExtension::FPerformanceMetricsExtension()
{
	ExtensionName = FName(TEXT("FPerformanceMetricsExtension"));
	FChaosVDMetricsHeatmapCommands::Register();
}
FPerformanceMetricsExtension::~FPerformanceMetricsExtension()
{}

void FPerformanceMetricsExtension::RegisterCustomTabSpawners(const TSharedRef<SChaosVDMainTab>& InParentTabWidget)
{
	InParentTabWidget->RegisterTabSpawner<FPerformanceMetricsTabSpawner>(TabID);

	WeakCVDToolKit = InParentTabWidget;
}

void FPerformanceMetricsExtension::RegisterComponentVisualizers(const TSharedRef<SChaosVDMainTab>& InCVDToolKit)
{
	EditorModeToolsWeakPtr = InCVDToolKit->GetEditorModeManager().AsWeak();
}

void FPerformanceMetricsExtension::HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController)
{
	if (const TSharedPtr<SChaosVDMainTab> ParentTabWidget = WeakCVDToolKit.Pin())
	{
		TWeakPtr<FPerformanceMetricsTabSpawner> WeakTabSpawner = ParentTabWidget->GetTabSpawnerInstance<FPerformanceMetricsTabSpawner>(TabID);

		if (TSharedPtr<FPerformanceMetricsTabSpawner> TabSpawner = WeakTabSpawner.Pin())
		{
			if (TSharedPtr<FChaosVDPlaybackController> Playback = InController.Pin())
			{
				if (Playback->IsRecordingLoaded())
				{
					TabSpawner->HandleRecordingFirstFrameLoaded(InController);
				}
			}
		}
	}
}