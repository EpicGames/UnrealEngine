// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "ExtensionsSystem/ChaosVDExtension.h"

class FChaosVDPlaybackController;
class FEditorModeTools;
class SChaosVDMainTab;

class FPerformanceMetricsExtension : public FChaosVDExtension
{
public:
	FPerformanceMetricsExtension();
	virtual ~FPerformanceMetricsExtension() override;
	virtual void RegisterCustomTabSpawners(const TSharedRef<SChaosVDMainTab>& InParentTabWidget) override;
	virtual void RegisterComponentVisualizers(const TSharedRef<SChaosVDMainTab>& InCVDToolKit) override;
	virtual void HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController) override;

	TWeakPtr<SChaosVDMainTab> WeakCVDToolKit;
	TWeakPtr<FEditorModeTools> EditorModeToolsWeakPtr;

	static FName TabID;
};