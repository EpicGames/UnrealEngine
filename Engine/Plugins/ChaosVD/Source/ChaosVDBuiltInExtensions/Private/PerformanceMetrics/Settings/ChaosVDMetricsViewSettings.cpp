// Copyright Epic Games, Inc. All Rights Reserved.

#include "ChaosVDMetricsViewSettings.h"
#include "ChaosVD/Public/ChaosVDSettingsManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ChaosVDMetricsViewSettings)

TStrongObjectPtr<UChaosVDMetricsViewSettings> UChaosVDMetricsViewSettings::DefaultSettings;

UChaosVDMetricsViewSettings::UChaosVDMetricsViewSettings()
{
	SetDefaults();
}

void UChaosVDMetricsViewSettings::PostInitProperties()
{
	using namespace Chaos::VD::PerformanceMetrics;

	Super::PostInitProperties();
	HeatmapCellSize = FMath::Clamp(HeatmapCellSize, SettingsBounds::MinCellSize, SettingsBounds::MaxCellSize);
}

uint32 UChaosVDMetricsViewSettings::GetHeatmapCellSize() const
{
	return HeatmapCellSize;
}

void UChaosVDMetricsViewSettings::SetHeatmapCellSize(uint32 InValue)
{
	if (UChaosVDMetricsViewSettings* MetricsSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMetricsViewSettings>())
	{
		MetricsSettings->HeatmapCellSize = InValue;
		MetricsSettings->BroadcastSettingsChanged();
	}
}

void UChaosVDMetricsViewSettings::SetHeatmapCellMinThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metrics, double InMin)
{
	if (UChaosVDMetricsViewSettings* MetricsSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMetricsViewSettings>())
	{
		MetricsSettings->MetricSettings.Find(Metrics)->SetMinThreshold(Complexity, InMin);
		MetricsSettings->BroadcastSettingsChanged();
	}
}

double UChaosVDMetricsViewSettings::GetHeatmapCellMinThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metrics) const
{
	const FChaosVDMetricsSettings* MetricSetting = MetricSettings.Find(Metrics);
	return MetricSetting ? MetricSettings.Find(Metrics)->GetMinThreshold(Complexity) : 0;
}

void UChaosVDMetricsViewSettings::SetHeatmapCellMaxThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metrics, double InMax)
{
	if (UChaosVDMetricsViewSettings* MetricsSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMetricsViewSettings>())
	{
		MetricsSettings->MetricSettings.Find(Metrics)->SetMaxThreshold(Complexity, InMax);
		MetricsSettings->BroadcastSettingsChanged();
	}
}

double UChaosVDMetricsViewSettings::GetHeatmapCellMaxThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metrics) const
{
	const FChaosVDMetricsSettings* MetricSetting = MetricSettings.Find(Metrics);
	return MetricSetting ? MetricSetting->GetMaxThreshold(Complexity) : 0;
}

void UChaosVDMetricsViewSettings::SetHeatmapColorSettings(FChaosVDHeatmapColorSettings& InSettings)
{
	if (UChaosVDMetricsViewSettings* MetricsSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMetricsViewSettings>())
	{
		MetricsSettings->HeatmapColorSettings = InSettings;
		MetricsSettings->BroadcastSettingsChanged();
	}
}

// Normal UE default object will contain the serialized values. Save a copy here for the reset to defaults feature.
const UChaosVDMetricsViewSettings* UChaosVDMetricsViewSettings::GetDefaultSettings()
{
	if (DefaultSettings)
	{
		return DefaultSettings.Get();
	}
	else
	{
		DefaultSettings = TStrongObjectPtr<UChaosVDMetricsViewSettings>(NewObject<UChaosVDMetricsViewSettings>());
		DefaultSettings->SetDefaults();
		return DefaultSettings.Get();
	}
}

void UChaosVDMetricsViewSettings::SetDefaults()
{
	static constexpr double SampleArea = (25 * (100* 100));

	static constexpr double SimplePrimitivesBaseline = 45.5;
	static constexpr double SimplePrimitivesHigh = 1455.5;
	static constexpr double ComplexPrimitivesBaseline = 2450;
	static constexpr double ComplexPrimitivesMax = 40000;

	FChaosVDMetricsSettings& DensityDefaultSettings = MetricSettings.FindOrAdd(ChaosVDParticleMetricsType::PrimitiveDensity);
	DensityDefaultSettings.SimpleMinThreshold = static_cast<float>(SimplePrimitivesBaseline / SampleArea);
	DensityDefaultSettings.SimpleMaxThreshold = static_cast<float>(SimplePrimitivesHigh / SampleArea);
	DensityDefaultSettings.ComplexMinThreshold = static_cast<float>(ComplexPrimitivesBaseline / SampleArea);
	DensityDefaultSettings.ComplexMaxThreshold = static_cast<float>(ComplexPrimitivesMax / SampleArea);
	DensityDefaultSettings.AllMinThreshold = DensityDefaultSettings.SimpleMinThreshold + DensityDefaultSettings.ComplexMinThreshold;
	DensityDefaultSettings.AllMaxThreshold = DensityDefaultSettings.SimpleMaxThreshold + DensityDefaultSettings.ComplexMaxThreshold;

	static constexpr double SimpleMemoryBaseline = 56.2587;
	static constexpr double SimpleMemoryMax = 20000;
	static constexpr double ComplexMemoryBaseline = 25000; 
	static constexpr double ComplexMemoryMax = 200000;

	FChaosVDMetricsSettings& MemoryDefaultSettings = MetricSettings.FindOrAdd(ChaosVDParticleMetricsType::MemoryUsage);
	MemoryDefaultSettings.SimpleMinThreshold = static_cast<float>(SimpleMemoryBaseline / SampleArea);
	MemoryDefaultSettings.SimpleMaxThreshold = static_cast<float>(SimpleMemoryMax / SampleArea);
	MemoryDefaultSettings.ComplexMinThreshold = static_cast<float>(ComplexMemoryBaseline / SampleArea);
	MemoryDefaultSettings.ComplexMaxThreshold = static_cast<float>(ComplexMemoryMax / SampleArea);
	MemoryDefaultSettings.AllMinThreshold = MemoryDefaultSettings.SimpleMinThreshold + MemoryDefaultSettings.ComplexMinThreshold;
	MemoryDefaultSettings.AllMaxThreshold = MemoryDefaultSettings.SimpleMaxThreshold + MemoryDefaultSettings.ComplexMaxThreshold;

	HeatmapCellSize = 2000;
}