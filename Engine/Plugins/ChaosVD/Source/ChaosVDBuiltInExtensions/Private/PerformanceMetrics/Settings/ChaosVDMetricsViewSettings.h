// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Settings/ChaosVDCoreSettings.h"
#include "PerformanceMetrics/ChaosVDMetrics.h"
#include "UObject/StrongObjectPtr.h"
#include "ChaosVDMetricsViewSettings.generated.h"

namespace Chaos::VD::PerformanceMetrics::SettingsBounds
{ 
	constexpr uint32 MinCellSize = 1000;
	constexpr uint32 MaxCellSize = 12800;
}

USTRUCT()
struct FChaosVDHeatmapColorSettings
{
	GENERATED_BODY()

	/** The color for the low values in the metric's expected range. */
	UPROPERTY()
	FColor LowValueColor = FColor(10, 10, 10);

	/** The color for the values in the middle of the metric's expected range. */
	UPROPERTY()
	FColor MidpointValueColor = FColor(0x85, 0x0C, 0x00);

	/** The color for the high values in the metric's expected range. */
	UPROPERTY()
	FColor HighValueColor = FColor(0xFF, 0, 0);

	/** The color for values exceeding the metric's expected range. */
	UPROPERTY()
	FColor MaxValueColor = FColor(0xFF, 0xE6, 0xE6);


	/** The minimum alpha value used for all spatial values. */
	UPROPERTY()
	float MinAlpha = 0.4f;

	/**
	 * The maximum alpha value used for spatial values.
	 * The alpha value increases linearly according to the spatial value range.
	 */
	UPROPERTY()
	float MaxAlpha = 0.8f;

	/** How fast the heat value's alpha increases from MinValue to MaxValue according to the spatial value range. */
	UPROPERTY()
	float AlphaFactor = 2.0f;

	/** The ratio position of the low point. All values up to this ratio should use LowValueColor. */
	UPROPERTY()
	float LowpointRatio = 0.2f;

	/** The ratio position of the mid blend point. The value at this ratio should use MidpointValueColor. */
	UPROPERTY()
	float MidpointRatio = 0.5f;
};

UCLASS(config = ChaosVD)
class UChaosVDMetricsViewSettings : public UChaosVDSettingsObjectBase
{
	GENERATED_BODY()
public:
	UChaosVDMetricsViewSettings();

	UPROPERTY(Config)
	uint32 HeatmapCellSize;

	UPROPERTY(Config)
	FChaosVDHeatmapColorSettings HeatmapColorSettings;

	virtual void PostInitProperties() override;

	uint32 GetHeatmapCellSize() const;
	static void SetHeatmapCellSize(uint32 InValue);

	static void SetHeatmapCellMinThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metrics, double InMin);
	double GetHeatmapCellMinThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metrics) const;

	static void SetHeatmapCellMaxThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metrics, double InMax);
	double GetHeatmapCellMaxThreshold(ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metrics) const;

	static void SetHeatmapColorSettings(FChaosVDHeatmapColorSettings& InSettings);

	static const UChaosVDMetricsViewSettings* GetDefaultSettings();
	void SetDefaults();

	static TStrongObjectPtr<UChaosVDMetricsViewSettings> DefaultSettings;
private:
	UPROPERTY(Config)
	TMap<ChaosVDParticleMetricsType, FChaosVDMetricsSettings> MetricSettings;
};