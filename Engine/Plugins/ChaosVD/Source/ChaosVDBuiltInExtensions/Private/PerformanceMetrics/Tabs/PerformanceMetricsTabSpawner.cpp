// Copyright Epic Games, Inc. All Rights Reserved.
#include "PerformanceMetricsTabSpawner.h"
#include "ChaosVDEngine.h"
#include "ChaosVDStyle.h"
#include "ChaosVD/Public/ChaosVDPlaybackController.h"
#include "PerformanceMetrics/Widgets/SPerfMetricsViewer.h"
#include "Widgets/SChaosVDMainTab.h"

#define LOCTEXT_NAMESPACE "ChaosVDPerformanceMetricsTabSpawner"

TSharedRef<SDockTab> FPerformanceMetricsTabSpawner::HandleTabSpawnRequest(const FSpawnTabArgs& Args)
{
	TSharedRef<SDockTab> ComplexityViewerTab =
		SNew(SDockTab)
		.TabRole(ETabRole::PanelTab)
		.Label(LOCTEXT("ComplexityViewerTabLabel", "Complexity Viewer"))
		.ToolTipText(LOCTEXT("ComplexityViewerTabToolTip", "Tool that allows you to analyze the current frame and analyze the collision geometry complexity in the recorded world"));
	if (TSharedPtr<SChaosVDMainTab> OwnerTab = OwningTabWidget.Pin())
	{
		ComplexityViewerTab->SetContent
		(
			SAssignNew(PerfViewerTabContent, SPerfMetricsViewer, OwnerTab->GetChaosVDEngineInstance()->GetCurrentScene(), OwningTabWidget)
		);
		ComplexityViewerTab->SetTabIcon(FChaosVDStyle::Get().GetBrush("TabIconWorldOutliner"));
	}
	else
	{
		ComplexityViewerTab->SetContent
		(
			GenerateErrorWidget()
		);
	}
	HandleTabSpawned(ComplexityViewerTab);
	return ComplexityViewerTab;
}

FPerformanceMetricsTabSpawner::~FPerformanceMetricsTabSpawner()
{}

void FPerformanceMetricsTabSpawner::HandleTabClosed(TSharedRef<SDockTab> InTabClosed)
{
	FChaosVDTabSpawnerBase::HandleTabClosed(InTabClosed);
}

void FPerformanceMetricsTabSpawner::HandleTabSpawned(TSharedRef<SDockTab> InTabSpawned)
{
	FChaosVDTabSpawnerBase::HandleTabSpawned(InTabSpawned);
}

void FPerformanceMetricsTabSpawner::HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController)
{
	if (PerfViewerTabContent)
	{
		PerfViewerTabContent->HandleRecordingFirstFrameLoaded(InController);
	}
}

#undef LOCTEXT_NAMESPACE