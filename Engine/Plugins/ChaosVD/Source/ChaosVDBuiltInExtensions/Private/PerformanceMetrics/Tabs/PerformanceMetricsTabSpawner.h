// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "ChaosVDTabSpawnerBase.h"
#include "PerformanceMetrics/Widgets/SPerfMetricsViewer.h"

class FChaosVDPlaybackController;

class FPerformanceMetricsTabSpawner : public FChaosVDTabSpawnerBase
{
public:
	FPerformanceMetricsTabSpawner(const FName& InTabID, const TSharedPtr<FTabManager>& InTabManager, const TWeakPtr<SChaosVDMainTab>& InOwningTabWidget)
		: FChaosVDTabSpawnerBase(InTabID, InTabManager, InOwningTabWidget)
		{}
	virtual TSharedRef<SDockTab> HandleTabSpawnRequest(const FSpawnTabArgs& Args) override;
	virtual ~FPerformanceMetricsTabSpawner() override;
	virtual void HandleTabClosed(TSharedRef<SDockTab> InTabClosed) override;
	virtual void HandleTabSpawned(TSharedRef<SDockTab> InTabSpawned) override;
	void HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController);
private:
	TSharedPtr<SPerfMetricsViewer> PerfViewerTabContent;
};