// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCVDMetricsHeatmapToolbar.h"

#include "Engine/Engine.h"
#include "PropertyCustomizationHelpers.h"
#include "SEditorViewportToolBarMenu.h"
#include "SEnumCombo.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"
#include "ToolMenuWidgetCollectionContext.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/NumericTypeInterface.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Colors/SColorBlock.h"
#include "Widgets/Colors/SColorPicker.h"
#include "Widgets/Input/SButton.h"
#include "PerformanceMetrics/Commands/ChaosVDMetricsHeatmapCommands.h"
#include "Framework/Commands/UICommandList.h"
#include "PerformanceMetrics/ChaosVDMetrics.h"
#include "SChaosVDMetricsViewerState.h"

#define LOCTEXT_NAMESPACE "SCVDMetricsHeatmapToolbar"

namespace Chaos::VD::PerformanceMetrics::Private
{

static const FName ToolbarOptionsMenuName(TEXT("CVDMetricsHeatmapToolbar.OptionsMenu"));
static const FName FocusMenuName(TEXT("CVDMetricsHeatmapToolbar.FocusMenu"));
static const FName HeatmapColorSettingsName(TEXT("CVDMetricsHeatmapToolbar.HeatmapColorSettings"));
static const FName MetricSettingsName(TEXT("CVDMetricsHeatmapToolbar.MetricSettings"));
static const FName ShowFlagsName(TEXT("CVDMetricsHeatmapToolbar.ShowFlagsName"));
static constexpr float DefaultFieldWidth = 90.0f;
static constexpr float DefaultContentWidth = 120.0f;

class SMenuPropertyEntry final : public SCompoundWidget
{
public:
	using FResetValueDelegate = TDelegate<FReply()>;

	// clang-format off
	SLATE_BEGIN_ARGS(SMenuPropertyEntry)
	{}
		SLATE_ATTRIBUTE(FText, Label)
		SLATE_ATTRIBUTE(FText, ToolTip)
		SLATE_ARGUMENT_DEFAULT(float, FieldWidth) = DefaultFieldWidth;
		SLATE_ARGUMENT_DEFAULT(float, ContentWidth) = DefaultContentWidth;
		SLATE_NAMED_SLOT(FArguments, CheckboxContent)
		SLATE_EVENT(FResetValueDelegate, OnResetValue)
		SLATE_DEFAULT_SLOT(FArguments, Content)
	SLATE_END_ARGS()
	// clang-format on

	void Construct(const FArguments& InArgs)
	{
		Label = InArgs._Label;
		ToolTip = InArgs._ToolTip;
		OnResetValue = InArgs._OnResetValue;

		ChildSlot
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(12, 0, 0, 0)
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(16)
				[
					InArgs._CheckboxContent.Widget
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.Padding(12, 0, 0, 0)
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(InArgs._FieldWidth)
				[
					SNew(STextBlock)
					.Text(Label)
					.ToolTipText(ToolTip)
					.OverflowPolicy(ETextOverflowPolicy::Ellipsis)
				]
			]
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			[
				SNew(SBox)
				.WidthOverride(InArgs._ContentWidth)
				[
					InArgs._Content.Widget
				]
			]

			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.AutoWidth()
			.Padding(0, 0, 8, 0)
			[
					SNew(SButton)
					.ToolTipText(LOCTEXT("ResetTooltip", "Reset"))
					.OnClicked(OnResetValue)
					.ButtonStyle(FAppStyle::Get(), "NoBorder") 
					[
						SNew(SBox)
						.HAlign(HAlign_Center)
						.VAlign(VAlign_Center)
						[
							SNew(SImage)
								.Image(FAppStyle::GetBrush("PropertyWindow.DiffersFromDefault"))
								.DesiredSizeOverride(FVector2D(16, 16))
								.ColorAndOpacity(FSlateColor::UseForeground())
						]
					]
			]
		];
	}

private:
	TAttribute<FText> Label;
	TAttribute<FText> ToolTip;
	TAttribute<bool> HasCustomValue;
	FResetValueDelegate OnResetValue;
};

class SNumericMenuPropertyEntry final : public SCompoundWidget
{
public:
	using FOnValueChangedDelegate = TDelegate<void(double)>;
	using FOnValueCommittedDelegate = TDelegate<void(double, ETextCommit::Type)>;
	using FResetValueDelegate = TDelegate<FReply()>;

	// clang-format off
	SLATE_BEGIN_ARGS(SNumericMenuPropertyEntry)
	{}
		SLATE_ATTRIBUTE(FText, Label)
		SLATE_ATTRIBUTE(FText, ToolTip)
		SLATE_ARGUMENT_DEFAULT(float, FieldWidth) = DefaultFieldWidth;
		SLATE_ARGUMENT_DEFAULT(float, ContentWidth) = DefaultContentWidth;
		SLATE_ATTRIBUTE(TOptional<double>, Value)
		SLATE_ATTRIBUTE(TOptional<double>, MinValue);
		SLATE_ATTRIBUTE(TOptional<double>, MaxValue);
		SLATE_ARGUMENT_DEFAULT(int, MaxDecimals) = 3;
		SLATE_EVENT(FOnValueChangedDelegate, OnValueChanged)
		SLATE_EVENT(FOnValueCommittedDelegate, OnValueCommitted)
		SLATE_EVENT(FResetValueDelegate, OnResetValue)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs)
	{
		ChildSlot
		[
			SNew(SMenuPropertyEntry)
			.Label(InArgs._Label)
			.ToolTip(InArgs._ToolTip)
			.FieldWidth(InArgs._FieldWidth)
			.ContentWidth(InArgs._ContentWidth)
			.OnResetValue(InArgs._OnResetValue)
			[
				SNew(SNumericEntryBox<double>)
				.MinFractionalDigits(0)
				.MaxFractionalDigits(InArgs._MaxDecimals)
				.ToolTipText(InArgs._ToolTip)
				.AllowSpin(false)
				.MinValue(InArgs._MinValue)
				.MaxValue(InArgs._MaxValue)
				.MinSliderValue(InArgs._MinValue)
				.MaxSliderValue(InArgs._MaxValue)
				.Value(InArgs._Value)
				.OnValueChanged(InArgs._OnValueChanged)
				.OnValueCommitted(InArgs._OnValueCommitted)
			]
		];
	}
};

class SColorMenuPropertyEntry final : public SCompoundWidget
{
	using ThisClass = SColorMenuPropertyEntry;
public:
	using FOnColorChangedDelegate = TDelegate<void(FColor)>;
	using FResetValueDelegate = TDelegate<FReply()>;

	// clang-format off
	SLATE_BEGIN_ARGS(SColorMenuPropertyEntry)
	{}
		SLATE_ATTRIBUTE(FText, Label)
		SLATE_ATTRIBUTE(FText, ToolTip)
		SLATE_ARGUMENT_DEFAULT(float, FieldWidth) = DefaultFieldWidth;
		SLATE_ARGUMENT_DEFAULT(float, ContentWidth) = DefaultContentWidth;
		SLATE_ATTRIBUTE(FColor, Color)
		SLATE_EVENT(FOnColorChangedDelegate, OnColorChanged)
		SLATE_EVENT(FResetValueDelegate, OnResetValue)
	SLATE_END_ARGS()
	// clang-format on

	void Construct(const FArguments& InArgs)
	{
		Color = InArgs._Color;
		OnColorChanged = InArgs._OnColorChanged;

		static const FVector2D ColorBlockSize(20.0f, 20.0f);
		static const FVector4 ColorBlockCornerRadius(4.0f, 4.0f, 4.0f, 4.0f);

		ChildSlot
		[
			SNew(SMenuPropertyEntry)
			.Label(InArgs._Label)
			.ToolTip(InArgs._ToolTip)
			.FieldWidth(InArgs._FieldWidth)
			.ContentWidth(InArgs._ContentWidth)
			.OnResetValue(InArgs._OnResetValue)
			[
				SNew(SColorBlock)
				.Size(ColorBlockSize)
				.ToolTipText(InArgs._ToolTip)
				.CornerRadius(ColorBlockCornerRadius)
				.AlphaDisplayMode(EColorBlockAlphaDisplayMode::Ignore)
				.UseSRGB(true)
				.Color(this, &SColorMenuPropertyEntry::GetColor)
				.OnMouseButtonDown(this, &SColorMenuPropertyEntry::OnColorBlockClicked)
			]
		];
	}

private:
	FLinearColor GetColor() const
	{
		return FLinearColor(Color.Get());
	}

	FReply OnColorBlockClicked(const FGeometry&, const FPointerEvent&)
	{
		FColorPickerArgs PickerArgs;
		PickerArgs.bUseAlpha = false;
		PickerArgs.DisplayGamma = MakeAttributeUObject<float>(GEngine, &UEngine::GetDisplayGamma);
		PickerArgs.InitialColor = Color.Get();
		PickerArgs.OnColorCommitted = FOnLinearColorValueChanged::CreateSP(this, &ThisClass::OnColorCommitted);
		OpenColorPicker(PickerArgs);
		return FReply::Handled();
	}

	void OnColorCommitted(FLinearColor LinearColor)
	{
		OnColorChanged.ExecuteIfBound(LinearColor.ToFColor(true));
	}

	TAttribute<FColor> Color;
	FOnColorChangedDelegate OnColorChanged;
};

}

SLATE_IMPLEMENT_WIDGET(SCVDMetricsHeatmapToolbar)

void SCVDMetricsHeatmapToolbar::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

void SCVDMetricsHeatmapToolbar::Construct(const FArguments& InArgs, TSharedPtr<FChaosVDMetricsViewerState> InViewerState)
{
	CommandList = InArgs._EditorCommands;

	check(CommandList);

	ViewerState = InViewerState;

	// clang-format off
	ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FAppStyle::Get().GetBrush("EditorViewportToolBar.Background"))
		.Cursor(EMouseCursor::Default)
		[
			SNew(SHorizontalBox)

			// Options menu
			+SHorizontalBox::Slot()
			.AutoWidth()
			.Padding(4.0f, 1.0f)
			[
				SNew(SEditorViewportToolbarMenu)
				.Image("EditorViewportToolBar.OptionsDropdown")
				.ParentToolBar(SharedThis(this))
				.OnGetMenuContent(this, &SCVDMetricsHeatmapToolbar::GenerateOptionsMenu)
			]
			// Focus menu
			+SHorizontalBox::Slot()
			.Padding(4.0f, 1.0f)
			.HAlign(HAlign_Right)
			[
				GenerateFocusMenu()
			]
		]
	];
	// clang-format on
}

TSharedRef<SWidget> SCVDMetricsHeatmapToolbar::GenerateOptionsMenu()
{
	using namespace Chaos::VD::PerformanceMetrics;

	RegisterOptionsMenu();

	FToolMenuContext MenuContext(CommandList);

	// Add this widget to the widget collection context object to allow dynamic menu creation logic to work,
	// as the registered menus are stateless.
	UToolMenuWidgetCollectionContext* WidgetCollection = UToolMenuWidgetCollectionContext::Get(MenuContext);
	WidgetCollection->AddWidget(SharedThis(this));

	return UToolMenus::Get()->GenerateWidget(Private::ToolbarOptionsMenuName, MenuContext);
}

TSharedRef<SWidget> SCVDMetricsHeatmapToolbar::GenerateFocusMenu() const
{
	using namespace Chaos::VD::PerformanceMetrics;

	RegisterFocusMenu();

	const FToolMenuContext MenuContext(CommandList);

	// This menu isn't dynamic - it doesn't need a reference to this widget.

	return UToolMenus::Get()->GenerateWidget(Private::FocusMenuName, MenuContext);
}

void SCVDMetricsHeatmapToolbar::RegisterOptionsMenu()
{
	using namespace Chaos::VD::PerformanceMetrics;

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus || ToolMenus->IsMenuRegistered(Private::ToolbarOptionsMenuName))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
	UToolMenu* OptionsMenu = ToolMenus->RegisterMenu(Private::ToolbarOptionsMenuName);
	OptionsMenu->bSearchable = false;

	
	// Metric Settings
	OptionsMenu->AddDynamicSection(Private::MetricSettingsName, FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
	{
		const UToolMenuWidgetCollectionContext* WidgetCollection = UToolMenuWidgetCollectionContext::Get(Menu->Context);
		const TSharedRef<SCVDMetricsHeatmapToolbar> OwningWidget = WidgetCollection->FindWidget<SCVDMetricsHeatmapToolbar>().ToSharedRef();

		OwningWidget->PopulateMenuMetricSettings(
			Menu->AddSection(Private::MetricSettingsName, LOCTEXT("MetricSettings", "Heatmap Metric Settings"))
		);
	}));

	
	// Heatmap Color Settings
	OptionsMenu->AddDynamicSection(Private::HeatmapColorSettingsName, FNewToolMenuDelegate::CreateLambda([](UToolMenu* Menu)
	{
		const UToolMenuWidgetCollectionContext* WidgetCollection = UToolMenuWidgetCollectionContext::Get(Menu->Context);
		const TSharedRef<SCVDMetricsHeatmapToolbar> OwningWidget = WidgetCollection->FindWidget<SCVDMetricsHeatmapToolbar>().ToSharedRef();

		OwningWidget->PopulateMenuHeatmapColorSettings(
			Menu->AddSection(Private::HeatmapColorSettingsName, LOCTEXT("HeatmapSettings", "Heatmap Color Settings"))
		);
	}));
}

void SCVDMetricsHeatmapToolbar::PopulateMenuMetricSettings(FToolMenuSection& Section)
{
	using namespace Chaos::VD::PerformanceMetrics;

	static constexpr double M2ToCm2 = 10000;
	static constexpr double Cm2ToM2 = 1 / M2ToCm2;

	{
		TSharedRef<SWidget> ThresholdMinValueWidget =
			SNew(Private::SNumericMenuPropertyEntry)
			.Label(LOCTEXT("ThresholdMinValueLabel", "Min Threshold"))
			.ToolTip(LOCTEXT("ThresholdMinValueToolTip", "Metric Min Threshold in Metric / M^2"))
			.MinValue(0)
			.MaxValue_Lambda([this](){return (ViewerState ? ViewerState->GetHeatmapCellMaxThreshold() : 0) * M2ToCm2;})
			.Value_Lambda([this](){return (ViewerState ? ViewerState->GetHeatmapCellMinThreshold() : 0) * M2ToCm2;})
			.OnValueChanged_Lambda([this](double NewValue) {
				if (ViewerState)
				{
					NewValue = FMath::Clamp(NewValue, 0, ViewerState->GetHeatmapCellMaxThreshold() * M2ToCm2);
					ViewerState->SetHeatmapCellMinThreshold(NewValue * Cm2ToM2, false);
				}
			})
			.OnValueCommitted_Lambda([this](double NewValue, ETextCommit::Type CommitType) {
				if (ViewerState)
				{
					NewValue = FMath::Clamp(NewValue, 0, ViewerState->GetHeatmapCellMaxThreshold() * M2ToCm2);
					ViewerState->SetHeatmapCellMinThreshold(NewValue * Cm2ToM2, true);
				}
			})
			.MaxDecimals(3)
			.OnResetValue_Lambda([this]() {
				if (ViewerState)
				{
					if(const UChaosVDMetricsViewSettings* DefaultSettings = UChaosVDMetricsViewSettings::GetDefaultSettings())
					{
						ViewerState->SetHeatmapCellMinThreshold(DefaultSettings->GetHeatmapCellMinThreshold(ViewerState->GetSelectedComplexity(), ViewerState->GetSelectedMetric()), true);
					}
				}
				return FReply::Handled();
			});

		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("ThresholdMinValueWidget"), ThresholdMinValueWidget, FText::GetEmpty()));
	}

	{
		TSharedRef<SWidget> ThresholdMaxValueWidget =
			SNew(Private::SNumericMenuPropertyEntry)
			.Label(LOCTEXT("ThresholdMaxValueLabel", "Max Threshold"))
			.ToolTip(LOCTEXT("ThresholdMaxValueToolTip", "Metric Max Threshold in Metric / M^2"))
			.MinValue_Lambda([this](){return (ViewerState ? ViewerState->GetHeatmapCellMinThreshold() : 0) * M2ToCm2;})
			.MaxValue(8000)
			.Value_Lambda([this](){return (ViewerState ? ViewerState->GetHeatmapCellMaxThreshold() : 0) * M2ToCm2;})
			.OnValueChanged_Lambda([this](double NewValue) {
				if (ViewerState)
				{
					NewValue = FMath::Clamp(NewValue, ViewerState->GetHeatmapCellMinThreshold() * M2ToCm2, 8000);
					ViewerState->SetHeatmapCellMaxThreshold(NewValue * Cm2ToM2, false);
				}
			})
			.OnValueCommitted_Lambda([this](double NewValue, ETextCommit::Type CommitType) {
				if (ViewerState)
				{
					NewValue = FMath::Clamp(NewValue, ViewerState->GetHeatmapCellMinThreshold() * M2ToCm2, 8000);
					ViewerState->SetHeatmapCellMaxThreshold(NewValue * Cm2ToM2, true);
				}
			})
			.MaxDecimals(3)
			.OnResetValue_Lambda([this]() {
				if (ViewerState)
				{
					if(const UChaosVDMetricsViewSettings* DefaultSettings = UChaosVDMetricsViewSettings::GetDefaultSettings())
					{
						ViewerState->SetHeatmapCellMaxThreshold(DefaultSettings->GetHeatmapCellMaxThreshold(ViewerState->GetSelectedComplexity(), ViewerState->GetSelectedMetric()), true);
					}
				}
				return FReply::Handled();
			});
		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("ThresholdMaxValueWidget"), ThresholdMaxValueWidget, FText::GetEmpty()));
	}

	{
		TSharedRef<SWidget> CellSizeWidget = SNew(Private::SNumericMenuPropertyEntry)
			.MinValue(SettingsBounds::MinCellSize).MaxValue(SettingsBounds::MaxCellSize)
			.Label(LOCTEXT("CellSizeLabel", "Cell Size"))
			.ToolTip(LOCTEXT("CellSizeToolTip", "Cell Size"))
			.Value_Lambda([this](){return ViewerState ? ViewerState->GetHeatmapCellSize() : 0;})
			.MaxDecimals(0)
			.OnValueCommitted_Lambda([this](double NewValue, ETextCommit::Type CommitType) {
				if (ViewerState)
				{
					NewValue = FMath::Clamp(NewValue, SettingsBounds::MinCellSize, SettingsBounds::MaxCellSize);
					ViewerState->SetHeatmapCellSize((uint32)NewValue);
				}
			})
			.OnResetValue_Lambda([this]() {
				if (ViewerState)
				{
					if (const UChaosVDMetricsViewSettings* DefaultSettings = UChaosVDMetricsViewSettings::GetDefaultSettings())
					{
						ViewerState->SetHeatmapCellSize(DefaultSettings->GetHeatmapCellSize(), true);
					}
				}
				return FReply::Handled();
			});
		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("CellSizeWidget"), CellSizeWidget, FText::GetEmpty()));
	}
}

void SCVDMetricsHeatmapToolbar::PopulateMenuHeatmapColorSettings(FToolMenuSection& Section)
{
	using namespace Chaos::VD::PerformanceMetrics;
	{
		TSharedRef<SWidget> LowValueColorWidget =
			SNew(Private::SColorMenuPropertyEntry)
			.Label(LOCTEXT("MinColorEntry", "Low"))
			.ToolTip(LOCTEXT("MinColorEntryToolTip", "Color for low values."))
			.Color_Lambda([this](){return ViewerState ? ViewerState->GetHeatmapColorSettings().LowValueColor : FColor::Black;})
			.OnColorChanged_Lambda([this](FColor Color){
				if (ViewerState)
				{
					FChaosVDHeatmapColorSettings& Settings = ViewerState->GetHeatmapColorSettings();
					Settings.LowValueColor = Color;
					ViewerState->SetHeatmapColorSettings(Settings);
				}
			})
			.OnResetValue_Lambda([this]() {
				if (ViewerState)
				{
					FChaosVDHeatmapColorSettings& Settings = ViewerState->GetHeatmapColorSettings();
					Settings.LowValueColor = FChaosVDHeatmapColorSettings().LowValueColor;
					ViewerState->SetHeatmapColorSettings(Settings);
				}
				return FReply::Handled();
			});

		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("MinColor"), LowValueColorWidget, FText::GetEmpty()));
	}

	{
		TSharedRef<SWidget> MidpointValueColorWidget =
			SNew(Private::SColorMenuPropertyEntry)
			.Label(LOCTEXT("MidpointColorEntry", "Midpoint"))
			.ToolTip(LOCTEXT("MidpointColorEntryToolTip", "Color for the midpoint value."))
			.Color_Lambda([this](){return ViewerState ? ViewerState->GetHeatmapColorSettings().MidpointValueColor : FColor::Black;})
			.OnColorChanged_Lambda([this](FColor Color){
				if (ViewerState)
				{
					FChaosVDHeatmapColorSettings& Settings = ViewerState->GetHeatmapColorSettings();
					Settings.MidpointValueColor = Color;
					ViewerState->SetHeatmapColorSettings(Settings);
				}
			})
			.OnResetValue_Lambda([this]() {
				if(ViewerState)
				{
					FChaosVDHeatmapColorSettings& Settings = ViewerState->GetHeatmapColorSettings();
					Settings.MidpointValueColor = FChaosVDHeatmapColorSettings().MidpointValueColor;
					ViewerState->SetHeatmapColorSettings(Settings);
				}
				return FReply::Handled();
			});

		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("MidpointValueColor"), MidpointValueColorWidget, FText::GetEmpty()));
	}

	{
		TSharedRef<SWidget> HighValueColorWidget =
			SNew(Private::SColorMenuPropertyEntry)
			.Label(LOCTEXT("HighColorEntry", "High"))
			.ToolTip(LOCTEXT("HighColorEntryToolTip", "Color for high values (reaching the threshold value)."))
			.Color_Lambda([this](){return ViewerState ? ViewerState->GetHeatmapColorSettings().HighValueColor : FColor::Black;})
			.OnColorChanged_Lambda([this](FColor Color){
				if (ViewerState)
				{
					FChaosVDHeatmapColorSettings& Settings = ViewerState->GetHeatmapColorSettings();
					Settings.HighValueColor = Color;
					ViewerState->SetHeatmapColorSettings(Settings);
				}
			})
			.OnResetValue_Lambda([this]() {
				if (ViewerState)
				{
					FChaosVDHeatmapColorSettings& Settings = ViewerState->GetHeatmapColorSettings();
					Settings.HighValueColor = FChaosVDHeatmapColorSettings().HighValueColor;
					ViewerState->SetHeatmapColorSettings(Settings);
				}
				return FReply::Handled();
			});

		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("HighValueColor"), HighValueColorWidget, FText::GetEmpty()));
	}

	{
		TSharedRef<SWidget> MaxValueColorWidget =
			SNew(Private::SColorMenuPropertyEntry)
			.Label(LOCTEXT("MaxColorEntry", "Max"))
			.ToolTip(LOCTEXT("MaxColorEntryToolTip", "Color for the maximum expected value."))
			.Color_Lambda([this](){return ViewerState ? ViewerState->GetHeatmapColorSettings().MaxValueColor : FColor::Black;})
			.OnColorChanged_Lambda([this](FColor Color){
				if (ViewerState)
				{
					FChaosVDHeatmapColorSettings& Settings = ViewerState->GetHeatmapColorSettings();
					Settings.MaxValueColor = Color;
					ViewerState->SetHeatmapColorSettings(Settings);
				}
			})
			.OnResetValue_Lambda([this]() {
				if (ViewerState)
				{
					FChaosVDHeatmapColorSettings& Settings = ViewerState->GetHeatmapColorSettings();
					Settings.MaxValueColor = FChaosVDHeatmapColorSettings().MaxValueColor;
					ViewerState->SetHeatmapColorSettings(Settings);
				}
				return FReply::Handled();
			});

		Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("MaxColor"), MaxValueColorWidget, FText::GetEmpty()));
	}
}

void SCVDMetricsHeatmapToolbar::RegisterFocusMenu()
{
	using namespace Chaos::VD::PerformanceMetrics;

	UToolMenus* ToolMenus = UToolMenus::Get();
	if (!ToolMenus || ToolMenus->IsMenuRegistered(Private::FocusMenuName))
	{
		return;
	}

	FToolMenuOwnerScoped OwnerScoped(UE_MODULE_NAME);
	UToolMenu* FocusMenu =
		ToolMenus->RegisterMenu(Private::FocusMenuName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
	FocusMenu->StyleName = "EditorViewportToolBar";

	FToolMenuSection& FocusSection = FocusMenu->FindOrAddSection("Toolbar.Focus");
	{
		const FChaosVDMetricsHeatmapCommands& Commands = FChaosVDMetricsHeatmapCommands::Get();

		const FName StyleSetName = FAppStyle::Get().GetStyleSetName();

		FToolMenuEntry& TrackFocusLocationEntry = FocusSection.AddMenuEntry(Commands.TrackEditorView);
		TrackFocusLocationEntry.Label = FText();
		TrackFocusLocationEntry.Icon = FSlateIcon(StyleSetName, "LevelEditor.SnapCameraToObject");
		TrackFocusLocationEntry.ToolBarData.BlockGroupName = "Focus";
		TrackFocusLocationEntry.UserInterfaceActionType = EUserInterfaceActionType::ToggleButton;

		FToolMenuEntry& FocusBoundsEntry = FocusSection.AddMenuEntry(Commands.FocusBounds);
		FocusBoundsEntry.Label = FText();
		FocusBoundsEntry.Icon = FSlateIcon(StyleSetName, "WorldPartition.FocusLoadedRegions");
		FocusBoundsEntry.ToolBarData.BlockGroupName = "Focus";
	}
}

#undef LOCTEXT_NAMESPACE