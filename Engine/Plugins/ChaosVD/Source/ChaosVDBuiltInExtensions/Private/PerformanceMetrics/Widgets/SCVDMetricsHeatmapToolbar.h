// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "SViewportToolBar.h"

class FUICommandList;
struct FToolMenuEntry;
struct FToolMenuSection;
class FChaosVDMetricsViewerState;

class SCVDMetricsHeatmapToolbar final : public SViewportToolBar
{
	SLATE_DECLARE_WIDGET(SCVDMetricsHeatmapToolbar, SViewportToolBar);

public:
	SLATE_BEGIN_ARGS(SCVDMetricsHeatmapToolbar) {}
		SLATE_ARGUMENT(TSharedPtr<FUICommandList>, EditorCommands)
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, TSharedPtr<FChaosVDMetricsViewerState> InViewerState);

private:
	TSharedRef<SWidget> GenerateOptionsMenu();
	TSharedRef<SWidget> GenerateFocusMenu() const;

	static void RegisterOptionsMenu();
	static void RegisterFocusMenu();

	void PopulateMenuHeatmapColorSettings(FToolMenuSection& Section);
	void PopulateMenuMetricSettings(FToolMenuSection& Section);

	TSharedPtr<FChaosVDMetricsViewerState> ViewerState;
	TSharedPtr<FUICommandList> CommandList;
	TArray<TSharedRef<FName>> ProfileNames;
};