// Copyright Epic Games, Inc. All Rights Reserved.

#include "SCVDMetricsHistogram.h"

#include "Math/Box2D.h"
#include "Styling/StyleColors.h"
#include "Widgets/Layout/SScrollBar.h"
#include "Widgets/Layout/SSpacer.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Layout/SGridPanel.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Layout/SSeparator.h"
#include "Framework/Commands/UICommandList.h"
#include "Algo/Sort.h"
#include "SChaosVDMetricsViewerState.h"
#include "Widgets/Input/SComboButton.h"
#include "Misc/DefaultValueHelper.h"
#include "Math/UnitConversion.h"
#include "PerformanceMetrics/ChaosVDMetrics.h"
#include "ToolMenu.h"
#include "ToolMenuContext.h"
#include "ToolMenus.h"

namespace Chaos::VD::PerformanceMetrics::Private
{

static constexpr float TopPadding = 10.f;
static const FName HistogramOptionsMenuName(TEXT("CVDMetricsHistogram.OptionsMenu"));

static constexpr FLinearColor BarSelectionColor{1.0f, 1.0f, 0.6f};
static constexpr FLinearColor BarHoverColor{1.0f, 1.0f, 1.0f};
static constexpr FLinearColor BarColors[] = {FLinearColor{0.4f, 0.4f, 0.4f}, FLinearColor{0.2f, 0.2f, 0.2f}};

}

#define LOCTEXT_NAMESPACE "SCVDMetricsHistogram"

FString FExponentialTypeInterface::ToString(const double& Value) const
{
    return Value > 1000000000000 ? TEXT("MAX") : FString::Printf(TEXT("%.5g"), Value);
}

bool FExponentialTypeInterface::IsCharacterValid(TCHAR InChar) const
{
	return FChar::IsDigit(InChar)
		|| InChar == TEXT('.')
		|| InChar == TEXT('-')
		|| InChar == TEXT('+')
		|| InChar == TEXT('e')
		|| InChar == TEXT('E');
}

TOptional<double> FExponentialTypeInterface::FromString(const FString& InString, const double& ExistingValue)
{
	double Result = 0.0;
	if (LexTryParseString(Result, *InString))
	{
		return Result;
	}
	return TOptional<double>();
}

SLATE_IMPLEMENT_WIDGET(SCVDMetricsHistogramPanelView)

void SCVDMetricsHistogramPanelView::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

SCVDMetricsHistogramPanelView::SCVDMetricsHistogramPanelView()
{
}

void SCVDMetricsHistogramPanelView::Construct(const FArguments& InArgs, TSharedPtr<FChaosVDMetricsViewerState> InViewerState)
{
	ViewerState = InViewerState;

	HistogramData = MakeShared<SCVDMetricsHistogramData>();

	ViewerState->OnSelectedMetricChanged().AddSPLambda(this, [this](const ChaosVDParticleMetricsType& Metrics, const ChaosVDCollisionComplexityFilteringOptions& Complexity)
	{
		UpdateHistogram();
	});
	ViewerState->OnSelectionBoxChanged().AddSPLambda(this, [this](const FBox2D& Selection)
	{
		UpdateHistogram();
	});

	ExponentialFormat = MakeShared<FExponentialTypeInterface>();

	ChildSlot
	.VAlign(VAlign_Fill)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew(SCVDMetricsHistogramCanvas, HistogramData, ViewerState)
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SBox)
			.Padding(10)
			[
				SNew(SGridPanel)
					.FillColumn(1, 1.f)
					+ SGridPanel::Slot(0, 0)
					.Padding(4.f)
					.VAlign(VAlign_Center)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxLabel", "Histogram"))
					]
					+ SGridPanel::Slot(1, 0)
					.Padding(4.f)
					.HAlign(HAlign_Left)  
					.VAlign(VAlign_Center)
					[
						SNew(SBox)
						.WidthOverride(18)
						.HeightOverride(18)
						[
							SNew(SComboButton)
								.OnGetMenuContent(this, &SCVDMetricsHistogramPanelView::GetHistogramSettingsMenuContent)
								.ContentPadding(FMargin(1.f, 2.f))
								.HasDownArrow(false)
								.ButtonStyle(FAppStyle::Get(), "NoBorder")
								.ButtonContent()

								[
									SNew(SImage)
										.ColorAndOpacity(FSlateColor::UseForeground())
										.DesiredSizeOverride(FVector2D(18, 18))
										.Image(FAppStyle::GetBrush("Icons.Toolbar.Settings"))
								]
						]
					]
					+ SGridPanel::Slot(0, 1)
					.Padding(4.f)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxValues", "Values:"))
					]
					+ SGridPanel::Slot(1, 1)
					.Padding(4.f)
					[
						SNew(STextBlock).Text_Lambda([this]()
						{
							return FText::AsNumber(HistogramData ? HistogramData->NumValues : 0);
						})
					]
					+ SGridPanel::Slot(0, 2)
					.Padding(4.f)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxMin", "Min:"))
					]
					+ SGridPanel::Slot(1, 2)
					.Padding(4.f)
					[
						SNew(STextBlock).Text_Lambda([this]()
						{
							return FText::AsNumber(HistogramData && HistogramData->NumValues ? HistogramData->MinStat : 0);
						})
					]
					+ SGridPanel::Slot(0, 3)
					.Padding(4.f)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxMax", "Max:"))
					]
					+ SGridPanel::Slot(1, 3)
					.Padding(4.f)
					[
						SNew(STextBlock).Text_Lambda([this]()
						{
							return FText::AsNumber(HistogramData && HistogramData->NumValues ? HistogramData->MaxStat : 0);
						})
					]
					+ SGridPanel::Slot(0, 4)
					.Padding(4.f)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxAvg", "Avg:"))
					]
					+ SGridPanel::Slot(1, 4)
					.Padding(4.f)
					[
						SNew(STextBlock).Text_Lambda([this]()
						{
							return FText::AsNumber(HistogramData && HistogramData->NumValues ? HistogramData->AvgStat : 0);
						})
					]
					+ SGridPanel::Slot(0, 5)
					.Padding(4.f)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxMed", "Med:"))
					]
					+ SGridPanel::Slot(1, 5)
					.Padding(4.f)
					[
						SNew(STextBlock).Text_Lambda([this]() 
						{
							return FText::AsNumber(HistogramData && HistogramData->NumValues ? HistogramData->MedianStat : 0);
						})
					]
					+ SGridPanel::Slot(0, 6)
					.ColumnSpan(2)
					.Padding(0,2)
					[
						SNew(SSeparator)
						.Orientation(Orient_Horizontal)
					]
					+ SGridPanel::Slot(0, 7)
					.Padding(4.f)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxNumBins", "Num Bins:"))
					]
					+ SGridPanel::Slot(1, 7)
					.Padding(4.f)
					[
						SNew(STextBlock).Text_Lambda([this]()
						{
							return FText::AsNumber(HistogramData ? HistogramData->NumBins : 0);
						})
					]
					+ SGridPanel::Slot(0, 8)
					.Padding(4.f)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxLowThreshold", "Lower Bound:"))
					]
					+ SGridPanel::Slot(1, 8)
					.Padding(4.f)
					[
						SNew(STextBlock).Text_Lambda([this]()
						{
							return FText::FromString(ExponentialFormat->ToString(HistogramData ? HistogramData->LowerThreshold : 0)); 
						})
					]
					+ SGridPanel::Slot(0, 9)
					.Padding(4.f)
					[
						SNew(STextBlock).Text(LOCTEXT("HistogramSideBoxUpperThreshold", "Upper Bound:"))
					]
					+ SGridPanel::Slot(1, 9)
					.Padding(4.f)
					[
						SNew(STextBlock).Text_Lambda([this]()
						{
							return FText::FromString(ExponentialFormat->ToString(HistogramData ? HistogramData->UpperThreshold : 0));
						})
					]
			]
		]
		
	];
}

void SCVDMetricsHistogramPanelView::UpdateHistogram()
{
	if (!HistogramData)
	{
		HistogramData = MakeShared<SCVDMetricsHistogramData>();

		if (!HistogramData)
		{
			return;
		}
	}

	HistogramData->MinStat = 10000000000;
	HistogramData->MaxStat = 0;
	HistogramData->AvgStat = 0;
	HistogramData->MedianStat = 0;
	HistogramData->NumValues = 0;
	HistogramData->MaxBinCount = 0;
	HistogramData->MinBinCount = 1000000000;

	HistogramData->Bins.Reset();

	if (!ViewerState || !ViewerState->IsParticleDataValid())
	{
		return;
	}

	TArray<TSharedPtr<FParticleMetricEntry>> ParticleMetrics = *(ViewerState->GetParticleEntries());

	HistogramData->Bins.Reserve(HistogramData->NumBins);
	HistogramData->NumValues = ParticleMetrics.Num();

	for (uint32 Index = 0; Index < HistogramData->NumBins; Index++)
	{
		HistogramData->Bins.Add(0);
	}

	if (!HistogramData->NumValues)
	{
		return;
	}

	TArray<double> Values;
	Values.Reserve(HistogramData->NumValues);

	const FBox2D& SelectionBox = ViewerState->GetSelectionBox();
	const FBox& SelectionBox3D = FBox(FVector(SelectionBox.Min.X,SelectionBox.Min.Y,0),FVector(SelectionBox.Max.X,SelectionBox.Max.Y,0));

	for (const TSharedPtr<FParticleMetricEntry>& Particle : ParticleMetrics)
	{
		double Stat = Particle->GetMetric(ViewerState->GetSelectedComplexity(), ViewerState->GetSelectedMetric());

		if ((SelectionBox.GetArea() > 0 && !Particle->ParticleBounds.IntersectXY(SelectionBox3D)) || Stat > HistogramData->UpperThreshold || Stat < HistogramData->LowerThreshold)
		{
			continue;
		}

		Values.Add(Stat);
		HistogramData->AvgStat += Stat;
		HistogramData->MinStat = FMath::Min(HistogramData->MinStat, Stat);
		HistogramData->MaxStat = FMath::Max(HistogramData->MaxStat, Stat);
	}

	HistogramData->NumValues = Values.Num();

	if (!HistogramData->NumValues)
	{
		return;
	}

	HistogramData->AvgStat = HistogramData->NumValues ? HistogramData->AvgStat / HistogramData->NumValues : 0;

	int32 Mid = Values.Num() / 2;
    Algo::Sort(Values); 
	HistogramData->MedianStat = Values[Mid];

	double BinSize = (HistogramData->MaxStat + 1 - HistogramData->MinStat) / HistogramData->NumBins;

	for (double Metric : Values)
	{
		uint32 Index = (uint32)((Metric - HistogramData->MinStat) / BinSize);
		Index = Index < HistogramData->NumBins ? Index : 0;
		HistogramData->Bins[Index] += 1;
	}

	for (uint32 Index = 0; Index < HistogramData->NumBins; Index++)
	{
		HistogramData->MinBinCount = FMath::Min(HistogramData->Bins[Index], HistogramData->MinBinCount);
		HistogramData->MaxBinCount = FMath::Max(HistogramData->Bins[Index], HistogramData->MaxBinCount);
	}
}

TSharedRef<SWidget> SCVDMetricsHistogramPanelView::GetHistogramSettingsMenuContent()
{
	using namespace Chaos::VD::PerformanceMetrics;

	UToolMenus* ToolMenus = UToolMenus::Get();
	
	UToolMenu* Menu = ToolMenus->FindMenu(Private::HistogramOptionsMenuName);
	if (!Menu || !ToolMenus->IsMenuRegistered(Private::HistogramOptionsMenuName))
	{
		Menu = UToolMenus::Get()->RegisterMenu(Private::HistogramOptionsMenuName, NAME_None, EMultiBoxType::Menu);
	}
	
	Menu->Sections.Empty();
	
	FToolMenuSection& Section = Menu->AddSection(FName("ChaosVDMetricsHistogramSection"), LOCTEXT("HistogramSectionLabel", "Histogram Settings"));

	TSharedRef<SWidget> NumBinsWidget = SNew(SNumericEntryBox<uint32>).Value_Lambda([this]()
		{
			return HistogramData->NumBins;
		})
		.AllowSpin(true)
		.MinValue(5)
		.MaxValue(20)
		.OnValueChanged_Lambda([this](uint32 InValue) {
			HistogramData->NumBins = InValue;
		})
		.OnValueCommitted_Lambda([this](uint32 InValue, ETextCommit::Type InCommit){
			HistogramData->NumBins = InValue;
			UpdateHistogram();
		});

	Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("NumBinsWidget"), NumBinsWidget, LOCTEXT("HistogramNumBinsLabel", "Num Bins")));

	TSharedRef<SWidget> LowThresholdWidget = SNew(SNumericEntryBox<double>).Value_Lambda([this]()
		{
			return HistogramData->LowerThreshold;
		})
		.MinValue(0)
		.MaxValue_Lambda([this]()
		{
			return HistogramData->UpperThreshold - 0.05;
		})
		.OnValueChanged_Lambda([this](double InValue)
		{
			HistogramData->LowerThreshold = FMath::Clamp(InValue, 0, HistogramData->UpperThreshold - 0.05);
		})
		.OnValueCommitted_Lambda([this](double InValue, ETextCommit::Type InCommit)
		{
			HistogramData->LowerThreshold = FMath::Clamp(InValue, 0, HistogramData->UpperThreshold - 0.05);
			UpdateHistogram();
		})
		.TypeInterface(ExponentialFormat);

	Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("LowThresholdWidget"), LowThresholdWidget, LOCTEXT("HistogramLowLabel", "Lower Cutoff")));

	TSharedRef<SWidget> HighThresholdWidget = SNew(SNumericEntryBox<double>).Value_Lambda([this]()
		{
			return HistogramData->UpperThreshold;
		})
		.MinValue_Lambda([this]()
		{
			return HistogramData->LowerThreshold + 0.05;
		})
		.OnValueChanged_Lambda([this](double InValue)
		{
			HistogramData->UpperThreshold = FMath::Max(InValue, HistogramData->LowerThreshold + 0.05);
		})
		.OnValueCommitted_Lambda([this](double InValue, ETextCommit::Type InCommit)
		{
			HistogramData->UpperThreshold = FMath::Max(InValue, HistogramData->LowerThreshold + 0.05);
			UpdateHistogram();
		})
		.TypeInterface(ExponentialFormat);

	Section.AddEntry(FToolMenuEntry::InitWidget(TEXT("UpperThresholdWidget"), HighThresholdWidget, LOCTEXT("HistogramUpperLabel", "Upper Cutoff")));

	return ToolMenus->GenerateWidget(Menu);
}

SLATE_IMPLEMENT_WIDGET(SCVDMetricsHistogramCanvas)

void SCVDMetricsHistogramCanvas::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

SCVDMetricsHistogramCanvas::SCVDMetricsHistogramCanvas()
{
}

void SCVDMetricsHistogramCanvas::Construct(const FArguments& InArgs, TSharedPtr<SCVDMetricsHistogramData>& InHistogramData, TSharedPtr<FChaosVDMetricsViewerState>& InViewerState)
{
	HistogramData = InHistogramData;
	ViewerState = InViewerState;

	const TAttribute<FText> ToolTipTextAttribute(this, &SCVDMetricsHistogramCanvas::GetTooltipText);
	SetToolTipText(ToolTipTextAttribute);

	ChildSlot
	.VAlign(VAlign_Bottom)
	[
		SNew(SHorizontalBox)
		+SHorizontalBox::Slot()
		[
			SNew(SOverlay)
			+SOverlay::Slot()
			[
				SAssignNew(ScrollBarAreaWidget, SSpacer)
			]			
			+SOverlay::Slot()
			[
				SAssignNew(ScrollBar, SScrollBar)
				.Orientation(Orient_Horizontal)
			]
		]
		+SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(SSpacer)
		]
		
	];
}

int32 SCVDMetricsHistogramCanvas::OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const
{
	LayerId = PaintBackground(AllottedGeometry, OutDrawElements, LayerId, InWidgetStyle);
	LayerId = PaintValues(HistogramBarsGeometry, OutDrawElements, LayerId);

	return Super::OnPaint(
		Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

int32 SCVDMetricsHistogramCanvas::PaintBackground(
	const FGeometry& AllottedGeometry,
	FSlateWindowElementList& OutDrawElements,
	const int32 LayerId,
	const FWidgetStyle& InWidgetStyle) const
{
	const FSlateBrush* ImageBrush = FAppStyle::GetBrush(TEXT("WhiteTexture"));
	const FSlateColor ColorAndOpacity = FStyleColors::Recessed;
	const FLinearColor FinalColorAndOpacity(
		InWidgetStyle.GetColorAndOpacityTint() * ColorAndOpacity.GetColor(InWidgetStyle) *
		ImageBrush->GetTint(InWidgetStyle));

	FSlateDrawElement::MakeBox(
		OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), ImageBrush, ESlateDrawEffect::None,
		FinalColorAndOpacity);

	return LayerId + 1;
}

int32 SCVDMetricsHistogramCanvas::PaintValues(
	const FGeometry& AllottedGeometry,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId) const
{
	using namespace Chaos::VD::PerformanceMetrics;

	if (!ViewerState || !HistogramData || HistogramData->NumValues <= 0 || HistogramData->Bins.Num() <= 0)
	{
		return LayerId + 1;
	}

	int32 MouseHoverBarIndex = INDEX_NONE;
	if (HistogramBarsGeometry.IsUnderLocation(GetPaintSpaceGeometry().LocalToAbsolute(CursorScreenSpacePosition)))
	{
		MouseHoverBarIndex = GetIndexFromScreenSpacePosition(CursorScreenSpacePosition);
	}

	uint32 NumBarsDrawn = HistogramData->Bins.Num();
	uint32 ScrollOffset = 0;
	float HistogramBarScale = 1;
	float HistogramBarPadding = 1;
	float HistogramBarWidth = (AllottedGeometry.GetLocalSize().X - HistogramBarPadding * NumBarsDrawn) / NumBarsDrawn;
	LastBarWidth = HistogramBarWidth;

	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush(TEXT("WhiteTexture"));

	for (uint32 Index = 0; Index < NumBarsDrawn; Index++)
	{
		const float NormalizedValue = (HistogramData->MaxBinCount == HistogramData->MinBinCount) ? 1.0f / NumBarsDrawn : 
			((float)(HistogramData->Bins[Index] - HistogramData->MinBinCount))/(HistogramData->MaxBinCount - HistogramData->MinBinCount);
		const float BarHeight = AllottedGeometry.GetLocalSize().Y * NormalizedValue * HistogramBarScale;

		const FVector2D BarSize(HistogramBarWidth, BarHeight);
		const FVector2D BarPosition(
			(HistogramBarPadding + HistogramBarWidth) * Index, AllottedGeometry.GetLocalSize().Y - BarHeight);
		const FPaintGeometry BarGeometry =
			AllottedGeometry.ToPaintGeometry(BarSize, FSlateLayoutTransform(BarPosition));

		FLinearColor BarColor = Private::BarColors[Index % 2];

		if (Index == SelectedBarIndex)
		{
			BarColor = Private::BarSelectionColor;

			const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();

			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { FVector2d(BarPosition.X - 1, AllottedGeometry.GetLocalSize().Y), FVector2d(BarPosition.X - 1, 0) }, ESlateDrawEffect::None, BarColor, false, 1);
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { FVector2d(BarPosition.X + HistogramBarWidth + 1, AllottedGeometry.GetLocalSize().Y), FVector2d(BarPosition.X + HistogramBarWidth + 1, 0) }, 
				ESlateDrawEffect::None, BarColor, false, 1);
		}

		if (Index == MouseHoverBarIndex)
		{
			BarColor = Private::BarHoverColor;
		}

		FSlateDrawElement::MakeBox(
			OutDrawElements, ++LayerId, BarGeometry, WhiteBrush, ESlateDrawEffect::None, BarColor);
	}

	return LayerId + 1;
}

FReply SCVDMetricsHistogramCanvas::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton)
	{
		if (!HistogramData || HistogramData->NumBins <= 0)
		{
			SelectedBarIndex = INDEX_NONE;
		}
		else
		{
			const int32 Index = GetIndexFromScreenSpacePosition(CursorScreenSpacePosition);

			double BinSize = (HistogramData->MaxStat + 1 - HistogramData->MinStat) / HistogramData->NumBins;

			if (SelectedBarIndex == Index)
			{
				ViewerState->SetHistogramFilter(-1, -1);
				SelectedBarIndex = INDEX_NONE;
			}
			else
			{
				ViewerState->SetHistogramFilter(Index * BinSize + HistogramData->MinStat, (Index + 1) * BinSize + HistogramData->MinStat);
				SelectedBarIndex = Index;
			}
		}
	}

	return SCompoundWidget::OnMouseButtonUp(MyGeometry, MouseEvent);
}

FReply SCVDMetricsHistogramCanvas::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	CursorScreenSpacePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	return SCompoundWidget::OnMouseMove(MyGeometry, MouseEvent);
}

void SCVDMetricsHistogramCanvas::Tick(
	const FGeometry& AllottedGeometry,
	const double InCurrentTime,
	const float InDeltaTime)
{
	HistogramBarsGeometry = GetHistogramBarsGeometry(GetPaintSpaceGeometry());
}

FText SCVDMetricsHistogramCanvas::GetTooltipText() const
{
	if (!HistogramBarsGeometry.IsUnderLocation(GetPaintSpaceGeometry().LocalToAbsolute(CursorScreenSpacePosition)))
	{
		return FText::GetEmpty();
	}

	if (!ViewerState || !HistogramData || HistogramData->Bins.IsEmpty())
	{
		return FText::GetEmpty();
	}

	const int32 Index = GetIndexFromScreenSpacePosition(CursorScreenSpacePosition);
	if (Index == INDEX_NONE)
	{
		return FText::GetEmpty();
	}

	double BinSize = (HistogramData->MaxStat - HistogramData->MinStat) / HistogramData->Bins.Num();

	return FText::Format(
			LOCTEXT("HistogramValueText", "Range: {0} - {1}\nNum: {2}"), FText::AsNumber(HistogramData->MinStat + BinSize * Index), FText::AsNumber( HistogramData->MinStat + BinSize * (Index + 1)), FText::AsNumber(HistogramData->Bins[Index]));
}

FGeometry SCVDMetricsHistogramCanvas::GetScrollBarAreaGeometry() const
{
	return FindChildGeometry(HistogramBarsGeometry, ScrollBarAreaWidget.ToSharedRef());
}

FGeometry SCVDMetricsHistogramCanvas::GetHistogramBarsGeometry(const FGeometry& AllottedGeometry) const
{
	using namespace Chaos::VD::PerformanceMetrics;

	const FGeometry ScrollBarAreaGeometry = GetScrollBarAreaGeometry();
	const float ScrollBarHeight = ScrollBarAreaGeometry.GetLocalSize().Y;

	FVector2f NewSize = AllottedGeometry.GetLocalSize();
	NewSize.X -= 10;
	NewSize.Y -= (Private::TopPadding * 2.0f) + ScrollBarHeight;

	return AllottedGeometry.MakeChild(NewSize.ComponentMax(FVector2f::ZeroVector), FSlateLayoutTransform(1.0f, FVector2f(0.0f, 0.0f)));
}

FGeometry SCVDMetricsHistogramCanvas::GetClippingGeometry(const FGeometry& AllottedGeometry) const
{
	return GetHistogramBarsGeometry(AllottedGeometry);
}

int32 SCVDMetricsHistogramCanvas::GetIndexFromScreenSpacePosition(const FVector2D& ScreenSpacePosition) const
{
	if (!HistogramData || HistogramData->Bins.IsEmpty())
	{
		return INDEX_NONE;
	}

	const float BarWidth = LastBarWidth;
	return FMath::Min(HistogramData->Bins.Num() - 1, ScreenSpacePosition.X / BarWidth);
}


#undef LOCTEXT_NAMESPACE