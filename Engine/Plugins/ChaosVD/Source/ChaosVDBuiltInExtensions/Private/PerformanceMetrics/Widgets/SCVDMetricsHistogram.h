// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Widgets/SCompoundWidget.h"
#include "Widgets/Input/NumericTypeInterface.h"

class FUICommandList;
class SScrollBar;
class FChaosVDMetricsViewerState;
struct FParticleMetricEntry;

class FExponentialTypeInterface : public INumericTypeInterface<double>
{
public:
    virtual FString ToString(const double& Value) const override;

    virtual TOptional<double> FromString(const FString& InString, const double& ExistingValue) override;

    virtual bool IsCharacterValid(TCHAR InChar) const override;

    virtual int32 GetMinFractionalDigits() const override { 
		return 0;
	}
    virtual int32 GetMaxFractionalDigits() const override { 
		return 6;
	}
    virtual void SetMinFractionalDigits(const TAttribute<TOptional<int32>>& InMinFractionalDigits) override {}
    virtual void SetMaxFractionalDigits(const TAttribute<TOptional<int32>>& InMaxFractionalDigits) override {}
};

struct SCVDMetricsHistogramData
{
	double MinStat = 10000000000;
	double MaxStat = 0;
	double AvgStat = 0;
	double MedianStat = 0;
	uint32 NumValues = 0;

	uint32 MaxBinCount = 0;
	uint32 MinBinCount = 1000000000;

	uint32 NumBins = 10;
	double LowerThreshold = 0;
	double UpperThreshold = std::numeric_limits<double>::max();

	TArray<uint32> Bins;
};

class SCVDMetricsHistogramPanelView final : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SCVDMetricsHistogramPanelView, SCompoundWidget);

public:
	SLATE_BEGIN_ARGS(SCVDMetricsHistogramPanelView) {}
	SLATE_END_ARGS()

	SCVDMetricsHistogramPanelView();

	void Construct(const FArguments& InArgs, TSharedPtr<FChaosVDMetricsViewerState> InViewerState);

	void UpdateHistogram();

	TSharedRef<SWidget> GetHistogramSettingsMenuContent();

private:
	TSharedPtr<FChaosVDMetricsViewerState> ViewerState;
	TSharedPtr<SCVDMetricsHistogramData> HistogramData;
	
	TSharedPtr<FExponentialTypeInterface> ExponentialFormat;

	float SideBarWidth = 300;

};

class SCVDMetricsHistogramCanvas final : public SCompoundWidget
{
	SLATE_DECLARE_WIDGET(SCVDMetricsHistogramCanvas, SCompoundWidget);

public:
	SLATE_BEGIN_ARGS(SCVDMetricsHistogramCanvas) {}
	SLATE_END_ARGS()

	SCVDMetricsHistogramCanvas();

	void Construct(const FArguments& InArgs, TSharedPtr<SCVDMetricsHistogramData>& InHistogramData, TSharedPtr<FChaosVDMetricsViewerState>& InViewerState);
	
private:
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	virtual int32 OnPaint(const FPaintArgs& Args, const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle, bool bParentEnabled) const override;
	virtual FReply OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual FReply OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;

	int32 PaintBackground(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId, const FWidgetStyle& InWidgetStyle) const;
	int32 PaintValues(const FGeometry& AllottedGeometry, FSlateWindowElementList& OutDrawElements, int32 LayerId) const;

	FGeometry GetScrollBarAreaGeometry() const;
	FGeometry GetHistogramBarsGeometry(const FGeometry& AllottedGeometry) const;

	FGeometry GetClippingGeometry(const FGeometry& AllottedGeometry) const;
	int32 GetIndexFromScreenSpacePosition(const FVector2D& ScreenSpacePosition) const;
	FText GetTooltipText() const;

	TSharedPtr<SScrollBar> ScrollBar;
	TSharedPtr<SWidget> ScrollBarAreaWidget;
	TSharedPtr<SCVDMetricsHistogramData> HistogramData;
	TSharedPtr<FChaosVDMetricsViewerState> ViewerState;
	
	FGeometry HistogramBarsGeometry;
	
	FVector2D CursorScreenSpacePosition = FVector2D::ZeroVector;

	mutable float LastBarWidth = 1;
	int32 SelectedBarIndex = INDEX_NONE;
};
