// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDMetricsView.h"
#include "Widgets/Input/SSearchBox.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "ChaosVDScene.h"
#include "TEDS/ChaosVDStructTypedElementData.h"
#include "SChaosVDMetricsViewerState.h"

#define LOCTEXT_NAMESPACE "ChaosVDMetricsView"

SChaosVDMetricsView::FColumnNames SChaosVDMetricsView::ColumnNames = SChaosVDMetricsView::FColumnNames();

void SParticleMetricRowWidget::Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FParticleMetricEntry> InItem, ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metric, TSharedPtr<FParticleMetricEntry> InSumMetric)
{
	Item = InItem;
	SelectedComplexity = Complexity;
	SelectedMetric = Metric;
	SumMetric = InSumMetric;
	SMultiColumnTableRow<TSharedPtr<FParticleMetricEntry>>::Construct(FSuperRowType::FArguments(), InOwnerTable);
}

TSharedRef<SWidget> SParticleMetricRowWidget::GenerateWidgetForColumn(const FName& ColumnName)
{
	FText ItemText = FText::GetEmpty();
	FText TooltipText = FText::GetEmpty();

	FNumberFormattingOptions MetricFormatOptions;
	MetricFormatOptions.SetMinimumFractionalDigits(2);
	MetricFormatOptions.SetMaximumFractionalDigits(2);

	ETextJustify::Type Alignment = ETextJustify::Left;

	if (ColumnName == SChaosVDMetricsView::ColumnNames.Name)
	{
		ItemText = FText::FromName(Item->ParticleName);
		TooltipText = ItemText;
	}
	else if (ColumnName == SChaosVDMetricsView::ColumnNames.Metric)
	{
		const double Metric = Item->GetMetric(SelectedComplexity, SelectedMetric);
		ItemText = FText::AsNumber(Metric, &MetricFormatOptions);
		
		if (SelectedMetric == ChaosVDParticleMetricsType::PrimitiveDensity)
		{
			const double Volume = Item->GetVolumeSafe();

			TooltipText = FText::Format(LOCTEXT("PrimitiveDensityTooltip", "{0}/{1}"),
				FText::AsNumber(Metric * Volume, &MetricFormatOptions),
				FText::AsNumber(Volume, &MetricFormatOptions)
			); 
		}
		else
		{
			TooltipText = ItemText;
		}

		Alignment = ETextJustify::Right;
	}
	else if (ColumnName == SChaosVDMetricsView::ColumnNames.Percentage)
	{
		const double TotalMetric = SumMetric->GetMetric(SelectedComplexity, SelectedMetric);
		ItemText = FText::AsNumber(TotalMetric == 0.0 ? 0 : Item->GetMetric(SelectedComplexity, SelectedMetric) / TotalMetric * 100, &MetricFormatOptions);
		TooltipText = ItemText;
		Alignment = ETextJustify::Right;
	}

	return SNew(STextBlock)
		.Text(ItemText)
		.ToolTipText(TooltipText)
		.OverflowPolicy(ETextOverflowPolicy::MiddleEllipsis)
		.Justification(Alignment);
}

void SChaosVDMetricsView::Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene> InCVDScene, TSharedPtr<FChaosVDMetricsViewerState> InViewerState)
{
	SumStatistics = MakeShared<FParticleMetricEntry>();
	ViewerState = InViewerState;

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.VAlign(VAlign_Center)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.FillWidth(1.0f)
			.VAlign(VAlign_Center)
			[
				SAssignNew(SearchBox, SSearchBox)
				.HintText(LOCTEXT("SearchBoxHint", "Search by objects"))
				.OnTextChanged_Lambda([this](const FText& SearchText)
				{
					this->ApplyFiltering();
				})
				.IsEnabled_Lambda([this]()
				{
					return ViewerState && ViewerState->IsParticleDataValid();
				})
				.ToolTipText(LOCTEXT("FilterSearchHint", "Type here to search by object name."))
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1)
		.Padding(0.0f, 8.0f, 0.0f, 0.0f)
		[
			SAssignNew(ComplexityListView, SListView<TSharedPtr<::FParticleMetricEntry>>)
			.Orientation(Orient_Vertical)
			.ListItemsSource(&FilteredParticleEntries)
			.OnGenerateRow_Raw(this, &SChaosVDMetricsView::GenerateComplexityEntryRow)
			.OnMouseButtonDoubleClick(this, &SChaosVDMetricsView::HandleListDoubleClick)
			.OnSelectionChanged(this, &SChaosVDMetricsView::HandleListSelectionChanged)
			.SelectionMode(ESelectionMode::Single)
			.HeaderRow
			(
				SNew(SHeaderRow)
				+ SHeaderRow::Column(SChaosVDMetricsView::ColumnNames.Name).DefaultLabel(LOCTEXT("ColumnNameLabel", "Name")).FillWidth(4)
				+ SHeaderRow::Column(SChaosVDMetricsView::ColumnNames.Metric).DefaultLabel(LOCTEXT("ColumnMetricLabel", "Metric")).FillWidth(1)
				[
					SNew( SBox )
					.HeightOverride(24.0f)
					.Padding(0.0f)
					.VAlign(VAlign_Center)
					[
						SAssignNew(MetricHeaderText, STextBlock)
							.Text(LOCTEXT("ColumnMetricLabel", "Metric"))
					]
				]
				+ SHeaderRow::Column(SChaosVDMetricsView::ColumnNames.Percentage).DefaultLabel(LOCTEXT("ColumnPercentageLabel", "Percentage")).FillWidth(1)
			)
		]
	];

	CVDScene = InCVDScene;

	ViewerState->OnSelectedMetricChanged().AddSPLambda(this, [this](const ChaosVDParticleMetricsType& Metrics, const ChaosVDCollisionComplexityFilteringOptions& Complexity)
	{
		ApplyFiltering();
	});
	ViewerState->OnSelectionBoxChanged().AddSPLambda(this, [this](const FBox2D& Selection)
	{
		ApplyFiltering();
	});
	ViewerState->OnHistogramFilterChanged().AddSPLambda(this, [this](double Min, double Max)
	{
		ApplyFiltering();
	});

	ApplyFiltering();
}

TSharedRef<ITableRow> SChaosVDMetricsView::GenerateComplexityEntryRow(TSharedPtr<FParticleMetricEntry> ParticleComplexityEntry, const TSharedRef<STableViewBase>& TableViewBase) const
{
	return SNew(SParticleMetricRowWidget, TableViewBase, ParticleComplexityEntry, ViewerState->GetSelectedComplexity(), ViewerState->GetSelectedMetric(), SumStatistics);
}
void SChaosVDMetricsView::HandleListDoubleClick(TSharedPtr<FParticleMetricEntry> ParticleComplexityEntry)
{
	if (!ParticleComplexityEntry)
	{
		return;
	}
	TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin();
	if (!ScenePtr)
	{
		return;
	}
	TSharedPtr<FChaosVDSceneParticle> ParticleInstance = ScenePtr->GetParticleInstance(ParticleComplexityEntry->SolverID, ParticleComplexityEntry->ParticleID);
	if (!ParticleInstance)
	{
		return;
	}
	
	ScenePtr->SetSelected(Chaos::VD::TypedElementDataUtil::AcquireTypedElementHandleForStruct(ParticleInstance.Get(), true));
	ScenePtr->OnFocusRequest().Broadcast(ParticleInstance->GetBoundingBox());
}
void SChaosVDMetricsView::HandleListSelectionChanged(TSharedPtr<FParticleMetricEntry> ParticleComplexityEntry,ESelectInfo::Type SelectionType)
{
	if (!ParticleComplexityEntry)
	{
		return;
	}
	TSharedPtr<FChaosVDScene> ScenePtr = CVDScene.Pin();
	if (!ScenePtr)
	{
		return;
	}
	TSharedPtr<FChaosVDSceneParticle> ParticleInstance = ScenePtr->GetParticleInstance(ParticleComplexityEntry->SolverID, ParticleComplexityEntry->ParticleID);
	if (!ParticleInstance)
	{
		return;
	}
}

void SChaosVDMetricsView::ApplyFiltering()
{
	FilteredParticleEntries.Reset();

	if (!ViewerState || !ViewerState->IsParticleDataValid())
	{
		ComplexityListView->RebuildList();
		return;
	}

	TArray<TSharedPtr<FParticleMetricEntry>> ParticleMetrics = *(ViewerState->GetParticleEntries());

	*SumStatistics = FParticleMetricEntry();

	const FBox2D& SelectionBox = ViewerState->GetSelectionBox();

	const FBox& SelectionBox3D = FBox(FVector(SelectionBox.Min.X,SelectionBox.Min.Y,0),FVector(SelectionBox.Max.X,SelectionBox.Max.Y,0));

	double HistogramMin;
	double HistogramMax;

	ViewerState->GetHistogramFilter(HistogramMin, HistogramMax);

	for (const TSharedPtr<FParticleMetricEntry>& Metric : ParticleMetrics)
	{
		SumStatistics->Aggregate(*Metric);

		double MetricValue = Metric->GetMetric(ViewerState->GetSelectedComplexity(), ViewerState->GetSelectedMetric());
		bool bInHistogramRange = HistogramMin < 0 || HistogramMax < 0 || (MetricValue >= HistogramMin && MetricValue <= HistogramMax);

		bool bInHeatmapSelection = (SelectionBox.GetArea() == 0 || Metric->ParticleBounds.IntersectXY(SelectionBox3D));

		bool bInNameFilter = Metric->ParticleName.ToString().ToLower().Contains(SearchBox->GetText().ToString().ToLower());

		if (bInHistogramRange && bInHeatmapSelection && bInNameFilter)
		{
			FilteredParticleEntries.Add(Metric);
		}
	}

	FilteredParticleEntries.Sort([this](const TSharedPtr<FParticleMetricEntry>& Entry1, const TSharedPtr<FParticleMetricEntry>& Entry2)
	{
		return Entry1->GetMetric(ViewerState->GetSelectedComplexity(), ViewerState->GetSelectedMetric()) > Entry2->GetMetric(ViewerState->GetSelectedComplexity(),ViewerState->GetSelectedMetric());
	});

	switch (ViewerState->GetSelectedMetric())
	{
		case ChaosVDParticleMetricsType::MemoryUsage:
			MetricHeaderText->SetText(LOCTEXT("MemoryUsageSelectorLabel", "Memory Usage"));
			MetricHeaderText->SetToolTipText(LOCTEXT("MemoryUsageToolTipLabel", "Memory Usage in Bytes"));
			break;
		case ChaosVDParticleMetricsType::PrimitiveDensity:
			MetricHeaderText->SetText(LOCTEXT("PrimitiveDensitySelectorLabel", "Primitive Density"));
			MetricHeaderText->SetToolTipText(LOCTEXT("PrimitiveDensityToolTipLabel", "Density as #Primitives / M^3"));
			break;
	}

	ComplexityListView->RebuildList();
}

#undef LOCTEXT_NAMESPACE