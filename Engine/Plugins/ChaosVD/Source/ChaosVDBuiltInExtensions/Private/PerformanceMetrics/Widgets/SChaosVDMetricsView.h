// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "Misc/FilterCollection.h"
#include "Misc/TextFilter.h"
#include "SlateFwd.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Input/SComboBox.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWidget.h"
#include "Widgets/Views/SHeaderRow.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STreeView.h"
#include "Widgets/Views/SListView.h"

#include "PerformanceMetrics/ChaosVDMetrics.h"

class SSearchBox;
class FChaosVDScene;
class FChaosVDMetricsViewerState;

class SParticleMetricRowWidget : public SMultiColumnTableRow< TSharedPtr<FParticleMetricEntry>>
{
	public:
		SLATE_BEGIN_ARGS(SParticleMetricRowWidget){}
		SLATE_END_ARGS()

		void Construct(const FArguments& InArgs, const TSharedRef<STableViewBase>& InOwnerTable, TSharedPtr<FParticleMetricEntry> InItem, ChaosVDCollisionComplexityFilteringOptions Complexity, ChaosVDParticleMetricsType Metric, TSharedPtr<FParticleMetricEntry> InSumMetric);

		TSharedRef<SWidget> GenerateWidgetForColumn(const FName& ColumnName);

		TSharedPtr<FParticleMetricEntry> Item;
		ChaosVDCollisionComplexityFilteringOptions SelectedComplexity = ChaosVDCollisionComplexityFilteringOptions::Simple;
		ChaosVDParticleMetricsType SelectedMetric = ChaosVDParticleMetricsType::PrimitiveDensity;
		TSharedPtr<FParticleMetricEntry> SumMetric;
};

class SChaosVDMetricsView : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SChaosVDMetricsView) {}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene> InCVDScene, TSharedPtr<FChaosVDMetricsViewerState> InViewerState);

	void Reset();

	void ApplyFiltering();
private:
	TSharedPtr<SListView<TSharedPtr<FParticleMetricEntry>>> ComplexityListView;

	TSharedPtr<SSearchBox> SearchBox;

	TSharedPtr<STextBlock> MetricHeaderText;

	TArray<TSharedPtr<FParticleMetricEntry>> FilteredParticleEntries;

	TWeakPtr<FChaosVDScene> CVDScene;

	TSharedPtr<FChaosVDMetricsViewerState> ViewerState;

	TSharedPtr<FParticleMetricEntry> SumStatistics; 

	TSharedRef<ITableRow> GenerateComplexityEntryRow(TSharedPtr<FParticleMetricEntry> ParticleComplexityEntry, const TSharedRef<STableViewBase>& TableViewBase) const;
	void HandleListDoubleClick(TSharedPtr<FParticleMetricEntry> ParticleComplexityEntry);
	void HandleListSelectionChanged(TSharedPtr<FParticleMetricEntry> ParticleComplexityEntry, ESelectInfo::Type SelectionType);

public:

	struct FColumnNames
	{
		const FName Name = FName("Name");
		const FName Metric = FName("Metric");
		const FName Percentage = FName("Percentage");
	};

	static FColumnNames ColumnNames;
};
