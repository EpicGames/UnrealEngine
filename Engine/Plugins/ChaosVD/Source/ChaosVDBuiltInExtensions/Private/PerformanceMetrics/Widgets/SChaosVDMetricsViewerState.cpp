// Copyright Epic Games, Inc. All Rights Reserved.

#include "SChaosVDMetricsViewerState.h"
#include "ChaosVD/Public/ChaosVDPlaybackViewportClient.h"
#include "ChaosVD/Public/ChaosVDSettingsManager.h"
#include "PerformanceMetrics/Settings/ChaosVDMetricsViewSettings.h"
#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "SEditorViewport.h"

const FName FChaosVDMetricsViewerState::ChaosViewportType = FName("SChaosVDPlaybackViewport");

TMulticastDelegateRegistration<FChaosVDMetricsViewerState::FSelectedMetricChangedSignature>& FChaosVDMetricsViewerState::OnSelectedMetricChanged()
{
	return SelectedMetricChangedDelegate;
}

TMulticastDelegateRegistration<FChaosVDMetricsViewerState::FSelectionBoxChangedSignature>& FChaosVDMetricsViewerState::OnSelectionBoxChanged()
{
	return SelectionBoxChangedDelegate;
}

TMulticastDelegateRegistration<FChaosVDMetricsViewerState::FHeatmapCellSizeChangedSignature>& FChaosVDMetricsViewerState::OnHeatmapCellSizeChanged()
{
	return HeatmapCellSizeChangedDelegate;
}

TMulticastDelegateRegistration<FChaosVDMetricsViewerState::FHistogramFilterSignature>& FChaosVDMetricsViewerState::OnHistogramFilterChanged()
{
	return HistogramFilterChangedDelegate;
}

void FChaosVDMetricsViewerState::SetSelectedComplexity(ChaosVDCollisionComplexityFilteringOptions Complexity)
{
	if (SelectedComplexity == Complexity)
	{
		return;
	}

	SelectedComplexity = Complexity;

	UpdateDataFromSettings();

	SelectedMetricChangedDelegate.Broadcast(SelectedMetric, SelectedComplexity);
}

void FChaosVDMetricsViewerState::SetSelectedMetric(ChaosVDParticleMetricsType Metric)
{
	if (SelectedMetric == Metric)
	{
		return;
	}

	SelectedMetric = Metric;

	UpdateDataFromSettings();

	SelectedMetricChangedDelegate.Broadcast(SelectedMetric, SelectedComplexity);
}

ChaosVDCollisionComplexityFilteringOptions FChaosVDMetricsViewerState::GetSelectedComplexity()
{
	return SelectedComplexity;
}

ChaosVDParticleMetricsType FChaosVDMetricsViewerState::GetSelectedMetric()
{
	return SelectedMetric;
}

void FChaosVDMetricsViewerState::SetSelectionBox(const FBox2D& InSelectionBox)
{
	SelectionBox = InSelectionBox;

	SelectionBoxChangedDelegate.Broadcast(SelectionBox);
}

const FBox2D& FChaosVDMetricsViewerState::GetSelectionBox() const
{
	return SelectionBox;
}

uint32 FChaosVDMetricsViewerState::GetHeatmapCellSize() const
{
	return HeatmapCellSize;
}

void FChaosVDMetricsViewerState::SetHeatmapCellSize(uint32 InValue, bool bSendUpdateEvent)
{
	HeatmapCellSize = FMath::Max(InValue, Chaos::VD::PerformanceMetrics::SettingsBounds::MinCellSize);

	if(bSendUpdateEvent)
	{
		UChaosVDMetricsViewSettings::SetHeatmapCellSize(HeatmapCellSize);

		HeatmapCellSizeChangedDelegate.Broadcast(HeatmapCellSize);
	}
}

void FChaosVDMetricsViewerState::ResetHeatmapCellSize()
{
	SetHeatmapCellSize(Chaos::VD::PerformanceMetrics::SettingsBounds::MinCellSize);
}

void FChaosVDMetricsViewerState::SetEditorModeTools(const TWeakPtr<FEditorModeTools> InWeakEditorModeTools)
{
	EditorModeToolsWeakPtr = InWeakEditorModeTools;
}

FChaosVDPlaybackViewportClient* FChaosVDMetricsViewerState::GetPlaybackViewportClient()
{
	if (TSharedPtr<FEditorModeTools> EditorModeToolsPtr = EditorModeToolsWeakPtr.Pin())
	{
		if (FEditorViewportClient* EditorClient = EditorModeToolsPtr->GetHoveredViewportClient())
		{
			if (TSharedPtr<SEditorViewport> EditorViewport = EditorClient->GetEditorViewportWidget())
			{
				if (EditorViewport->GetType() == ChaosViewportType)
				{
					return static_cast<FChaosVDPlaybackViewportClient*>(EditorModeToolsPtr->GetHoveredViewportClient());
				}
			}
		}
	}
	return nullptr;
}

FChaosVDHeatmapColorSettings& FChaosVDMetricsViewerState::GetHeatmapColorSettings()
{
	return HeatmapColorSettings;
}

void FChaosVDMetricsViewerState::SetHeatmapColorSettings(FChaosVDHeatmapColorSettings& ColorSettings)
{
	HeatmapColorSettings = ColorSettings;

	UChaosVDMetricsViewSettings::SetHeatmapColorSettings(HeatmapColorSettings);
}

void FChaosVDMetricsViewerState::SetHeatmapCellMinThreshold(double InMin, bool bSendUpdateEvent)
{
	HeatmapCellMin = InMin;

	if (bSendUpdateEvent)
	{
		UChaosVDMetricsViewSettings::SetHeatmapCellMinThreshold(SelectedComplexity, SelectedMetric, InMin);
	}
}

double FChaosVDMetricsViewerState::GetHeatmapCellMinThreshold()
{
	return HeatmapCellMin;
}

void FChaosVDMetricsViewerState::SetHeatmapCellMaxThreshold(double InMax, bool bSendUpdateEvent)
{
	HeatmapCellMax = InMax;

	if (bSendUpdateEvent)
	{
		UChaosVDMetricsViewSettings::SetHeatmapCellMaxThreshold(SelectedComplexity, SelectedMetric, InMax);
	}
}

double FChaosVDMetricsViewerState::GetHeatmapCellMaxThreshold()
{
	return HeatmapCellMax;
}

void FChaosVDMetricsViewerState::SetHistogramFilter(double InMin, double InMax)
{
	if (InMin == HistogramMin && InMax == HistogramMax)
	{
		return;
	}

	HistogramMin = InMin;
	HistogramMax = InMax;

	HistogramFilterChangedDelegate.Broadcast(InMin, InMax);
}

void FChaosVDMetricsViewerState::GetHistogramFilter(double& OutMin, double& OutMax)
{
	OutMin = HistogramMin;
	OutMax = HistogramMax;
}

void FChaosVDMetricsViewerState::SetParticleMetrics(const TSharedPtr<TArray<TSharedPtr<FParticleMetricEntry>>>& InParticleEntries)
{
	ParticleEntries = InParticleEntries;
}

TSharedPtr<TArray<TSharedPtr<FParticleMetricEntry>>> FChaosVDMetricsViewerState::GetParticleEntries()
{
	return ParticleEntries;
}

bool FChaosVDMetricsViewerState::IsParticleDataValid()
{
	return ParticleEntries && !ParticleEntries->IsEmpty();
}

void FChaosVDMetricsViewerState::UpdateDataFromSettings()
{
	if (UChaosVDMetricsViewSettings* MetricsSettings = FChaosVDSettingsManager::Get().GetSettingsObject<UChaosVDMetricsViewSettings>())
	{
		HeatmapCellMin = MetricsSettings->GetHeatmapCellMinThreshold(SelectedComplexity, SelectedMetric);
		HeatmapCellMax = MetricsSettings->GetHeatmapCellMaxThreshold(SelectedComplexity, SelectedMetric);
		HeatmapCellSize = MetricsSettings->GetHeatmapCellSize();
		HeatmapColorSettings = MetricsSettings->HeatmapColorSettings;
	}
}