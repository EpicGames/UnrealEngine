// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "PerformanceMetrics/ChaosVDMetrics.h"
#include "PerformanceMetrics/Settings/ChaosVDMetricsViewSettings.h"
#include "Math/Box2D.h"

class FChaosVDPlaybackViewportClient;
class FEditorModeTools;
struct FChaosVDHeatmapColorSettings;

class FChaosVDMetricsViewerState : public TSharedFromThis<FChaosVDMetricsViewerState>
{
public:

	using FSelectedMetricChangedSignature = void(const ChaosVDParticleMetricsType&, const ChaosVDCollisionComplexityFilteringOptions&);
	using FSelectionBoxChangedSignature = void(const FBox2D&);
	using FHeatmapCellSizeChangedSignature = void(uint32 InCellSize);
	using FHistogramFilterSignature = void(double HistogramMin, double HistogramMax);

	TMulticastDelegateRegistration<FSelectedMetricChangedSignature>& OnSelectedMetricChanged();
	TMulticastDelegateRegistration<FSelectionBoxChangedSignature>& OnSelectionBoxChanged();
	TMulticastDelegateRegistration<FHeatmapCellSizeChangedSignature>& OnHeatmapCellSizeChanged();
	TMulticastDelegateRegistration<FHistogramFilterSignature>& OnHistogramFilterChanged();
	
	void SetSelectedComplexity(ChaosVDCollisionComplexityFilteringOptions Complexity);

	ChaosVDCollisionComplexityFilteringOptions GetSelectedComplexity();

	void SetSelectedMetric(ChaosVDParticleMetricsType Metric);
	ChaosVDParticleMetricsType GetSelectedMetric();

	void SetSelectionBox(const FBox2D& InFilterBox);
	const FBox2D& GetSelectionBox() const;

	uint32 GetHeatmapCellSize() const;
	void SetHeatmapCellSize(uint32 InValue, bool bSendUpdateEvent = true);
	void ResetHeatmapCellSize();

	void SetEditorModeTools(const TWeakPtr<FEditorModeTools> InWeakEditorModeTools);
	FChaosVDPlaybackViewportClient* GetPlaybackViewportClient();
	FChaosVDHeatmapColorSettings& GetHeatmapColorSettings();
	void SetHeatmapColorSettings(FChaosVDHeatmapColorSettings& ColorSettings);

	void SetHeatmapCellMinThreshold(double InMin, bool bSendUpdateEvent = true);
	double GetHeatmapCellMinThreshold();

	void SetHeatmapCellMaxThreshold(double InMax, bool bSendUpdateEvent = true);
	double GetHeatmapCellMaxThreshold();

	void SetHistogramFilter(double InMin, double InMax);
	void GetHistogramFilter(double& OutMin, double& OutMax);

	void SetParticleMetrics(const TSharedPtr<TArray<TSharedPtr<FParticleMetricEntry>>>& InParticleEntries);
	TSharedPtr<TArray<TSharedPtr<FParticleMetricEntry>>> GetParticleEntries();
	bool IsParticleDataValid();

	void UpdateDataFromSettings();

private:
	
	FBox2D SelectionBox = FBox2D(ForceInit);
	ChaosVDCollisionComplexityFilteringOptions SelectedComplexity = ChaosVDCollisionComplexityFilteringOptions::Simple;
	ChaosVDParticleMetricsType SelectedMetric = ChaosVDParticleMetricsType::PrimitiveDensity;

	FChaosVDHeatmapColorSettings HeatmapColorSettings;

	TWeakPtr<FEditorModeTools> EditorModeToolsWeakPtr;
	static const FName ChaosViewportType;

	uint32 HeatmapCellSize = 2000;

	double HeatmapCellMin;
	double HeatmapCellMax;

	double HistogramMin = -1;
	double HistogramMax = -1;

	TSharedPtr<TArray<TSharedPtr<FParticleMetricEntry>>> ParticleEntries;

	TMulticastDelegate<FSelectedMetricChangedSignature> SelectedMetricChangedDelegate;
	TMulticastDelegate<FSelectionBoxChangedSignature> SelectionBoxChangedDelegate;
	TMulticastDelegate<FHeatmapCellSizeChangedSignature> HeatmapCellSizeChangedDelegate;
	TMulticastDelegate<FHistogramFilterSignature> HistogramFilterChangedDelegate;
};