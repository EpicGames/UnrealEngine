// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMetricsHeatmap.h"

#include "Math/Box2D.h"
#include "Math/Color.h"
#include "Math/TransformCalculus2D.h"

#include "Fonts/FontMeasure.h"
#include "Styling/AppStyle.h"
#include "Styling/StyleColors.h"

#include "Framework/Commands/UIAction.h"
#include "Framework/Commands/UICommandList.h"

#include "GenericQuadTree.h"
#include "Misc/AxisDisplayInfo.h"

#include "ChaosVDScene.h"
#include "ChaosVD/Public/ChaosVDPlaybackViewportClient.h"
#include "PerformanceMetrics/Commands/ChaosVDMetricsHeatmapCommands.h"
#include "PerformanceMetrics/Settings/ChaosVDMetricsViewSettings.h"
#include "SChaosVDMetricsViewerState.h"
#include "SCVDMetricsHeatmapToolbar.h"
#include "SChaosVDMetricsView.h"

namespace Chaos::VD::PerformanceMetrics::Private
{

constexpr float GridThickness = 2.0f;
constexpr float GridBoxThickness = 1.0f;
constexpr float FilterThickness = 2.0f;

/* 2D Axes Constants */
constexpr float AxesYOffset = 50.0f;
constexpr float AxesXOffset = 40.0f;
constexpr float AxisSize = 25.0f;
constexpr float AxesOutlineThickness = 1.0f;
constexpr float AxisTextDistance = 12.0f;
const FVector2f AxisCenterPointInnerSize(2.f, 2.f);
constexpr FLinearColor AxisCenterPointInnerColor(0.2f, 0.2f, 0.2f);
const FVector2f AxisCenterPointOuterSize(4.f, 4.f);
constexpr FLinearColor AxisCenterPointOuterColor(0.05f, 0.05f, 0.05f);

/* Heatmap Color Legend Constants */
constexpr float LegendYOffset = 10.0f;
constexpr float LegendXOffset = 10.0f;
constexpr float LegendHeight = 6.0f;
constexpr float LegendWidth = 160.0f;
constexpr float LegendOutlineThickness = 1.0f;

constexpr float ThresholdPositionRatio = 0.75f;
constexpr float MinOffsetPositionRatio = 0.2f;

constexpr float ZoomDeltaScalingFactor = 8.0f;

template<typename T>
static UE::Math::TBox2<T> TransformBox2D(const UE::Math::TBox2<T>& Box, const TTransform2<T>& Transform)
{
	if (!Box.bIsValid)
	{
		return UE::Math::TBox2<T>(ForceInit);
	}

	UE::Math::TVector2<T> Centre, Extents;
	Box.GetCenterAndExtents(Centre, Extents);

	const UE::Math::TVector2<T> TransformedCentre = Transform.TransformPoint(Centre);

	const TMatrix2x2<T>& Matrix = Transform.GetMatrix();

	T M00, M01, M10, M11;
	Matrix.GetMatrix(M00, M01, M10, M11);
	const UE::Math::TVector2<T> AbsTransformX(FMath::Abs(M00), FMath::Abs(M10));
	const UE::Math::TVector2<T> AbsTransformY(FMath::Abs(M01), FMath::Abs(M11));

	const UE::Math::TVector2<T> NewHalfExtents = AbsTransformX * Extents.X + AbsTransformY * Extents.Y;

	return UE::Math::TBox2<T>(TransformedCentre - NewHalfExtents, TransformedCentre + NewHalfExtents);
}

static FLinearColor GetAxisShadowColor(const FLinearColor& Color)
{
	FLinearColor ColorHSV = Color.LinearRGBToHSV();
	ColorHSV.B *= .2f;
	return ColorHSV.HSVToLinearRGB();
};

FLinearColor LerpColor3(const FLinearColor& LeftColor, const FLinearColor& RightColor, float Factor)
{
	return FLinearColor(
		FMath::Lerp(LeftColor.R, RightColor.R, Factor), FMath::Lerp(LeftColor.G, RightColor.G, Factor),
		FMath::Lerp(LeftColor.B, RightColor.B, Factor), 1.f);
}

static FVector2f GetAxisShadowOffset(FVector2f OriginPoint, FVector2f EndPoint)
{
	const float DeltaX = FMath::Abs(EndPoint.X - OriginPoint.X);
	const float DeltaY = FMath::Abs(EndPoint.Y - OriginPoint.Y);
	return (DeltaX > DeltaY) ? FVector2f(0.f, 1.f) : FVector2f(1.f, 0.f);
}

FColor GetHeatValueColor(
	float NormalizedValue,
	const FChaosVDHeatmapColorSettings& ColorProperties)
{
	FLinearColor Result;
	const float LowpointRatio = FMath::Clamp(ColorProperties.LowpointRatio, 0.f, 1.f);
	const float MidpointRatio = FMath::Clamp(ColorProperties.MidpointRatio, 0.f, 1.f);

	if (NormalizedValue > 1.f)
	{
		Result = Private::LerpColor3(ColorProperties.HighValueColor, ColorProperties.MaxValueColor, NormalizedValue - 1.f);
	}
	else
	{
		if (NormalizedValue > LowpointRatio)
		{
			FColor StartColor = ColorProperties.LowValueColor;
			FColor EndColor = ColorProperties.MidpointValueColor;
			FVector2D StartRange = FVector2D(LowpointRatio, MidpointRatio);

			if (NormalizedValue > MidpointRatio)
			{
				StartColor = EndColor;
				EndColor = ColorProperties.HighValueColor;
				StartRange = FVector2D(MidpointRatio, 1.f);
			}
			const FVector2D EndRange(0.f, 1.f);
			const float ColorFactor = FMath::GetMappedRangeValueUnclamped(StartRange, EndRange, NormalizedValue);
			Result = Private::LerpColor3(StartColor, EndColor, ColorFactor);
		}
		else
		{
			Result = ColorProperties.LowValueColor;
		}
	}

	const float EaseInValue = (NormalizedValue > 1.f) ? 1.f : pow(NormalizedValue, ColorProperties.AlphaFactor);
	Result.A = FMath::Lerp(ColorProperties.MinAlpha, ColorProperties.MaxAlpha, EaseInValue);
	return Result.ToFColor(true);
}

}

SLATE_IMPLEMENT_WIDGET(SCVDMetricsHeatmapView)

void SCVDMetricsHeatmapView::PrivateRegisterAttributes(FSlateAttributeInitializer&)
{
}

SCVDMetricsHeatmapView::SCVDMetricsHeatmapView() : CommandList(MakeShareable(new FUICommandList))
{
}

void SCVDMetricsHeatmapView::Construct(
	const FArguments& InArgs, const TWeakPtr<FChaosVDScene> InCVDScene, TSharedPtr<FChaosVDMetricsViewerState> InViewerState)
{
	SetClipping(EWidgetClipping::ClipToBounds);
	BindCommands();

	CVDScene = InCVDScene;
	ViewerState = InViewerState;
	QuadTree = MakeUnique<TQuadTree<int32, 16>>(FBox2d(ForceInitToZero));

	TSharedRef<SOverlay> GridOverlay =
		SNew(SOverlay)
		+ SOverlay::Slot()
		.VAlign(VAlign_Top)
		.Padding(10.f)
		[
			SAssignNew(Toolbar, SCVDMetricsHeatmapToolbar, ViewerState)
			.EditorCommands(CommandList)
		];

	ChildSlot
	[
		GridOverlay
	];

	ViewerState->OnSelectedMetricChanged().AddSPLambda(this, [this](const ChaosVDParticleMetricsType& Metrics, const ChaosVDCollisionComplexityFilteringOptions& Complexity)
	{
		RefreshHeatmapView(true);
	});

	ViewerState->OnHeatmapCellSizeChanged().AddSPLambda(this, [this](int32 InCellSize)
	{
		RefreshHeatmapView(false);
	});
}

void SCVDMetricsHeatmapView::RefreshHeatmapView(bool bInvalidateBounds)
{
	CellData.Reset();
	CalculatedCells.Reset();

	if (!ViewerState || !ViewerState->IsParticleDataValid())
	{
		if (QuadTree)
		{
			QuadTree->Empty();
		}
		return;
	}

	TArray<TSharedPtr<FParticleMetricEntry>> ParticleMetrics = *ViewerState->GetParticleEntries();

	if (bInvalidateBounds || !QuadTree)
	{
		FBox2d TotalBounds(ForceInitToZero);

		for(const TSharedPtr<FParticleMetricEntry>& Entry : ParticleMetrics)
		{
			TotalBounds += FBox2d(FVector2D(Entry->ParticleBounds.Min.X, Entry->ParticleBounds.Min.Y), FVector2D(Entry->ParticleBounds.Max.X, Entry->ParticleBounds.Max.Y));
		}
		QuadTree = MakeUnique<TQuadTree<int32, 16>>(TotalBounds);
	}
	else
	{
		QuadTree->Empty();
	}

	for (int32 Index = 0; Index < ParticleMetrics.Num(); Index++)
	{
		const TSharedPtr<FParticleMetricEntry>& Entry = ParticleMetrics[Index];
		QuadTree->Insert(Index, FBox2d(FVector2D(Entry->ParticleBounds.Min.X, Entry->ParticleBounds.Min.Y), FVector2D(Entry->ParticleBounds.Max.X, Entry->ParticleBounds.Max.Y)));
	}
}

void SCVDMetricsHeatmapView::FocusEditorView() const
{
	if(ViewerState)
	{
		if (FChaosVDPlaybackViewportClient* ViewportClient = ViewerState->GetPlaybackViewportClient())
		{
			FViewportCameraTransform& ViewTransform = ViewportClient->GetViewTransform();
			State.FocusCenter = -FVector2D(ViewTransform.GetLocation().X, ViewTransform.GetLocation().Y);

			UpdateTransform();
		}
	}
}

void SCVDMetricsHeatmapView::FocusSampleBounds()
{
	bIsTrackingCamera = false;
	if (ViewerState && ViewerState->IsParticleDataValid())
	{
		const FVector2d Extent(ViewerState->GetHeatmapCellSize() * 0.5f);
		FBox SelectionBox(ForceInitToZero);

		FVector Avg = FVector::ZeroVector;

		TArray<TSharedPtr<FParticleMetricEntry>> ParticleMetrics = *(ViewerState->GetParticleEntries());
		for (const TSharedPtr<FParticleMetricEntry>& Particle : ParticleMetrics)
		{
			Avg += Particle->ParticleBounds.GetCenter();
		}

		Avg /= ParticleMetrics.Num();

		State.FocusCenter = -FVector2D(Avg.X, Avg.Y);
		UpdateTransform();
	}
}

void SCVDMetricsHeatmapView::Teleport()
{
	if(ViewerState)
	{
		if (FChaosVDPlaybackViewportClient* ViewportClient = ViewerState->GetPlaybackViewportClient())
		{
			FViewportCameraTransform& ViewTransform = ViewportClient->GetViewTransform();

			FVector WorldLocation = FVector(CursorWorldPosition, 400);

			ViewTransform.SetLocation(WorldLocation);
			ViewportClient->Invalidate();
		}
	}
}

void SCVDMetricsHeatmapView::BindCommands()
{
	const FChaosVDMetricsHeatmapCommands& Commands = FChaosVDMetricsHeatmapCommands::Get();

	FUIAction FocusBoundsAction;
	FocusBoundsAction.ExecuteAction = FExecuteAction::CreateSP(this, &SCVDMetricsHeatmapView::FocusSampleBounds);
	FocusBoundsAction.CanExecuteAction = FCanExecuteAction::CreateSPLambda(this, [this]
	{ 
		return ViewerState && ViewerState->IsParticleDataValid();
	});
	CommandList->MapAction(Commands.FocusBounds, FocusBoundsAction);

	FUIAction TrackEditorViewAction;
	TrackEditorViewAction.ExecuteAction = FExecuteAction::CreateSPLambda(this, [this]()
	{
		bIsTrackingCamera = !bIsTrackingCamera;
	});
	TrackEditorViewAction.CanExecuteAction = FCanExecuteAction::CreateSPLambda(this, [this]()
	{ 
		return ViewerState && ViewerState->IsParticleDataValid();
	});
	TrackEditorViewAction.GetActionCheckState = FGetActionCheckState::CreateSPLambda(this, [this]()
	{
		return bIsTrackingCamera ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	});
	CommandList->MapAction(Commands.TrackEditorView, TrackEditorViewAction);
}

void SCVDMetricsHeatmapView::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	if (bIsTrackingCamera)
	{
		FocusEditorView();
	}
}

FReply SCVDMetricsHeatmapView::OnMouseButtonDown(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsLeftMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	const bool bIsRightMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;

	TotalMouseDelta = 0.0f;

	if (bIsLeftMouseButtonEffecting || bIsRightMouseButtonEffecting)
	{
		if (bIsLeftMouseButtonEffecting)
		{
			SelectionStart = SelectionEnd = CursorWorldPosition;
		}

		//If the user decides to start panning, unlock the camera tracking.
		if (bIsRightMouseButtonEffecting)
		{
			bIsTrackingCamera = false;
		}

		return FReply::Handled().CaptureMouse(SharedThis(this));
	}

	return FReply::Unhandled();
}

FReply SCVDMetricsHeatmapView::OnMouseButtonUp(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	if (!HasMouseCapture())
	{
		return FReply::Unhandled();
	}

	const bool bIsLeftMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;
	const bool bIsRightMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::RightMouseButton;

	TotalMouseDelta = 0.0f;

	if (bIsLeftMouseButtonEffecting || bIsRightMouseButtonEffecting)
	{
		UpdateMouseByEvent(MyGeometry, MouseEvent);

		FReply ReplyState = FReply::Handled();

		if (bIsLeftMouseButtonEffecting)
		{
			SelectionEnd = CursorWorldPosition;
			UpdateSelectionBox();
			bIsFiltering = false;
		}

		if (bIsRightMouseButtonEffecting)
		{
			bIsPanning = false;
		}

		if (HasMouseCapture() && !IsMouseDragging())
		{
			ReplyState.ReleaseMouseCapture();
		}

		return ReplyState;
	}

	return FReply::Unhandled();
}

FReply SCVDMetricsHeatmapView::OnMouseButtonDoubleClick(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const bool bIsLeftMouseButtonEffecting = MouseEvent.GetEffectingButton() == EKeys::LeftMouseButton;

	bool bHandled = false;

	if (IsScreenRectValid())
	{
		if (bIsLeftMouseButtonEffecting)
		{
			Teleport();

			bHandled = true;
		}
	}

	return bHandled ? FReply::Handled() : FReply::Unhandled();
}

void SCVDMetricsHeatmapView::UpdateMouseByEvent(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	CursorScreenSpacePosition = MyGeometry.AbsoluteToLocal(MouseEvent.GetScreenSpacePosition());
	CursorWorldPosition = State.ScreenToWorld.TransformPoint(FVector2D(CursorScreenSpacePosition));
}

void SCVDMetricsHeatmapView::UpdateTransform() const
{
	FTransform2d Translation(1.0f, State.FocusCenter);
	FTransform2d ScaledExtentsOffset(State.Scale, State.ScreenRect.GetExtent());

	State.WorldToScreen = Concatenate(Translation, ScaledExtentsOffset);
	State.ScreenToWorld = State.WorldToScreen.Inverse();
}

FReply SCVDMetricsHeatmapView::OnMouseMove(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	const FVector2f CursorDelta = MouseEvent.GetCursorDelta();

	UpdateMouseByEvent(MyGeometry, MouseEvent);

	if (HasMouseCapture())
	{
		TotalMouseDelta += CursorDelta.Size();

		const bool bIsLeftMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::LeftMouseButton);
		const bool bIsRightMouseButtonDown = MouseEvent.IsMouseButtonDown(EKeys::RightMouseButton);
		const bool bIsDragTrigger = TotalMouseDelta > FSlateApplication::Get().GetDragTriggerDistance();

		const bool bIsDragging = IsMouseDragging();

		if (bIsLeftMouseButtonDown)
		{
			if (!bIsFiltering && bIsDragTrigger && !bIsDragging)
			{
				bIsFiltering = true;
			}

			if (bIsFiltering)
			{
				SelectionEnd = CursorWorldPosition;
				return FReply::Handled();
			}
		}

		if (bIsRightMouseButtonDown)
		{
			if (!bIsPanning && bIsDragTrigger && !bIsDragging)
			{
				bIsPanning = true;
				LastCursorWorldDragPosition = CursorWorldPosition;
			}

			if (bIsPanning)
			{
				State.FocusCenter += (CursorWorldPosition - LastCursorWorldDragPosition);
				UpdateTransform();
				return FReply::Handled();
			}
		}
	}

	return FReply::Unhandled();
}

FReply SCVDMetricsHeatmapView::OnMouseWheel(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{
	using namespace Chaos::VD::PerformanceMetrics;

	if (IsScreenRectValid() && !IsMouseDragging())
	{
		const FVector2D CursorLocalPosition = CursorScreenSpacePosition - (MyGeometry.GetLocalSize() * 0.5f);
		const FVector2D Position0 = State.ScreenToWorld.TransformVector(CursorLocalPosition);
		const float Delta = 1.0f + FMath::Abs(MouseEvent.GetWheelDelta() / Private::ZoomDeltaScalingFactor);
		State.Scale = FMath::Clamp(
			State.Scale * (MouseEvent.GetWheelDelta() > 0 ? Delta : (1.0f / Delta)), SCVDMetricsHeatmapState::HeatmapMinScale,
			SCVDMetricsHeatmapState::HeatmapMaxScale);
		UpdateTransform();

		const FVector2D Position1 = State.ScreenToWorld.TransformVector(CursorLocalPosition);
		State.FocusCenter += (Position1 - Position0);
		UpdateTransform();

		return FReply::Handled();
	}
	return FReply::Unhandled();
}

int32 SCVDMetricsHeatmapView::OnPaint(
	const FPaintArgs& Args,
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId,
	const FWidgetStyle& InWidgetStyle,
	bool bParentEnabled) const
{
	State.ScreenRect = FBox2f(FVector2f::ZeroVector, AllottedGeometry.GetLocalSize());

	UpdateTransform();

	LayerId = PaintSpatialData(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	LayerId = PaintGrid(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	LayerId = PaintEditorView(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	LayerId = PaintSelectionBounds(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	LayerId = PaintAxes(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
	LayerId = PaintLegend(AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);

	return Super::OnPaint(
		Args, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId, InWidgetStyle, bParentEnabled);
}

int32 SCVDMetricsHeatmapView::PaintGrid(
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId) const
{
	using namespace Chaos::VD::PerformanceMetrics;

	const FBox2f ViewRectS(FVector2f(ForceInitToZero), AllottedGeometry.GetLocalSize());
	const FBox2d ViewRectW = Private::TransformBox2D(FBox2D(ViewRectS), State.ScreenToWorld);
	if (ViewRectW.GetArea() <= 0.0f)
	{
		return LayerId;
	}

	const int32 CellSizeW = ViewerState->GetHeatmapCellSize();
	const float CellSizeS = State.Scale * CellSizeW;

	const FBox2f VisibleGridRectW(
		FVector2f(
			FMath::Max(FMath::FloorToFloat(ViewRectW.Min.X / CellSizeW) * CellSizeW, ViewRectW.Min.X),
			FMath::Max(FMath::FloorToFloat(ViewRectW.Min.Y / CellSizeW) * CellSizeW, ViewRectW.Min.Y)),
		FVector2f(
			FMath::Min(FMath::CeilToFloat(ViewRectW.Max.X / CellSizeW) * CellSizeW, ViewRectW.Max.X),
			FMath::Min(FMath::CeilToFloat(ViewRectW.Max.Y / CellSizeW) * CellSizeW, ViewRectW.Max.Y)));

	FLinearColor Color = FLinearColor(0.1f, 0.1f, 0.1f, 0.25f);

	const FInt64Vector2 TopLeftW(
		FMath::FloorToFloat(VisibleGridRectW.Min.X / CellSizeW) * CellSizeW - (CellSizeW * 0.5f),
		FMath::FloorToFloat(VisibleGridRectW.Min.Y / CellSizeW) * CellSizeW - (CellSizeW * 0.5f));

	const FInt64Vector2 BottomRightW(
		FMath::CeilToFloat(VisibleGridRectW.Max.X / CellSizeW) * CellSizeW + (CellSizeW * 0.5f),
		FMath::CeilToFloat(VisibleGridRectW.Max.Y / CellSizeW) * CellSizeW + (CellSizeW * 0.5f));

	const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();

	// Horizontal
	for (int64 Y = TopLeftW.Y; Y <= BottomRightW.Y; Y += CellSizeW)
	{
		const FVector2d LineStartW(TopLeftW.X, Y);
		const FVector2d LineEndW(BottomRightW.X, Y);

		const FVector2f LineStartS(State.WorldToScreen.TransformPoint(LineStartW));
		const FVector2f LineEndS(State.WorldToScreen.TransformPoint(LineEndW));

		FSlateDrawElement::MakeLines(OutDrawElements, LayerId, PaintGeometry, { LineStartS, LineEndS },
			ESlateDrawEffect::None, Color, false, Private::GridThickness);
	}

	// Vertical
	for (int64 X = TopLeftW.X; X <= BottomRightW.X; X += CellSizeW)
	{
		const FVector2d LineStartW(X, TopLeftW.Y);
		const FVector2d LineEndW(X, BottomRightW.Y);

		const FVector2f LineStartS(State.WorldToScreen.TransformPoint(LineStartW));
		const FVector2f LineEndS(State.WorldToScreen.TransformPoint(LineEndW));

		FSlateDrawElement::MakeLines(OutDrawElements, LayerId, PaintGeometry, { LineStartS, LineEndS }, 
			ESlateDrawEffect::None, Color, false, Private::GridThickness);
	}

	return LayerId + 1;
}

int32 SCVDMetricsHeatmapView::PaintSpatialCell(
	const SCVDMetricsHeatmapCellData& Cell,
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId) const
{
	using namespace Chaos::VD::PerformanceMetrics;

	const FLinearColor PointColor = USlateThemeManager::Get().GetColor(EStyleColor::AccentOrange);
	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush(TEXT("WhiteTexture"));

	const int32 CellSizeW = ViewerState->GetHeatmapCellSize();
	const float CellSizeS = State.Scale * CellSizeW;

	double Top = ViewerState->GetHeatmapCellMaxThreshold() * CellSizeW * CellSizeW;
	double Bottom = ViewerState->GetHeatmapCellMinThreshold() * CellSizeW * CellSizeW;

	float Factor = (Top - Bottom) < KINDA_SMALL_NUMBER ? 0 : FMath::Max(Cell.CalculatedMetric - Bottom, 0.0) / (Top - Bottom);
	const FVector2d GridTopLeftW(Cell.XCoord, Cell.YCoord);
	const FVector2f GridTopLeftS(State.WorldToScreen.TransformPoint(GridTopLeftW));

	const FPaintGeometry SampleGeometry =
		AllottedGeometry.ToPaintGeometry(FVector2f(CellSizeS), FSlateLayoutTransform(GridTopLeftS));

	const FColor ValueColor = Private::GetHeatValueColor(Factor, ViewerState->GetHeatmapColorSettings());

	FSlateDrawElement::MakeBox(
		OutDrawElements, ++LayerId, SampleGeometry, WhiteBrush, ESlateDrawEffect::None, ValueColor);

	return LayerId + 1;
}

int32 SCVDMetricsHeatmapView::PaintEditorView(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	if (FChaosVDPlaybackViewportClient* ViewportClient = ViewerState->GetPlaybackViewportClient())
	{
		FViewportCameraTransform& ViewTransform = ViewportClient->GetViewTransform();

		FVector WorldLocation = ViewTransform.GetLocation();

		FRotator ViewRotation = ViewportClient->GetViewRotation();
		FRotationMatrix RotMatrix(ViewRotation);

		FVector Forward = RotMatrix.GetScaledAxis(EAxis::X);
		FVector Right = RotMatrix.GetScaledAxis(EAxis::Y);
		FVector Up = RotMatrix.GetScaledAxis(EAxis::Z);

		float FarPlane = ViewportClient->GetFarClipPlaneOverride();
		FarPlane = FarPlane > 0 ? FarPlane : 20000;
		float FarWidth = 2 * FarPlane * FMath::Tan(FMath::DegreesToRadians(ViewportClient->ViewFOV / 2));
		float FarHeight = FarWidth / ViewportClient->AspectRatio;

		float VerticalFOV = FMath::Atan2(FarHeight / 2,FarPlane) * 2;

		FVector FarCenter = WorldLocation + Forward * FarPlane;

		FVector TopLeft = FarCenter - Right * (FarWidth / 2) + Up * (FarHeight / 2);
		FVector TopRight = FarCenter + Right * (FarWidth / 2) + Up * (FarHeight / 2);
		FVector BottomLeft = FarCenter - Right * (FarWidth / 2) - Up * (FarHeight / 2);
		FVector BottomRight = FarCenter + Right * (FarWidth / 2) - Up * (FarHeight / 2);

		const FVector2d EditorViewW(WorldLocation.X, WorldLocation.Y);
		const FVector2f EditorViewS(State.WorldToScreen.TransformPoint(EditorViewW));

		const FVector2d TopLeftW(TopLeft.X, TopLeft.Y);
		const FVector2f TopLeftS(State.WorldToScreen.TransformPoint(TopLeftW));

		const FVector2d TopRightW(TopRight.X, TopRight.Y);
		const FVector2f TopRightS(State.WorldToScreen.TransformPoint(TopRightW));

		const FVector2d BottomLeftW(BottomLeft.X, BottomLeft.Y);
		const FVector2f BottomLeftS(State.WorldToScreen.TransformPoint(BottomLeftW));

		const FVector2d BottomRightW(BottomRight.X, BottomRight.Y);
		const FVector2f BottomRightS(State.WorldToScreen.TransformPoint(BottomRightW));

		const FVector2f PointExtents = FVector2f(2.5f);

		const FLinearColor PointColor = USlateThemeManager::Get().GetColor(EStyleColor::AccentOrange);
		const FSlateBrush* WhiteBrush = FAppStyle::GetBrush(TEXT("WhiteTexture"));

		const FPaintGeometry SamplePointGeometry =
			AllottedGeometry.ToPaintGeometry(PointExtents * 2.0f, FSlateLayoutTransform(EditorViewS - PointExtents / 2));
		FSlateDrawElement::MakeBox(
			OutDrawElements, ++LayerId, SamplePointGeometry, WhiteBrush, ESlateDrawEffect::None, PointColor);

		const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();

		const float HalfFOV = ViewportClient->ViewFOV/2;
		const float HalfVFOV = FMath::RadiansToDegrees(VerticalFOV) / 2;

		FVector LeftNorm = Forward.RotateAngleAxis((90 + HalfFOV), Up);
		FVector RightNorm = Forward.RotateAngleAxis(-(90 + HalfFOV), Up);

		FVector TopNorm = Forward.RotateAngleAxis(-(90 + HalfVFOV), Right);
		FVector BottomNorm = Forward.RotateAngleAxis((90 + HalfVFOV), Right);

		// Cull an edge if the two faces it is connected to both face the overhead camera or both don't face the overhead camera.
		if ((LeftNorm.Z >= 0) != (TopNorm.Z >= 0))
		{
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { EditorViewS, TopLeftS }, ESlateDrawEffect::None, PointColor, false, 1.5);
		}

		if ((RightNorm.Z >= 0) != (TopNorm.Z >= 0))
		{
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { EditorViewS, TopRightS }, ESlateDrawEffect::None, PointColor, false, 1.5);
		}
		
		if ((LeftNorm.Z >= 0) != (BottomNorm.Z >= 0))
		{
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { EditorViewS, BottomLeftS }, ESlateDrawEffect::None, PointColor, false, 1.5);
		}

		if ((RightNorm.Z >= 0) != (BottomNorm.Z >= 0))
		{
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { EditorViewS, BottomRightS }, ESlateDrawEffect::None, PointColor, false, 1.5);
		}

		if ((Forward.Z >= 0) != (TopNorm.Z >= 0))
		{
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { TopLeftS, TopRightS }, ESlateDrawEffect::None, PointColor, false, 1.5);
		}

		if ((Forward.Z >= 0) != (BottomNorm.Z >= 0))
		{
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { BottomLeftS, BottomRightS }, ESlateDrawEffect::None, PointColor, false, 1.5);
		}

		if ((Forward.Z >= 0) != (LeftNorm.Z >= 0))
		{
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { TopLeftS, BottomLeftS }, ESlateDrawEffect::None, PointColor, false, 1.5);
		}

		if ((Forward.Z >= 0) != (RightNorm.Z >= 0))
		{
			FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, PaintGeometry, { TopRightS, BottomRightS }, ESlateDrawEffect::None, PointColor, false, 1.5);
		}
	}

	return LayerId + 1;
}

int32 SCVDMetricsHeatmapView::PaintSpatialData(
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId) const
{
	using namespace Chaos::VD::PerformanceMetrics;

	if (!ViewerState || !ViewerState->IsParticleDataValid())
	{
		return LayerId + 1;
	}

	const FLinearColor PointColor = USlateThemeManager::Get().GetColor(EStyleColor::AccentOrange);
	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush(TEXT("WhiteTexture"));

	TArray<TSharedPtr<FParticleMetricEntry>> ParticleMetrics = *ViewerState->GetParticleEntries();

	const FBox2f ViewRectS(FVector2f(ForceInitToZero), AllottedGeometry.GetLocalSize());
	const FBox2d ViewRectW = Private::TransformBox2D(FBox2D(ViewRectS), State.ScreenToWorld);
	if (ViewRectW.GetArea() <= 0.0f)
	{
		return LayerId;
	}

	const int32 CellSizeW = ViewerState->GetHeatmapCellSize();
	const float CellSizeS = State.Scale * CellSizeW;

	const FBox2f VisibleGridRectW(
		FVector2f(
			FMath::Max(FMath::FloorToFloat(ViewRectW.Min.X / CellSizeW) * CellSizeW, ViewRectW.Min.X),
			FMath::Max(FMath::FloorToFloat(ViewRectW.Min.Y / CellSizeW) * CellSizeW, ViewRectW.Min.Y)),
		FVector2f(
			FMath::Min(FMath::CeilToFloat(ViewRectW.Max.X / CellSizeW) * CellSizeW, ViewRectW.Max.X),
			FMath::Min(FMath::CeilToFloat(ViewRectW.Max.Y / CellSizeW) * CellSizeW, ViewRectW.Max.Y)));

	FLinearColor Color = FLinearColor(0.1f, 0.1f, 0.1f, 0.25f);
	const FInt64Vector2 TopLeftW(
		FMath::FloorToFloat(VisibleGridRectW.Min.X / CellSizeW) * CellSizeW - (CellSizeW * 0.5f),
		FMath::FloorToFloat(VisibleGridRectW.Min.Y / CellSizeW) * CellSizeW - (CellSizeW * 0.5f));

	const FInt64Vector2 BottomRightW(
		FMath::CeilToFloat(VisibleGridRectW.Max.X / CellSizeW) * CellSizeW + (CellSizeW * 0.5f),
		FMath::CeilToFloat(VisibleGridRectW.Max.Y / CellSizeW) * CellSizeW + (CellSizeW * 0.5f));

	const FPaintGeometry PaintGeometry = AllottedGeometry.ToPaintGeometry();

	//Paint all cached cells and remove cells that are off camera.
	CellData.RemoveAllSwap([this, &ViewRectS, &AllottedGeometry, &MyCullingRect, &OutDrawElements, &LayerId, &CellSizeW](const SCVDMetricsHeatmapCellData& Item)
	{
		const FVector2d GridTopLeftW(Item.XCoord, Item.YCoord);
		const FVector2d GridBottomRightW(Item.XCoord + CellSizeW, Item.YCoord + CellSizeW);

		const FVector2f GridTopLeftS(State.WorldToScreen.TransformPoint(GridTopLeftW));
		const FVector2f GridBottomRightS(State.WorldToScreen.TransformPoint(GridBottomRightW));

		if (ViewRectS.Intersect(FBox2f(GridTopLeftS, GridBottomRightS)))
		{
			PaintSpatialCell(Item, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
			return false;
		}

		CalculatedCells.Remove(GridTopLeftW);
		return true;
	});

	TArray<int32> ComplexityEntriesIndexes;
	ComplexityEntriesIndexes.Reserve(32);

	// Paint all remaining cells and calculate stats if necessary 
	for (int64 Y = TopLeftW.Y; Y <= BottomRightW.Y; Y += CellSizeW)
	{
		for (int64 X = TopLeftW.X; X <= BottomRightW.X; X += CellSizeW)
		{
			const FVector2d GridTopLeftW(X, Y);
			const FVector2d GridBottomRightW(X + CellSizeW, Y + CellSizeW);

			const FVector2f GridTopLeftS(State.WorldToScreen.TransformPoint(GridTopLeftW));
			const FVector2f GridBottomRightS(State.WorldToScreen.TransformPoint(GridBottomRightW));
				
			FBox2d Grid2D = FBox2d(GridTopLeftW, GridBottomRightW);

			if (CalculatedCells.Contains(GridTopLeftW))
			{
				continue;
			}

			if (QuadTree->GetTreeBox().Intersect(Grid2D))
			{
				SCVDMetricsHeatmapCellData NewData;
				NewData.XCoord = X;
				NewData.YCoord = Y;
				FParticleMetricEntry GridStats;
				ComplexityEntriesIndexes.Reset();

				QuadTree->GetElements(Grid2D, ComplexityEntriesIndexes);

				FBox ParticlesBounds;

				double Stat = 0;

				for (int32 EntryIndex : ComplexityEntriesIndexes)
				{
					if (TSharedPtr<FParticleMetricEntry>& ParticleEntry = ParticleMetrics[EntryIndex])
					{
						float Volume = ParticleEntry->GetVolumeSafe();
						double Metric = ParticleEntry->GetMetric(ViewerState->GetSelectedComplexity(), ViewerState->GetSelectedMetric());

						FBox2d ParticleBounds = FBox2d(FVector2d(ParticleEntry->ParticleBounds.Min.X, ParticleEntry->ParticleBounds.Min.Y), FVector2d(ParticleEntry->ParticleBounds.Max.X, ParticleEntry->ParticleBounds.Max.Y));

						double OverlapArea = Grid2D.Overlap(ParticleBounds).GetArea();
						double Area = FMath::Max(ParticleBounds.GetArea(), UE_KINDA_SMALL_NUMBER);

						// Assume a uniform density per particle when splitting metrics across cells.

						if (ViewerState->GetSelectedMetric() == ChaosVDParticleMetricsType::PrimitiveDensity)
						{
							Metric *= Volume * (OverlapArea / Area);
						}
						else if (ViewerState->GetSelectedMetric() == ChaosVDParticleMetricsType::MemoryUsage)
						{
							Metric *= (OverlapArea / Area);
						}

						Stat += Metric;
					}
				}

				NewData.CalculatedMetric = Stat;

				CellData.Add(NewData);
				CalculatedCells.Add(GridTopLeftW);
				PaintSpatialCell(NewData, AllottedGeometry, MyCullingRect, OutDrawElements, LayerId);
			}
		}
	}

	return LayerId + 1;
}


int32 SCVDMetricsHeatmapView::PaintSelectionBounds(const FGeometry& AllottedGeometry, const FSlateRect& MyCullingRect, FSlateWindowElementList& OutDrawElements, int32 LayerId) const
{
	using namespace Chaos::VD::PerformanceMetrics;

	const FBox2D FilterBox = FBox2D(FVector2D::Min(SelectionStart, SelectionEnd), FVector2D::Max(SelectionStart, SelectionEnd));
	if (FilterBox.GetArea() > 0.0f)
	{
		const FVector2D SelectionMinBounds = State.WorldToScreen.TransformPoint(FilterBox.Min);
		const FVector2D SelectionMaxBounds = State.WorldToScreen.TransformPoint(FilterBox.Max);
		const bool bAntiAlias = true;
		const ESlateDrawEffect UsedEffect = ESlateDrawEffect::None;

		TArray<FVector2D> LinePoints;
		LinePoints.Add(SelectionMinBounds);
		LinePoints.Add(FVector2D(SelectionMinBounds.X, SelectionMaxBounds.Y));
		LinePoints.Add(SelectionMaxBounds);
		LinePoints.Add(FVector2D(SelectionMaxBounds.X, SelectionMinBounds.Y));
		LinePoints.Add(SelectionMinBounds);

		const FSlateColor FilterColor = FSlateColor::UseForeground();

		FSlateDrawElement::MakeLines(
			OutDrawElements, LayerId, AllottedGeometry.ToPaintGeometry(), LinePoints, UsedEffect,
			FilterColor.GetSpecifiedColor(), bAntiAlias, Private::FilterThickness);
	}

	return LayerId + 1;
}

int32 SCVDMetricsHeatmapView::PaintAxes(
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId) const
{
	using namespace Chaos::VD::PerformanceMetrics;

	const FVector2f AxisOrigin(
		State.ScreenRect.Min.X + Private::AxesXOffset, State.ScreenRect.Max.Y - Private::AxesYOffset);

	const FSlateFontInfo SmallLayoutFont = FCoreStyle::GetDefaultFontStyle("Regular", 10);
	const TSharedRef<FSlateFontMeasure> FontMeasureService =
		FSlateApplication::Get().GetRenderer()->GetFontMeasureService();
	const FVector2f TextSize = FontMeasureService->Measure(TEXT("Z"), SmallLayoutFont);
	const float XL(TextSize.X);
	const float YL(TextSize.Y);

	const FQuat2f Rotation(0);

	auto DrawAxis = [&Rotation, &AxisOrigin, &SmallLayoutFont, XL, YL](
						const FGeometry& AllottedGeometry, FVector2f Axis, const FText& AxisName,
						const FLinearColor& AxisBaseColor, const FLinearColor& AxisShadowColor,
						FSlateWindowElementList& OutDrawElements, int32 LayerId)
	{
		const FVector2f AxisUnitVec = Rotation.TransformVector(Axis);
		const FVector2f AxisVec = Private::AxisSize * AxisUnitVec;
		const FVector2f AxisEnd = AxisOrigin + AxisVec;

		const FVector2f AxisShadowOffset = Private::GetAxisShadowOffset(AxisOrigin, AxisEnd);

		/// Line (shadow)
		FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, AllottedGeometry.ToPaintGeometry(),
									{AxisOrigin + AxisShadowOffset, AxisEnd + AxisShadowOffset},
									ESlateDrawEffect::None, AxisShadowColor, true, Private::AxesOutlineThickness);

		// Line (foreground)
		FSlateDrawElement::MakeLines(OutDrawElements, ++LayerId, AllottedGeometry.ToPaintGeometry(),
									{AxisOrigin, AxisEnd}, ESlateDrawEffect::None, AxisBaseColor, true, Private::AxesOutlineThickness);

		const FVector2f TextVec = (Private::AxisSize + Private::AxisTextDistance) * AxisUnitVec;
		const FVector2f TextEnd = AxisOrigin + TextVec;

		const FSlateLayoutTransform Offset(FVector2f(TextEnd.X - 0.5f * XL, TextEnd.Y - 0.5f * YL));
		const FSlateLayoutTransform ShadowOffset(FVector2f(1.0f, 1.0f));

		// Text (shadow)
		FSlateDrawElement::MakeText(OutDrawElements, ++LayerId, AllottedGeometry.ToPaintGeometry(Offset.Concatenate(ShadowOffset)),
									AxisName, SmallLayoutFont, ESlateDrawEffect::None, AxisShadowColor);

		// Text (foreground)
		FSlateDrawElement::MakeText(OutDrawElements, ++LayerId, AllottedGeometry.ToPaintGeometry(Offset), AxisName,
								SmallLayoutFont, ESlateDrawEffect::None, AxisBaseColor);

		return LayerId;
	};

	const bool bUsingLUFCoordinateSystem = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;

	// X/Forward
	{
		const FLinearColor XAxisBaseColor = AxisDisplayInfo::GetAxisColor(EAxisList::X);
		const FLinearColor XAxisShadowColor = Private::GetAxisShadowColor(XAxisBaseColor);
		LayerId = DrawAxis(AllottedGeometry, FVector2f(bUsingLUFCoordinateSystem ? FVector3f::ForwardVector : FVector3f::XAxisVector),
							AxisDisplayInfo::GetAxisDisplayNameShort(bUsingLUFCoordinateSystem ? EAxisList::Forward : EAxisList::X),
							XAxisBaseColor, XAxisShadowColor, OutDrawElements, LayerId);
	}

	// Y/Left
	{
		const FLinearColor YAxisBaseColor = AxisDisplayInfo::GetAxisColor(EAxisList::Y);
		const FLinearColor YAxisShadowColor = Private::GetAxisShadowColor(YAxisBaseColor);
		LayerId = DrawAxis(AllottedGeometry, FVector2f(bUsingLUFCoordinateSystem ? FVector3f::LeftVector : FVector3f::YAxisVector),
							AxisDisplayInfo::GetAxisDisplayNameShort(bUsingLUFCoordinateSystem ? EAxisList::Left : EAxisList::Y),
							YAxisBaseColor, YAxisShadowColor, OutDrawElements, LayerId);
	}

	const FSlateBrush* WhiteBrush = FAppStyle::GetBrush(TEXT("WhiteTexture"));

	// Center point (outer)
	const FPaintGeometry AxisCenterPointOuterGeometry = AllottedGeometry.ToPaintGeometry(
		Private::AxisCenterPointOuterSize, FSlateLayoutTransform(AxisOrigin - (Private::AxisCenterPointOuterSize * 0.5f)));
	FSlateDrawElement::MakeBox(OutDrawElements, ++LayerId, AxisCenterPointOuterGeometry, WhiteBrush, ESlateDrawEffect::None, Private::AxisCenterPointOuterColor);

	// Center point (inner)
	const FPaintGeometry AxisCenterPointInnerGeometry = AllottedGeometry.ToPaintGeometry(
		Private::AxisCenterPointInnerSize, FSlateLayoutTransform(AxisOrigin - (Private::AxisCenterPointInnerSize * 0.5f)));
	FSlateDrawElement::MakeBox(OutDrawElements, ++LayerId, AxisCenterPointInnerGeometry, WhiteBrush, ESlateDrawEffect::None, Private::AxisCenterPointInnerColor);

	return LayerId + 1;
}

int32 SCVDMetricsHeatmapView::PaintLegend(
	const FGeometry& AllottedGeometry,
	const FSlateRect& MyCullingRect,
	FSlateWindowElementList& OutDrawElements,
	int32 LayerId) const
{
	using namespace Chaos::VD::PerformanceMetrics;

	const FPaintGeometry GradientGeometry = AllottedGeometry.ToPaintGeometry(
		FVector2d(Private::LegendWidth, Private::LegendHeight),
		FSlateLayoutTransform{FVector2d(
			AllottedGeometry.GetLocalSize().X - Private::LegendXOffset - Private::LegendWidth,
			AllottedGeometry.GetLocalSize().Y - Private::LegendYOffset - Private::LegendHeight)});

	const FPaintGeometry OutlineGeometry = AllottedGeometry.ToPaintGeometry(
		FVector2d(
			Private::LegendWidth + Private::LegendOutlineThickness * 2,
			Private::LegendHeight + Private::LegendOutlineThickness * 2),
		FSlateLayoutTransform{FVector2d(
			AllottedGeometry.GetLocalSize().X - Private::LegendXOffset - Private::LegendWidth -
				Private::LegendOutlineThickness,
			AllottedGeometry.GetLocalSize().Y - Private::LegendYOffset - Private::LegendHeight -
				Private::LegendOutlineThickness)});

	const FChaosVDHeatmapColorSettings& ColorProperties = ViewerState->GetHeatmapColorSettings();

	TArray<FSlateGradientStop> GradientStops;
	const float ThresholdRangeWidth = Private::LegendWidth * Private::ThresholdPositionRatio;

	for (const float Percentage : {ColorProperties.LowpointRatio, ColorProperties.MidpointRatio, 1.0f})
	{
		GradientStops.Add(FSlateGradientStop(
			ThresholdRangeWidth * Percentage, Private::GetHeatValueColor(Percentage, ColorProperties).WithAlpha(255)));
	}

	GradientStops.Add(FSlateGradientStop(ThresholdRangeWidth, ColorProperties.HighValueColor));
	GradientStops.Add(FSlateGradientStop(Private::LegendWidth, ColorProperties.MaxValueColor));

	FSlateDrawElement::MakeBox(
		OutDrawElements, ++LayerId, OutlineGeometry, FAppStyle::GetBrush(TEXT("WhiteTexture")), ESlateDrawEffect::None,
		USlateThemeManager::Get().GetColor(EStyleColor::Black));

	FSlateDrawElement::MakeGradient(
		OutDrawElements, ++LayerId, GradientGeometry, GradientStops, Orient_Vertical, ESlateDrawEffect::None);

	return LayerId + 1;
}

void SCVDMetricsHeatmapView::UpdateSelectionBox()
{
	FBox2D NewFilterBox(FVector2D::Min(SelectionStart, SelectionEnd), FVector2D::Max(SelectionStart, SelectionEnd));
	if (NewFilterBox.GetArea() <= 0.0f)
	{
		NewFilterBox.Init();
	}

	if (ViewerState)
	{
		const FBox2D CurrentFilterBox = ViewerState->GetSelectionBox();
		if (CurrentFilterBox != NewFilterBox)
		{
			ViewerState->SetSelectionBox(NewFilterBox);
		}
	}
}

bool SCVDMetricsHeatmapView::IsScreenRectValid() const
{
	return State.ScreenRect.bIsValid;
}

bool SCVDMetricsHeatmapView::IsMouseDragging() const
{
	return bIsPanning || bIsFiltering;
}

