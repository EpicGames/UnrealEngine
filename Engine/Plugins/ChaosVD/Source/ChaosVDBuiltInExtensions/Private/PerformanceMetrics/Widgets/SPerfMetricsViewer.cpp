// Copyright Epic Games, Inc. All Rights Reserved.

#include "SPerfMetricsViewer.h"

#include "SChaosVDMetricsView.h"
#include "SChaosVDMetricsViewerState.h"
#include "SCVDMetricsHistogram.h"
#include "SMetricsHeatmap.h"

#include "ChaosVD/Private/Actors/ChaosVDSolverInfoActor.h"
#include "ChaosVD/Public/ChaosVDPlaybackController.h"
#include "ChaosVDScene.h"
#include "Components/ChaosVDParticleDataComponent.h"
#include "PerformanceMetrics/Settings/ChaosVDMetricsViewSettings.h"
#include "Widgets/SChaosVDMainTab.h"

#include "Chaos/ImplicitObject.h"

#include "EditorModeManager.h"
#include "EditorViewportClient.h"
#include "SEditorViewport.h"

#include "Engine/Texture2D.h"
#include "ImageUtils.h"
#include "TextureResource.h"
#include "Async/Async.h"

#include "Brushes/SlateImageBrush.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Input/SComboButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SSplitter.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableRow.h"
#include "ToolMenus.h"

#define LOCTEXT_NAMESPACE "SPerfMetricsViewer"

FName SPerfMetricsViewer::ToolBarName = FName("ComplexityViewerToolBar");

void SPerfMetricsViewer::Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene>& InCVDScene, const TWeakPtr<SChaosVDMainTab>& InWeakMainTab )
{
	WeakCVDScene = InCVDScene;
	WeakMainTab = InWeakMainTab;

	ViewerState = MakeShared<FChaosVDMetricsViewerState>();
	ViewerState->SetParticleMetrics(MakeShared<TArray<TSharedPtr<FParticleMetricEntry>>>());

	if (TSharedPtr<SChaosVDMainTab> MainTab = InWeakMainTab.Pin())
	{
		ViewerState->SetEditorModeTools(MainTab->GetEditorModeManager().AsWeak());
	}

	ViewerState->UpdateDataFromSettings();

	ChildSlot[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(NoPadding)
		[
			SNew(SBorder)
			.Padding(FMargin(0, 0, 0, 2.0))
			[
				GenerateMainToolbarWidget()
			]
		]
		+ SVerticalBox::Slot()
		.FillHeight(1)
		.Padding(NoPadding)
		[
			SNew(SSplitter)
			.Orientation(EOrientation::Orient_Horizontal)
			+SSplitter::Slot()
			.Resizable(true)
			.Value(0.6f)
			[
				SNew(SVerticalBox)
				+ SVerticalBox::Slot()
				.Padding(FMargin(6.0, 6.0, 6.0, 4.0))
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ComplexitySelectLabel","Selected Collision Complexity: "))
							
					]
					+SHorizontalBox::Slot()
					[
						SNew(SComboButton)
						.HasDownArrow(true)
						.ComboButtonStyle(FAppStyle::Get(), "ComboButton")
						.ContentPadding(FMargin(4.0f, 2.0f))
						.OnGetMenuContent_Lambda([this]() -> TSharedRef<SWidget>
						{
							FMenuBuilder MenuBuilder(true, nullptr);
							MenuBuilder.BeginSection("SelectParameter", LOCTEXT("SelectParameter", "Select Parameter"));
							MenuBuilder.AddMenuEntry(
								LOCTEXT("SimpleSelectorLabel", "Simple"),
								LOCTEXT("SimpleSelectorTooltip", "Simple"),
								FSlateIcon(),
								FUIAction(
								FExecuteAction::CreateLambda([this]()
								{
									ViewerState->SetSelectedComplexity(ChaosVDCollisionComplexityFilteringOptions::Simple);
								}),
								FCanExecuteAction())
							);
							MenuBuilder.AddMenuEntry(
								LOCTEXT("ComplexSelectorLabel", "Complex"),
								LOCTEXT("ComplexSelectorTooltip", "Complex"),
								FSlateIcon(),
								FUIAction(
								FExecuteAction::CreateLambda([this]()
								{
									ViewerState->SetSelectedComplexity(ChaosVDCollisionComplexityFilteringOptions::Complex);
								}),
								FCanExecuteAction())
							);
							MenuBuilder.AddMenuEntry(
								LOCTEXT("AllSelectorLabel", "All"),
								LOCTEXT("AllSelectorTooltip", "All"),
								FSlateIcon(),
								FUIAction(
								FExecuteAction::CreateLambda([this]()
								{
									ViewerState->SetSelectedComplexity(ChaosVDCollisionComplexityFilteringOptions::All);
								}),
								FCanExecuteAction())
							);
							MenuBuilder.EndSection();

							return MenuBuilder.MakeWidget();
						})
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text_Lambda([this]()
							{
								switch (ViewerState->GetSelectedComplexity())
								{
									case ChaosVDCollisionComplexityFilteringOptions::Simple:
										return LOCTEXT("SimpleSelectorLabel", "Simple");
									case ChaosVDCollisionComplexityFilteringOptions::Complex:
										return LOCTEXT("ComplexSelectorLabel", "Complex");
									case ChaosVDCollisionComplexityFilteringOptions::All:
										return LOCTEXT("AllSelectorLabel", "All");
								}
								return LOCTEXT("SimpleSelectorLabel", "Simple");
							})
						]
					]	
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(6.0, 4.0, 6.0, 4.0))
				.AutoHeight()
				[
					SNew(SHorizontalBox)
					+SHorizontalBox::Slot()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("MetricSelectLabel","Selected Metric: "))
							
					]
					+SHorizontalBox::Slot()
					[
						SNew(SComboButton)
						.HasDownArrow(true)
						.ComboButtonStyle(FAppStyle::Get(), "ComboButton")
						.ContentPadding(FMargin(4.0f, 2.0f))
						.OnGetMenuContent_Lambda([this]() -> TSharedRef<SWidget>
						{
							FMenuBuilder MenuBuilder(true, nullptr);
							MenuBuilder.BeginSection("SelectParameter", LOCTEXT("SelectParameter", "Select Parameter"));
							MenuBuilder.AddMenuEntry(
								LOCTEXT("PrimitiveDensitySelectorLabel", "Primitive Density"),
								LOCTEXT("PrimitiveDensitySelectorTooltip", "Primitive Density"),
								FSlateIcon(),
								FUIAction(
								FExecuteAction::CreateLambda([this]()
								{
									ViewerState->SetSelectedMetric(ChaosVDParticleMetricsType::PrimitiveDensity);
								}),
								FCanExecuteAction())
							);
							MenuBuilder.AddMenuEntry(
								LOCTEXT("MemoryUsageSelectorLabel", "Memory Usage"),
								LOCTEXT("MemoryUsageSelectorTooltip", "Memory Usage"),
								FSlateIcon(),
								FUIAction(
								FExecuteAction::CreateLambda([this]()
								{
									ViewerState->SetSelectedMetric(ChaosVDParticleMetricsType::MemoryUsage);
								}),
								FCanExecuteAction())
							);
							MenuBuilder.EndSection();

							return MenuBuilder.MakeWidget();
						})
						.ButtonContent()
						[
							SNew(STextBlock)
							.Text_Lambda([this]()
							{
								switch (ViewerState->GetSelectedMetric())
								{
									case ChaosVDParticleMetricsType::PrimitiveDensity:
										return LOCTEXT("PrimitiveDensitySelectorLabel", "Primitive Density");
									case ChaosVDParticleMetricsType::MemoryUsage:
										return LOCTEXT("MemoryUsageSelectorLabel", "Memory Usage");
								}
								return LOCTEXT("SelectParameterEllipsis", "Select Metric ...");
							})
						]
					]
				]
				+ SVerticalBox::Slot()
				.Padding(FMargin(6.0, 4.0, 6.0, 4.0))
				.FillHeight(1)
				[
					SAssignNew(Metrics, SChaosVDMetricsView, InCVDScene, ViewerState)
				]
			]
			+SSplitter::Slot()
			.Value(0.4f)
			[
				SNew(SSplitter)
				.Orientation(EOrientation::Orient_Vertical)
				+ SSplitter::Slot()
				.Value(0.6f)
				[
					SNew(SBox)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(Heatmap, SCVDMetricsHeatmapView, InCVDScene, ViewerState)
					]
				]
				+ SSplitter::Slot()
				.Resizable(true)
				.Value(0.4f)
				[
					SNew(SBox)
					.HAlign(HAlign_Fill)
					.VAlign(VAlign_Fill)
					[
						SAssignNew(Histogram, SCVDMetricsHistogramPanelView, ViewerState)
					]
				]
			]
		]
	];
}

void SPerfMetricsViewer::SetTargetSolverID(FTargetSolverEntry NewTarget)
{
	CurrentTargetSolver = NewTarget;
}

void SPerfMetricsViewer::AnalyzeCurrentFrame(TSharedPtr<TArray<TSharedPtr<FParticleMetricEntry>>>& OutMetrics, TWeakPtr<FChaosVDScene> WeakCVDScenePtr, int32 SolverID)
{
	TSharedPtr<FChaosVDScene> CVDScene = WeakCVDScenePtr.Pin();
	if (!CVDScene || !OutMetrics)
	{
		return;
	}

	AChaosVDSolverInfoActor* SolverInfo = CVDScene->GetSolverInfoActor(SolverID);
	if (UChaosVDParticleDataComponent* ParticleDataComponent = SolverInfo ? SolverInfo->GetParticleDataComponent() : nullptr)
	{
		ParticleDataComponent->VisitAllParticleInstances([OutMetrics, CVDScene](const TSharedRef<FChaosVDSceneParticle>& InParticleInstance)
		{
			if (InParticleInstance->IsActive())
			{
				FParticleMetricEntry Entry;
				ChaosVDMetrics::CalculateMetrics(InParticleInstance, CVDScene->AsWeak(), Entry);
				OutMetrics->Add(MakeShared<FParticleMetricEntry>(Entry));
			}
			return true;
		});
	}

	return;
}

void SPerfMetricsViewer::HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController)
{
	bAbortPendingAnalysis = true;

	SetTargetSolverID(FTargetSolverEntry());
	
	if (ViewerState)
	{
		ViewerState->SetParticleMetrics(MakeShared<TArray<TSharedPtr<FParticleMetricEntry>>>());
	}

	Metrics->ApplyFiltering();
	Heatmap->RefreshHeatmapView();
	Histogram->UpdateHistogram();
}

TSharedRef<SWidget> SPerfMetricsViewer::GenerateSolverSelectorMenu()
{
	FMenuBuilder MenuBuilder(true, nullptr);
	MenuBuilder.BeginSection("CVDComplexityTargetSelector", LOCTEXT("CVDComplexityTargetSelectorMenu", "Available Solvers"));
	{
		if (TSharedPtr<FChaosVDScene> ScenePtr = WeakCVDScene.Pin())
		{
			for (const TPair<int32, AChaosVDSolverInfoActor*>& SolverWithID : ScenePtr->GetSolverInfoActorsMap())
			{
				if (AChaosVDSolverInfoActor* SolverInfo = SolverWithID.Value)
				{
					FTargetSolverEntry SolverEntry;
					SolverEntry.Name = FText::AsCultureInvariant(SolverInfo->GetSolverName().ToString());
					SolverEntry.ID = SolverWithID.Key;
					MenuBuilder.AddMenuEntry(
					SolverEntry.Name,
					FText::GetEmpty(),
					FSlateIcon(),
					FUIAction(FExecuteAction::CreateSP(this, &SPerfMetricsViewer::SetTargetSolverID, SolverEntry), EUIActionRepeatMode::RepeatDisabled));
				}
			}
		}
	}
	
	MenuBuilder.EndSection();
	return MenuBuilder.MakeWidget();
}

TSharedRef<SWidget> SPerfMetricsViewer::GenerateMainToolbarWidget()
{
	RegisterMainToolbarMenu();
	FToolMenuContext MenuContext;
	UChaosVDPerfMetricsViewerToolbarMenuContext* CommonContextObject = NewObject<UChaosVDPerfMetricsViewerToolbarMenuContext>();
	CommonContextObject->ViewerInstance = SharedThis(this);
	MenuContext.AddObject(CommonContextObject);
	return UToolMenus::Get()->GenerateWidget(ToolBarName, MenuContext);
}

void SPerfMetricsViewer::RegisterMainToolbarMenu()
{
	UToolMenus* ToolMenus = UToolMenus::Get();
	
	UToolMenu* ToolBar = ToolMenus->FindMenu(ToolBarName);
	if (!ToolBar || !ToolMenus->IsMenuRegistered(ToolBarName))
	{
		ToolBar = UToolMenus::Get()->RegisterMenu(ToolBarName, NAME_None, EMultiBoxType::SlimHorizontalToolBar);
	}
	
	ToolBar->Sections.Empty();

	FToolMenuSection& Section = ToolBar->AddSection(TEXT("ComplexityViewer.Toolbar"));
	TAttribute<FText> SolverSelectorButtonLabel;
	SolverSelectorButtonLabel.Bind(this, &SPerfMetricsViewer::GetSolverSelectorButtonLabel);
	Section.AddEntry(FToolMenuEntry::InitComboButton("TargetSolverDataSelector", FUIAction(), FOnGetContent::CreateSP(this, &SPerfMetricsViewer::GenerateSolverSelectorMenu), SolverSelectorButtonLabel, LOCTEXT("SolverSelectorButtonToolTip", "Select from which solver we should generate the complexity view.")));
	FCanExecuteAction CanAnalyze = FCanExecuteAction::CreateSPLambda(this, [this]()
	{
		return CurrentTargetSolver.ID != INDEX_NONE && !bProcessingTask;
	});

	const FUIAction AnalyzeFrameAction = FUIAction(
	FExecuteAction::CreateSPLambda(this, [this](){
		if (bProcessingTask)
		{
			return;
		}

		bProcessingTask = true;
		bAbortPendingAnalysis = false;

		TWeakPtr<SPerfMetricsViewer> WeakMetricsViewerPtr = SharedThis(this);
		TWeakPtr<FChaosVDScene> WeakCVDScenePtr = WeakCVDScene;

		int32 SolverID = CurrentTargetSolver.ID;

		Async(EAsyncExecution::ThreadPool, [SolverID, WeakMetricsViewerPtr, WeakCVDScenePtr]()
		{
			TSharedPtr<TArray<TSharedPtr<FParticleMetricEntry>>> MetricsPtr = MakeShared<TArray<TSharedPtr<FParticleMetricEntry>>>();
			SPerfMetricsViewer::AnalyzeCurrentFrame(MetricsPtr, WeakCVDScenePtr, SolverID);

			AsyncTask(ENamedThreads::GameThread, [WeakMetricsViewerPtr, MetricsPtr]()
			{
				if (TSharedPtr<SPerfMetricsViewer> MetricsViewer = WeakMetricsViewerPtr.Pin())
				{
					// If a new recording is being loaded while the old analysis finishes, discard the results.
					if (!MetricsViewer->bAbortPendingAnalysis)
					{
						MetricsViewer->ViewerState->SetParticleMetrics(MetricsPtr);
						MetricsViewer->Metrics->ApplyFiltering();
						MetricsViewer->Heatmap->RefreshHeatmapView();
						MetricsViewer->Histogram->UpdateHistogram();
					}
					MetricsViewer->bAbortPendingAnalysis = false;
					MetricsViewer->bProcessingTask = false;
				}
			});
		});

		
	}), CanAnalyze, EUIActionRepeatMode::RepeatDisabled);

	Section.AddEntry(FToolMenuEntry::InitToolBarButton("AnalyzeFrameButton", AnalyzeFrameAction, 
		TAttribute<FText>::CreateLambda([this]()
		{
			return bProcessingTask ? LOCTEXT("AnalyzeFrameButtonProcessingLabel", "Processing... ") : LOCTEXT("AnalyzeFrameButtonLabel", "Analyze Current Frame");
		}),
		LOCTEXT("AnalyzeFrameButtonToolTip", "Analyzes the current loaded frame and generates a complexity report")));
}

FText SPerfMetricsViewer::GetSolverSelectorButtonLabel() const
{
	return CurrentTargetSolver.ID == INDEX_NONE ? LOCTEXT("SolverSelectorButtonDefaultLabel", "Select a Solver...") : CurrentTargetSolver.Name;
}

#undef LOCTEXT_NAMESPACE