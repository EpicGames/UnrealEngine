// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once
#include "Chaos/AABBTree.h"
#include "Widgets/SCompoundWidget.h"
#include "Chaos/ImplicitFwd.h"
#include "PerformanceMetrics/ChaosVDMetrics.h"

#include "SPerfMetricsViewer.generated.h"

class FChaosVDScene;
class SChaosVDMainTab;
class FChaosVDPlaybackController;
class SButton;
class SCVDMetricsHistogramPanelView;
class SCVDMetricsHeatmapView;
class FChaosVDMetricsViewerState;
class SChaosVDMetricsView;

class SPerfMetricsViewer : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SPerfMetricsViewer)
		{}
	SLATE_END_ARGS()

	void Construct(const FArguments& InArgs, const TWeakPtr<FChaosVDScene>& InCVDScene, const TWeakPtr<SChaosVDMainTab>& InWeakMainTab);
	static void AnalyzeCurrentFrame(TSharedPtr<TArray<TSharedPtr<FParticleMetricEntry>>>& OutMetrics, TWeakPtr<FChaosVDScene> WeakCVDScenePtr, int32 SolverID);

	struct FTargetSolverEntry
	{
		int32 ID = INDEX_NONE;
		FText Name;
	};
	void SetTargetSolverID(FTargetSolverEntry NewTarget);
	void HandleRecordingFirstFrameLoaded(TWeakPtr<FChaosVDPlaybackController> InController);
private:
	
	bool bProcessingTask = false;
	bool bAbortPendingAnalysis = false;

	TWeakPtr<FChaosVDScene> WeakCVDScene;
	TWeakPtr<SChaosVDMainTab> WeakMainTab;
	TSharedPtr<SButton> RefreshButton;
	TSharedPtr<SChaosVDMetricsView> Metrics;
	TSharedPtr<SCVDMetricsHeatmapView> Heatmap;
	TSharedPtr<SCVDMetricsHistogramPanelView> Histogram;
	static FName ToolBarName;
	FTargetSolverEntry CurrentTargetSolver;

	TSharedPtr<FChaosVDMetricsViewerState> ViewerState;

	ChaosVDCollisionComplexityFilteringOptions SelectedComplexity = ChaosVDCollisionComplexityFilteringOptions::Simple;
	ChaosVDParticleMetricsType SelectedMetric = ChaosVDParticleMetricsType::PrimitiveDensity;

	void RegisterMainToolbarMenu();
	TSharedRef<SWidget> GenerateMainToolbarWidget();
	TSharedRef<SWidget> GenerateSolverSelectorMenu();
	FText GetSolverSelectorButtonLabel() const;

};

UCLASS()
class UChaosVDPerfMetricsViewerToolbarMenuContext : public UObject
{
	GENERATED_BODY()
public:
	TWeakPtr<SPerfMetricsViewer> ViewerInstance;
};