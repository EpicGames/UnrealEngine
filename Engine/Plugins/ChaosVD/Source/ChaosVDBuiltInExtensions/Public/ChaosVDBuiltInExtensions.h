// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "ExtensionsSystem/ChaosVDExtensionsManager.h"
#include "Modules/ModuleManager.h"

class FChaosVDExtension;

class FChaosVDBuiltInExtensionsModule : public IModuleInterface
{
public:
    virtual void StartupModule() override;
    virtual void ShutdownModule() override;

private:
	template <typename ExtensionType>
	void CreateAndRegisterExtensionInstance();
	
	void UnregisterCreatedExtensions();
	
	TArray<TWeakPtr<FChaosVDExtension>> AvailableExtensions;
};

template <typename ExtensionType>
void FChaosVDBuiltInExtensionsModule::CreateAndRegisterExtensionInstance()
{
	TSharedRef<ExtensionType> NewExtension = MakeShared<ExtensionType>();
	FChaosVDExtensionsManager::Get().RegisterExtension(NewExtension);
	AvailableExtensions.Add(NewExtension);
}
